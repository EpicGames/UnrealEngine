[CoreRedirects]

; Trajectory type redirects
+StructRedirects=(OldName="/Script/PoseSearch.PoseSearchQueryTrajectory", NewName="/Script/Engine.TransformTrajectory")
+StructRedirects=(OldName="/Script/PoseSearch.PoseSearchQueryTrajectorySample", NewName="/Script/Engine.TransformTrajectorySample")

; Trajectory variables
+PropertyRedirects=(OldName="/Script/PoseSearch.PoseSearchQueryTrajectory.Samples", NewName="/Script/Engine.TransformTrajectory.Samples")

; Trajectory Sample variables
+PropertyRedirects=(OldName="/Script/PoseSearch.PoseSearchQueryTrajectorySample.Position", NewName="/Script/Engine.TransformTrajectorySample.Position")
+PropertyRedirects=(OldName="/Script/PoseSearch.PoseSearchQueryTrajectorySample.Facing", NewName="/Script/Engine.TransformTrajectorySample.Facing")
+PropertyRedirects=(OldName="/Script/PoseSearch.PoseSearchQueryTrajectorySample.AccumulatedSeconds", NewName="/Script/Engine.TransformTrajectorySample.TimeInSeconds")

; Fix up all pose search trajectory library functions
+FunctionRedirects=(OldName="PoseSearchTrajectoryLibrary.PoseSearchGenerateTrajectory", NewName="PoseSearchTrajectoryLibrary.PoseSearchGenerateTransformTrajectory")
+FunctionRedirects=(OldName="PoseSearchTrajectoryLibrary.PoseSearchGeneratePredictorTrajectory", NewName="PoseSearchTrajectoryLibrary.PoseSearchGeneratePredictorTransformTrajectory")
+FunctionRedirects=(OldName="PoseSearchTrajectoryLibrary.PoseSearchGenerateTrajectoryWithPredictor", NewName="PoseSearchTrajectoryLibrary.PoseSearchGenerateTransformTrajectoryWithPredictor")
+FunctionRedirects=(OldName="PoseSearchTrajectoryLibrary.HandleTrajectoryWorldCollisions", NewName="PoseSearchTrajectoryLibrary.HandleTransformTrajectoryWorldCollisions")
+FunctionRedirects=(OldName="PoseSearchTrajectoryLibrary.HandleTrajectoryWorldCollisionsWithGravity", NewName="PoseSearchTrajectoryLibrary.HandleTransformTrajectoryWorldCollisionsWithGravity")
+FunctionRedirects=(OldName="PoseSearchTrajectoryLibrary.GetTrajectorySampleAtTime", NewName="PoseSearchTrajectoryLibrary.GetTransformTrajectorySampleAtTime")
+FunctionRedirects=(OldName="PoseSearchTrajectoryLibrary.GetTrajectoryVelocity", NewName="PoseSearchTrajectoryLibrary.GetTransformTrajectoryVelocity")
+FunctionRedirects=(OldName="PoseSearchTrajectoryLibrary.GetTrajectoryAngularVelocity", NewName="PoseSearchTrajectoryLibrary.GetTransformTrajectoryAngularVelocity")
+FunctionRedirects=(OldName="PoseSearchTrajectoryLibrary.GetTransform", NewName="PoseSearchTrajectoryLibrary.GetTransformTrajectorySampleTransform")
+FunctionRedirects=(OldName="PoseSearchTrajectoryLibrary.DrawTrajectory", NewName="PoseSearchTrajectoryLibrary.DrawTransformTrajectory")

; Fix up all pose search history collector node library functions
+FunctionRedirects=(OldName="PoseSearchHistoryCollectorAnimNodeLibrary.GetPoseHistoryNodeTrajectory", NewName="PoseSearchHistoryCollectorAnimNodeLibrary.GetPoseHistoryNodeTransformTrajectory")
+FunctionRedirects=(OldName="PoseSearchHistoryCollectorAnimNodeLibrary.SetPoseHistoryNodeTrajectory", NewName="PoseSearchHistoryCollectorAnimNodeLibrary.SetPoseHistoryNodeTransformTrajectory")

; Fix up pose history node and PoseSearchBlueprintResult stuff
+PropertyRedirects=(OldName="AnimNode_PoseSearchHistoryCollector_Base.Trajectory", NewName="AnimNode_PoseSearchHistoryCollector_Base.TransformTrajectory")
+PropertyRedirects=(OldName="AnimNode_PoseSearchHistoryCollector.Trajectory", NewName="AnimNode_PoseSearchHistoryCollector.TransformTrajectory")
+PropertyRedirects=(OldName="PoseSearchBlueprintResult.SelectedAnimation", NewName="PoseSearchBlueprintResult.SelectedAnim")