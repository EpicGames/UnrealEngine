{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Pose Search",
	"Description": "Framework for indexing and searching pose features. Used in techniques such as Motion Matching.",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "PoseSearch",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "PoseSearchEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "AnimationWarping",
			"Enabled": true
		},
		{
			"Name": "BlendStack",
			"Enabled": true
		},
		{
			"Name": "Chooser",
			"Enabled": true
		},
		{
			"Name": "GameplayInsights",
			"Enabled": true,
			"TargetAllowList": [
				"Editor"
			]
		},
		{
			"Name": "MotionWarping",
			"Enabled": true
		},
		{
			"Name": "RigVM",
			"Enabled": true
		}
	]
}