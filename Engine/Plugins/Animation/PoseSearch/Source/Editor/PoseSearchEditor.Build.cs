// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class PoseSearchEditor : ModuleRules
{
	public PoseSearchEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"BlendStackEditor",
				"BlendStack",
				"AnimGraph",
				"AnimGraphRuntime",
				"AnimationBlueprintLibrary",
				"AnimationCore",
				"Core",
				"CoreUObject",
				"Engine",
				"PoseSearch",
				"Chooser",
				
				// Trace-related dependencies
				"TraceLog",
				"TraceAnalysis",
				"TraceServices",
				"TraceInsights",
				"GameplayInsights",
				
				// UI 
				"PropertyEditor",
				"Slate",
				"SlateCore",
				"RewindDebuggerInterface",
				"UnrealEd",
				"InputCore",
				"ChooserEditor"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"AnimationEditor",
				"AssetDefinition",
				"BlueprintGraph",
				"EditorStyle",
				"DetailCustomizations",
				"AdvancedPreviewScene",
				"EditorFramework",
				"ToolWidgets"
			}
		);
	}
}
