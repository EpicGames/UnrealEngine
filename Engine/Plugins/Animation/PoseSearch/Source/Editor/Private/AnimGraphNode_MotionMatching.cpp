// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_MotionMatching.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Animation/AnimRootMotionProvider.h"
#include "FindInBlueprintManager.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_MotionMatching)

#define LOCTEXT_NAMESPACE "AnimGraphNode_MotionMatching"


FLinearColor UAnimGraphNode_MotionMatching::GetNodeTitleColor() const
{
	return FColor(86, 182, 194);
}

FText UAnimGraphNode_MotionMatching::GetTooltipText() const
{
	return LOCTEXT("NodeToolTip", "Motion Matching");
}

FText UAnimGraphNode_MotionMatching::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FText Title = LOCTEXT("NodeTitle", "Motion Matching");
	AddSyncGroupToNodeTitle(TitleType, Title);
	return Title;
}

FText UAnimGraphNode_MotionMatching::GetMenuCategory() const
{
	return LOCTEXT("NodeCategory", "Pose Search");
}

UScriptStruct* UAnimGraphNode_MotionMatching::GetTimePropertyStruct() const
{
	return FAnimNode_MotionMatching::StaticStruct();
}

void UAnimGraphNode_MotionMatching::OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	Super::OnProcessDuringCompilation(InCompilationContext, OutCompiledData);
	Node.OnMotionMatchingStateUpdated.SetFromFunction(OnMotionMatchingStateUpdatedFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()));
}

void UAnimGraphNode_MotionMatching::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	ValidateFunctionRef(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_MotionMatching, OnMotionMatchingStateUpdatedFunction), OnMotionMatchingStateUpdatedFunction, LOCTEXT("OnMotionMatchingStateUpdatedFunctionName", "On Motion Matching State Updated"), MessageLog);
}

void UAnimGraphNode_MotionMatching::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimGraphNode_MotionMatching, OnMotionMatchingStateUpdatedFunction))
	{
		Node.OnMotionMatchingStateUpdated.SetFromFunction(OnMotionMatchingStateUpdatedFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()));
		GetAnimBlueprint()->RequestRefreshExtensions();
		GetSchema()->ReconstructNode(*this);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UAnimGraphNode_MotionMatching::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);
	DetailBuilder.EditCategory("Sync").SetToolTip(LOCTEXT("CategoryToolTip", "Motion Matching is intended to only act as a leader in a sync group. Internally, only the most recent blend stack sample will have sync information for other asset players to follow and the rest of blend stack samples will tick without syncing."));
	
	auto HeaderContentWidget = SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign(HAlign_Center)
	.Padding(4.0f, 4.0f)
	.VAlign(VAlign_Center)
	[
		SNew(STextBlock)
		.Text(LOCTEXT("SyncTextLabel", "Sync"))
		.Font(IDetailLayoutBuilder::GetDetailFontBold())
		.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
	]
	
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	[
		SNew(SImage)
		.DesiredSizeOverride(FVector2D(13.f, 13.f))
		.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		.Image(FAppStyle::Get().GetBrush("Icons.Info"))
		.ToolTipText(LOCTEXT("SyncCategoryToolTip", "Motion Matching is intended to only act as a leader in a sync group. Internally, only the most recent blend stack sample will have sync information for other asset players to follow and the rest of blend stack samples will tick without syncing."))
	];
	
	DetailBuilder.EditCategory("Sync").HeaderContent(HeaderContentWidget, true);
	DetailBuilder.EditCategory("Sync").SetSortOrder(0);
}

void UAnimGraphNode_MotionMatching::AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddSearchMetaDataInfo(OutTaggedMetaData);

	const auto ConditionallyTagNodeFuncRef = [&OutTaggedMetaData](const FMemberReference& FuncMember, const FText& LocText)
	{
		if (IsPotentiallyBoundFunction(FuncMember))
		{
			const FText FunctionName = FText::FromName(FuncMember.GetMemberName());
			OutTaggedMetaData.Add(FSearchTagDataPair(LocText, FunctionName));
		}
	};

	// Conditionally include anim node function references as part of the node's search metadata
	ConditionallyTagNodeFuncRef(OnMotionMatchingStateUpdatedFunction, LOCTEXT("OnMotionMatchingStateUpdatedFunctionName", "On Motion Matching State Updated"));
}

void UAnimGraphNode_MotionMatching::GetBoundFunctionsInfo(TArray<TPair<FName, FName>>& InOutBindingsInfo)
{
	Super::GetBoundFunctionsInfo(InOutBindingsInfo);
	const FName CategoryName = TEXT("Functions|Motion Matching");

	if (OnMotionMatchingStateUpdatedFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()) != nullptr)
	{
		InOutBindingsInfo.Emplace(CategoryName, GET_MEMBER_NAME_CHECKED(UAnimGraphNode_MotionMatching, OnMotionMatchingStateUpdatedFunction));
	}
}

bool UAnimGraphNode_MotionMatching::ReferencesFunction(const FName& InFunctionName, const UStruct* InScope) const
{
	return Super::ReferencesFunction(InFunctionName, InScope)
		|| Node.OnMotionMatchingStateUpdated.GetFunctionName() == InFunctionName;
}

#undef LOCTEXT_NAMESPACE
