// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_PoseSearchHistoryCollector.h"
#include "AnimationGraphSchema.h"
#include "AnimGraphNode_Base.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_PoseSearchHistoryCollector)

#define LOCTEXT_NAMESPACE "AnimGraphNode_PoseSearchHistoryCollector"

/////////////////////////////////////////////////////
// UAnimGraphNode_PoseSearchHistoryCollector_Base

FLinearColor UAnimGraphNode_PoseSearchHistoryCollector_Base::GetNodeTitleColor() const
{
	return FColor(86, 182, 194);
}

FText UAnimGraphNode_PoseSearchHistoryCollector_Base::GetTooltipText() const
{
	return LOCTEXT("NodeToolTip", "Collects bones transforms for motion matching");
}

FText UAnimGraphNode_PoseSearchHistoryCollector_Base::GetMenuCategory() const
{
	return LOCTEXT("NodeCategory", "Pose Search");
}

void UAnimGraphNode_PoseSearchHistoryCollector_Base::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FName OldPropertyName = GET_MEMBER_NAME_CHECKED(FAnimNode_PoseSearchHistoryCollector_Base, Trajectory);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	if (HasBinding(OldPropertyName))
	{
		MessageLog.Error(*LOCTEXT("OldTrajectoryType", "In node (@@), please manually re-bind pin \"Trajectory\" to it's respective variable now that FPoseSearchQueryTrajectory has been deprecated").ToString(), this);
	}
}

void UAnimGraphNode_PoseSearchHistoryCollector_Base::PostReconstructNode()
{
	Super::PostReconstructNode();

#if WITH_EDITOR
	if (!IsTemplate())
	{
		// Make sure we're not dealing with a menu node
		UEdGraph* OuterGraph = GetGraph();
		if (OuterGraph && OuterGraph->Schema)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			const FName OldPropertyName = GET_MEMBER_NAME_CHECKED(FAnimNode_PoseSearchHistoryCollector_Base, Trajectory);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			const FName NewPropertyName = GET_MEMBER_NAME_CHECKED(FAnimNode_PoseSearchHistoryCollector_Base, TransformTrajectory);

			// Fix-up binding(s) to properly redirect from our old member variable, with type FPoseSearchQueryTrajectory, to the new one, with type FTransformTrajectory.
			if (HasBinding(OldPropertyName))
			{
				if (RedirectBinding(OldPropertyName, NewPropertyName))
				{
					// Let the graph know to refresh
					GetGraph()->NotifyNodeChanged(this);

					UBlueprint* Blueprint = GetBlueprint();
					if (!Blueprint->bBeingCompiled)
					{
						FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
					}
				}
			}
		}
	}
#endif
}

UK2Node::ERedirectType UAnimGraphNode_PoseSearchHistoryCollector_Base::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	// @note: Once the "Trajectory" member variable is removed, we'd have to manually create an FCoreRedirectObjectName that **also** contains
	// the class type in the string and not just the pin name to prevent our redirector(s) from failing. We are avoiding it rn, since that is not as simple
	// forcing the redirect type.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FName OldPropertyName = GET_MEMBER_NAME_CHECKED(FAnimNode_PoseSearchHistoryCollector_Base, Trajectory);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	const FName NewPropertyName = GET_MEMBER_NAME_CHECKED(FAnimNode_PoseSearchHistoryCollector_Base, TransformTrajectory);
	
	// Old pin, FPoseSearchQueryTrajectory, input can't be linked to new pin, FTransformTrajectory. CoreRedirects doesn't seem to help here so we have to manually redirect it.
	if (OldPin->PinName == OldPropertyName && NewPin->PinName == NewPropertyName)
	{
		return ERedirectType_Value;
	}

	return Super::DoPinsMatchForReconstruction(NewPin, NewPinIndex, OldPin, OldPinIndex);
}

/////////////////////////////////////////////////////
// UAnimGraphNode_PoseSearchHistoryCollector

FText UAnimGraphNode_PoseSearchHistoryCollector::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Pose History");
}

/////////////////////////////////////////////////////
// UAnimGraphNode_PoseSearchComponentSpaceHistoryCollector

FText UAnimGraphNode_PoseSearchComponentSpaceHistoryCollector::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitleComponentSpace", "Component Space Pose History");
}

void UAnimGraphNode_PoseSearchComponentSpaceHistoryCollector::CreateOutputPins()
{
	CreatePin(EGPD_Output, UAnimationGraphSchema::PC_Struct, FComponentSpacePoseLink::StaticStruct(), TEXT("Pose"));
}

#undef LOCTEXT_NAMESPACE
