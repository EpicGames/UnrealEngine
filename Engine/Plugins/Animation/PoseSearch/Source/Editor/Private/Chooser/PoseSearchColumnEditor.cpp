// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchColumnEditor.h"
#include "Animation/AnimationAsset.h"
#include "ChooserColumnHeader.h"
#include "ContentBrowserModule.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "IContentBrowserSingleton.h"
#include "ObjectChooserWidgetFactories.h"
#include "PoseSearch/Chooser/PoseSearchChooserColumn.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "ScopedTransaction.h"
#include "SPropertyAccessChainWidget.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "PoseSearchColumnEditor"

namespace UE::PoseSearchEditor
{

TSharedRef<SWidget> CreatePoseSearchColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int RowIndex)
{
	using namespace UE::ChooserEditor;

	FPoseSearchColumn* PoseSearchColumn = static_cast<FPoseSearchColumn*>(Column);

	if (RowIndex == UE::ChooserEditor::ColumnWidget_SpecialIndex_Header)
	{
		// create column header widget
		const FSlateBrush* ColumnIcon = FAppStyle::Get().GetBrush("Icons.Search");
		const FText ColumnTooltip = LOCTEXT("Pose Match Tooltip", "Pose Match: Selects results based on the animations with the best matching poses scored via this column \"Internal Database\", and outputs the StartTime of the best matching animation. AutoPopulate will fill in Column data in case it mismatch the Chooser Table \"Result\" column.");
		const FText ColumnName = LOCTEXT("Pose Match", "Pose Match");

		TSharedPtr<SWidget> DebugWidget = nullptr;

		return UE::ChooserEditor::MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget);
	}

	// create cell widget
	TSharedRef<SComboButton> Button = SNew(SComboButton)
		.ContentPadding(0)
		.ButtonContent()
		[
			SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text_Lambda([PoseSearchColumn, RowIndex]()
					{
						if (UPoseSearchDatabase* Database = PoseSearchColumn->GetDatabase(RowIndex))
						{
							return FText::FromString(Database->GetName());
						}
						return LOCTEXT("None", "None");
					})
		];

	Button->SetOnGetMenuContent(
		FOnGetContent::CreateLambda(
			[Button, Chooser, PoseSearchColumn, RowIndex]()
			{
				FMenuBuilder MenuBuilder(true, nullptr);

				MenuBuilder.BeginSection(NAME_None, LOCTEXT("DatabaseSelection", "Database"));
				{
					MenuBuilder.AddMenuEntry(
						LOCTEXT("None", "None"),
						LOCTEXT("NoDatabase_Tooltip", "Disable this row to be indexed in any Database"),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([Button, Chooser, PoseSearchColumn, RowIndex]()
							{
								const FScopedTransaction Transaction(LOCTEXT("Clear Database From Row", "Clear Database"));
								Chooser->Modify();
								Button->SetIsOpen(false);
								PoseSearchColumn->SwitchDatabase(nullptr, RowIndex);
								Chooser->PostEditChange();
							})));

					MenuBuilder.AddWidget(
						SNew(SBox)
							.ToolTipText(LOCTEXT("NewDatabase_Tooltip", "Create a new database used to index this row asset"))
							[
								SNew(SEditableTextBox)
									.Font(IDetailLayoutBuilder::GetDetailFont())
									.HintText(LOCTEXT("NewDatabaseHint", "Enter database name..."))
									.OnTextCommitted_Lambda([Button, Chooser, PoseSearchColumn, RowIndex](const FText& InText, ETextCommit::Type InCommitType)
										{
											if (InCommitType == ETextCommit::OnEnter)
											{
												Button->SetIsOpen(false);

												FAssetReferenceFilterContext AssetReferenceFilterContext;
												UObject* ChooserAsObject = Chooser;
												AssetReferenceFilterContext.AddReferencingAssets({&ChooserAsObject, 1});
												TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = GEditor ? GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext) : nullptr;

												// picking a schema for the newly created database
												FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

												FAssetPickerConfig AssetPickerConfig;
												TSharedPtr<SWindow> PickerWindow;
												UPoseSearchSchema* TargetSchema = nullptr;

												// The asset picker will only show UPoseSearchSchema(s)
												AssetPickerConfig.Filter.ClassPaths.Add(UPoseSearchSchema::StaticClass()->GetClassPathName());
												AssetPickerConfig.Filter.bRecursiveClasses = true;

												// The delegate that fires when an asset was selected
												AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda([&PickerWindow, &TargetSchema](const FAssetData& SelectedAsset)
													{
														TargetSchema = Cast<UPoseSearchSchema>(SelectedAsset.GetAsset());
														PickerWindow->RequestDestroyWindow();
													});

												// The default view mode should be a list view
												AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

												AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda([AssetReferenceFilter](const FAssetData& AssetData)
													{
														if (AssetReferenceFilter.IsValid())
														{
															return !AssetReferenceFilter->PassesFilter(AssetData);
														}
														return false;
													});

												PickerWindow = SNew(SWindow)
													.Title(LOCTEXT("CreateDatabaseOptions", "Pick Database Schema"))
													.ClientSize(FVector2D(500, 600))
													.SupportsMinimize(false)
													.SupportsMaximize(false)
													[
														SNew(SBorder)
															.BorderImage(FAppStyle::GetBrush("Menu.Background"))
															[
																ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
															]
													];

												GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
												PickerWindow.Reset();

												if (TargetSchema)
												{
													FName NewDatabaseName = FName(InText.ToString());
													UChooserTable* RootChooser = Chooser->GetRootChooser();
													if (NewDatabaseName == NAME_None || FindObject<UObject>(Chooser, NewDatabaseName.ToString()))
													{
														NewDatabaseName = MakeUniqueObjectName(Chooser, UChooserTable::StaticClass(), NewDatabaseName);
													}

													UPoseSearchDatabase* NewDatabase = NewObject<UPoseSearchDatabase>(RootChooser, UPoseSearchDatabase::StaticClass(), NewDatabaseName, RF_Transactional);
													NewDatabase->Schema = TargetSchema;
													// KDTree frequently fails in Chooser columns so use Brute Force instead.
													NewDatabase->PoseSearchMode = EPoseSearchMode::BruteForce;

													const FScopedTransaction Transaction(LOCTEXT("Assign New Database", "Assign New Database"));
													RootChooser->Modify();
													RootChooser->AddNestedObject(NewDatabase);

													PoseSearchColumn->SwitchDatabase(NewDatabase, RowIndex);
													RootChooser->PostEditChange();
												}
											}
										})
							],
						LOCTEXT("NewDatabaseLabel", "New"),
						/*bNoIndent=*/true);

				}

				MenuBuilder.BeginSection("Existing", LOCTEXT("Existing Databases", "Select Existing"));
				{
					if (UChooserTable* RootChooser = Chooser->GetRootChooser())
					{
						for (UObject* Object : RootChooser->NestedObjects)
						{
							if (UPoseSearchDatabase* Database = Cast<UPoseSearchDatabase>(Object))
							{
								MenuBuilder.AddMenuEntry(FText::FromString(Database->GetName()), LOCTEXT("AddExistingObjectTooltip", "Add a reference to this existing Database."), FSlateIcon(),
									FUIAction(FExecuteAction::CreateLambda([Button, Chooser, PoseSearchColumn, Database, RowIndex]()
										{
											const FScopedTransaction Transaction(LOCTEXT("Set Database", "Set Database"));
											Chooser->Modify();
											Button->SetIsOpen(false);
											PoseSearchColumn->SwitchDatabase(Database, RowIndex);
											Chooser->PostEditChange();
										})));
							}
						}
					}
				}

				return MenuBuilder.MakeWidget();

			})
	);

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot().FillWidth(1.0)
		[
			Button
		]
		+ SHorizontalBox::Slot().AutoWidth()
		[
			SNew(SButton)
				.Text(LOCTEXT("Edit", "Edit"))
				.OnClicked_Lambda([PoseSearchColumn, RowIndex]()
					{
						// Open editor
						if (UPoseSearchDatabase* Database = PoseSearchColumn->GetDatabase(RowIndex))
						{
							if (UAssetEditorSubsystem* AssetEditorSS = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
							{
								AssetEditorSS->OpenEditorForAsset(Database);
							}
						}

						return FReply::Handled();
					})
		];
}


void RegisterPoseSearchChooserWidgets()
{
	UE::ChooserEditor::FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FPoseSearchColumn::StaticStruct(), CreatePoseSearchColumnWidget);
}

} // UE::PoseSearchEditor

#undef LOCTEXT_NAMESPACE
