// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchAssetDefinitions.h"
#include "PoseSearchDatabaseEditor.h"
#include "PoseSearchInteractionAssetEditor.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchAssetDefinitions)

namespace UE::PoseSearch
{

FLinearColor GetAssetColor()
{
	static const FLinearColor AssetColor(FColor(29, 96, 125));
	return AssetColor;
}

TConstArrayView<FAssetCategoryPath> GetAssetCategories()
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Animation, NSLOCTEXT("PoseSearchAssetDefinition", "PoseSearchAssetDefinitionMenu", "Motion Matching")) };
	return Categories;
}

UThumbnailInfo* LoadThumbnailInfo(const FAssetData & InAssetData)
{
	return UE::Editor::FindOrCreateThumbnailInfo(InAssetData.GetAsset(), USceneThumbnailInfo::StaticClass());
}

} // namespace UE::PoseSearch

EAssetCommandResult UAssetDefinition_PoseSearchDatabase::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	using namespace UE::PoseSearch;

	TArray<UPoseSearchDatabase*> Objects = OpenArgs.LoadObjects<UPoseSearchDatabase>();
	const EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;
	for (UPoseSearchDatabase* Database : Objects)
	{
		if (Database)
		{
			TSharedRef<FDatabaseEditor> NewEditor(new FDatabaseEditor());
			NewEditor->InitAssetEditor(Mode, OpenArgs.ToolkitHost, Database);
		}
	}
	
	return EAssetCommandResult::Handled;
}

EAssetCommandResult UAssetDefinition_PoseSearchInteractionAsset::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	using namespace UE::PoseSearch;

	TArray<UPoseSearchInteractionAsset*> Objects = OpenArgs.LoadObjects<UPoseSearchInteractionAsset>();
	const EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;
	for (UPoseSearchInteractionAsset* InteractionAsset : Objects)
	{
		if (InteractionAsset)
		{
			TSharedRef<FInteractionAssetEditor> NewEditor(new FInteractionAssetEditor());
			NewEditor->InitAssetEditor(Mode, OpenArgs.ToolkitHost, InteractionAsset);
		}
	}
	
	return EAssetCommandResult::Handled;
}
