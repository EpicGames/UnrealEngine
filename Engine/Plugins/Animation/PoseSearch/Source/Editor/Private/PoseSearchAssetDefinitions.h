// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchInteractionAsset.h"
#include "PoseSearch/PoseSearchNormalizationSet.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearchAssetDefinitions.generated.h"

namespace UE::PoseSearch
{
	FLinearColor GetAssetColor();
	TConstArrayView<FAssetCategoryPath> GetAssetCategories();
	UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAssetData);
} // namespace UE::PoseSearch

UCLASS()
class UAssetDefinition_PoseSearchDatabase : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	virtual FLinearColor GetAssetColor() const override { return UE::PoseSearch::GetAssetColor(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override { return UE::PoseSearch::GetAssetCategories(); }
	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAssetData) const override { return UE::PoseSearch::LoadThumbnailInfo(InAssetData); }

	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("PoseSearchAssetDefinition", "DisplayName_UPoseSearchDatabase", "Pose Search Database"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UPoseSearchDatabase::StaticClass(); }
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
};

UCLASS()
class UAssetDefinition_PoseSearchSchema : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	virtual FLinearColor GetAssetColor() const override { return UE::PoseSearch::GetAssetColor(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override { return UE::PoseSearch::GetAssetCategories(); }
	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAssetData) const override { return UE::PoseSearch::LoadThumbnailInfo(InAssetData); }

	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("PoseSearchAssetDefinition", "DisplayName_UPoseSearchSchema", "Pose Search Schema"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UPoseSearchSchema::StaticClass(); }
};

UCLASS()
class UAssetDefinition_PoseSearchNormalizationSet : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	virtual FLinearColor GetAssetColor() const override { return UE::PoseSearch::GetAssetColor(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override { return UE::PoseSearch::GetAssetCategories(); }
	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAssetData) const override { return UE::PoseSearch::LoadThumbnailInfo(InAssetData); }

	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("PoseSearchAssetDefinition", "DisplayName_UPoseSearchNormalizationSet", "Pose Search Normalization Set"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UPoseSearchNormalizationSet::StaticClass(); }
};

UCLASS(Experimental)
class UAssetDefinition_PoseSearchInteractionAsset : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	virtual FLinearColor GetAssetColor() const override { return UE::PoseSearch::GetAssetColor(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override { return UE::PoseSearch::GetAssetCategories(); }
	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAssetData) const override { return UE::PoseSearch::LoadThumbnailInfo(InAssetData); }

	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("PoseSearchInteractionAssetDefinition", "DisplayName_PoseSearchInteractionAsset", "Pose Search Interaction Asset"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UPoseSearchInteractionAsset::StaticClass(); }
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
};


