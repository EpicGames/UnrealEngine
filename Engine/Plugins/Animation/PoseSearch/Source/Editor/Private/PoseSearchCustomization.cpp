// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchCustomization.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "AssetRegistry/AssetData.h"
#include "Chooser.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EditorClassUtils.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/GameViewportClient.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailsView.h"
#include "IPropertyUtilities.h"
#include "ObjectEditorUtils.h"
#include "PoseSearch/Chooser/PoseSearchChooserColumn.h"
#include "PoseSearch/MultiAnimAsset.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "PoseSearchCustomization"

//////////////////////////////////////////////////////////////////////////
// FPoseSearchColumnRowReflectionCustomization

void FPoseSearchColumnRowReflectionCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow.NameContent()
	[
		InStructPropertyHandle->CreatePropertyNameWidget()
	];
}

void FPoseSearchColumnRowReflectionCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TSharedPtr<IPropertyHandle> DataHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPoseSearchColumnRow, Data));
	TSharedPtr<IPropertyHandle> DatabaseHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPoseSearchColumnRow, Database));
	TSharedPtr<IPropertyHandle> RootChooserHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPoseSearchColumnRowReflection, RootChooser));

	if (DataHandle.IsValid())
	{
		StructBuilder.AddProperty(DataHandle.ToSharedRef());
	}

	// Custom Database row: combobox restricted to UPoseSearchDatabase instances
	// found in RootChooser->NestedObjects, matching the column-widget behaviour.
	if (DatabaseHandle.IsValid())
	{
		StructBuilder.AddProperty(DatabaseHandle.ToSharedRef()).CustomWidget()
			.NameContent()
			[
				DatabaseHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SComboButton)
				.ContentPadding(FMargin(2.f, 0.f))
				.ButtonContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text_Lambda([DatabaseHandle]()
					{
						UObject* DatabaseObj = nullptr;
						const FPropertyAccess::Result Result = DatabaseHandle->GetValue(DatabaseObj);
						if (Result == FPropertyAccess::MultipleValues)
						{
							return LOCTEXT("MultipleValuesText", "Multiple Values");
						}

						if (Result == FPropertyAccess::Success)
						{
							if (const UPoseSearchDatabase* Database = Cast<UPoseSearchDatabase>(DatabaseObj))
							{
								return FText::FromName(Database->GetFName());
							}
						}

						return LOCTEXT("None", "None");
					})
				]
				.OnGetMenuContent_Lambda([DatabaseHandle, RootChooserHandle]()
				{
					FMenuBuilder MenuBuilder(true, nullptr);
					MenuBuilder.BeginSection(NAME_None, LOCTEXT("DatabaseSelection", "Database"));

					// "None" entry to clear the selection
					MenuBuilder.AddMenuEntry(
						LOCTEXT("None", "None"),
						LOCTEXT("NoneDatabase_Tooltip", "Remove the database assignment for this row"),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([DatabaseHandle]()
						{
							const FScopedTransaction Transaction(LOCTEXT("ClearDatabase", "Clear Database"));
							DatabaseHandle->SetValue((UObject*)nullptr);
						}))
					);

					// One entry per UPoseSearchDatabase in RootChooser->NestedObjects
					if (RootChooserHandle.IsValid())
					{
						UObject* ChooserObj = nullptr;
						if (RootChooserHandle->GetValue(ChooserObj) == FPropertyAccess::Success)
						{
							if (const UChooserTable* RootChooser = Cast<UChooserTable>(ChooserObj))
							{
								for (UObject* NestedObject : RootChooser->NestedObjects)
								{
									if (UPoseSearchDatabase* Database = Cast<UPoseSearchDatabase>(NestedObject))
									{
										MenuBuilder.AddMenuEntry(
											FText::FromName(Database->GetFName()),
											FText::GetEmpty(),
											FSlateIcon(),
											FUIAction(FExecuteAction::CreateLambda([DatabaseHandle, Database]()
											{
												const FScopedTransaction Transaction(LOCTEXT("SetDatabase", "Set Database"));
												DatabaseHandle->SetValue(Database);
											}))
										);
									}
								}
							}
						}
					}

					MenuBuilder.EndSection();
					return MenuBuilder.MakeWidget();
				})
			];
	}
}

void FPoseSearchDatabaseAnimationAssetCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FText SequenceNameText;

	TArray<UObject*> Objects;
	InStructPropertyHandle->GetOuterObjects(Objects);

	if (Objects.Num() == 1)
	{
		FPoseSearchDatabaseAnimationAsset* DatabaseAnimationAsset = (FPoseSearchDatabaseAnimationAsset*)InStructPropertyHandle->GetValueBaseAddress((uint8*)Objects[0]);
		check(DatabaseAnimationAsset);
		
		if (const UObject* AnimationAsset = DatabaseAnimationAsset->GetAnimationAsset())
		{
			SequenceNameText = FText::FromName(AnimationAsset->GetFName());
		}
		else
		{
			SequenceNameText = LOCTEXT("NewDatabaseAnimationAssetLabel", "New Animation Asset");
		}
	}

	HeaderRow
	.NameContent()
	[
		SNew(STextBlock)
		.Text(SequenceNameText)
	];
}

void FPoseSearchDatabaseAnimationAssetCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	InStructPropertyHandle->GetNumChildren(NumChildren);

	TSharedPtr<IPropertyHandle> bEnabledHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPoseSearchDatabaseAnimationAsset, bEnabled));
	TSharedPtr<IPropertyHandle> bDisableReselectionHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPoseSearchDatabaseAnimationAsset, bDisableReselection));
	TSharedPtr<IPropertyHandle> MirrorOptionHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPoseSearchDatabaseAnimationAsset, MirrorOption));
	TSharedPtr<IPropertyHandle> BranchInIdHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPoseSearchDatabaseAnimationAsset, BranchInId));

	TSharedPtr<IPropertyHandle> AnimAssetHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPoseSearchDatabaseAnimationAsset, AnimAsset));
	TSharedPtr<IPropertyHandle> bUseSingleSampleHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPoseSearchDatabaseAnimationAsset, bUseSingleSample));
	TSharedPtr<IPropertyHandle> bUseGridForSamplingHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPoseSearchDatabaseAnimationAsset, bUseGridForSampling));
	TSharedPtr<IPropertyHandle> NumberOfHorizontalSamplesHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPoseSearchDatabaseAnimationAsset, NumberOfHorizontalSamples));
	TSharedPtr<IPropertyHandle> NumberOfVerticalSamplesHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPoseSearchDatabaseAnimationAsset, NumberOfVerticalSamples));
	TSharedPtr<IPropertyHandle> BlendParamXHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPoseSearchDatabaseAnimationAsset, BlendParamX));
	TSharedPtr<IPropertyHandle> BlendParamYHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPoseSearchDatabaseAnimationAsset, BlendParamY));
	TSharedPtr<IPropertyHandle> SamplingRangeHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPoseSearchDatabaseAnimationAsset, SamplingRange));

	const bool bIsPoseSearchDatabaseAnimationAssetLayoutValid =
		NumChildren == 12 &&
		bEnabledHandle.IsValid() &&
		bDisableReselectionHandle.IsValid() &&
		MirrorOptionHandle.IsValid() &&
		BranchInIdHandle.IsValid() &&
		AnimAssetHandle.IsValid() &&
		bUseSingleSampleHandle.IsValid() &&
		bUseGridForSamplingHandle.IsValid() &&
		NumberOfHorizontalSamplesHandle.IsValid() &&
		NumberOfVerticalSamplesHandle.IsValid() &&
		BlendParamXHandle.IsValid() &&
		BlendParamYHandle.IsValid() &&
		SamplingRangeHandle.IsValid();

	// update FPoseSearchDatabaseAnimationAssetCustomization if FPoseSearchDatabaseAnimationAsset or FPoseSearchDatabaseAnimationAssetBase changed!
	if (!ensure(bIsPoseSearchDatabaseAnimationAssetLayoutValid))
	{
		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			StructBuilder.AddProperty(InStructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef());
		}
		return;
	}


	StructBuilder.AddProperty(AnimAssetHandle.ToSharedRef());
	StructBuilder.AddProperty(bEnabledHandle.ToSharedRef());
	StructBuilder.AddProperty(SamplingRangeHandle.ToSharedRef());
	StructBuilder.AddProperty(bDisableReselectionHandle.ToSharedRef());
	StructBuilder.AddProperty(MirrorOptionHandle.ToSharedRef());
	StructBuilder.AddProperty(BranchInIdHandle.ToSharedRef());

	UObject* AnimAsset = nullptr;
	const FPropertyAccess::Result AnimAssetGetValueResult = AnimAssetHandle->GetValue(AnimAsset);
	if (AnimAssetGetValueResult == FPropertyAccess::Success && AnimAsset)
	{
		if (AnimAsset->IsA<UBlendSpace>())
		{
			StructBuilder.AddProperty(bUseSingleSampleHandle.ToSharedRef());
			StructBuilder.AddProperty(bUseGridForSamplingHandle.ToSharedRef());
			StructBuilder.AddProperty(NumberOfHorizontalSamplesHandle.ToSharedRef());
			StructBuilder.AddProperty(NumberOfVerticalSamplesHandle.ToSharedRef());
			StructBuilder.AddProperty(BlendParamXHandle.ToSharedRef());
			StructBuilder.AddProperty(BlendParamYHandle.ToSharedRef());
		}
		else if (AnimAsset->IsA<UMultiAnimAsset>())
		{
			StructBuilder.AddProperty(NumberOfHorizontalSamplesHandle.ToSharedRef());
			StructBuilder.AddProperty(NumberOfVerticalSamplesHandle.ToSharedRef());
		}
	}
	else if (FPropertyAccess::MultipleValues == AnimAssetGetValueResult)
	{
		StructBuilder.AddProperty(bUseSingleSampleHandle.ToSharedRef());
		StructBuilder.AddProperty(bUseGridForSamplingHandle.ToSharedRef());
		StructBuilder.AddProperty(NumberOfHorizontalSamplesHandle.ToSharedRef());
		StructBuilder.AddProperty(NumberOfVerticalSamplesHandle.ToSharedRef());
		StructBuilder.AddProperty(BlendParamXHandle.ToSharedRef());
		StructBuilder.AddProperty(BlendParamYHandle.ToSharedRef());
	}

	FFloatInterval SamplingRange = FFloatInterval(0.f, 0.f);
	void* StructData = nullptr;
	const FPropertyAccess::Result SamplingRangeGetValueResult = SamplingRangeHandle->GetValueData(StructData);
	if (SamplingRangeGetValueResult == FPropertyAccess::Success && StructData)
	{
		SamplingRange = *static_cast<FFloatInterval*>(StructData);
	}

	// Create and inject a check box to expose the animation looping state
	const bool bLooping = FPoseSearchDatabaseAnimationAsset::IsLooping(AnimAsset, SamplingRange);
	StructBuilder.AddProperty(InStructPropertyHandle).CustomWidget()
		.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget(
				LOCTEXT("FPoseSearchDatabaseAnimationAssetCustomizationLoopingLabel", "Looping"),
				LOCTEXT("FPoseSearchDatabaseAnimationAssetCustomizationLoopingTooltip", "Is this animation set as looping? If you want to change this you need to change it in the animation asset. Changing the sampling range will disable looping")
			)
		]
		.ValueContent()
		[
			SNew(SCheckBox)
				.IsChecked_Lambda([bLooping]()
					{
						return bLooping ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				.IsEnabled(false)
		];

	// Create and inject a check box to expose the animation root motion state
	const bool bHasRootMotion = FPoseSearchDatabaseAnimationAsset::IsRootMotionEnabled(AnimAsset);
	StructBuilder.AddProperty(InStructPropertyHandle).CustomWidget()
		.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget(
				LOCTEXT("FPoseSearchDatabaseAnimationAssetCustomizationHasRootMotionLabel", "HasRootMotion"),
				LOCTEXT("FPoseSearchDatabaseAnimationAssetCustomizationHasRootMotionTooltip", "Does this animation have root motion enabled ? If you want to change this you need to change it in the animation asset.")
			)
		]
		.ValueContent()
		[
			SNew(SCheckBox)
				.IsChecked_Lambda([bHasRootMotion]()
					{ 
						return bHasRootMotion ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				.IsEnabled(false)
		];
}
#undef LOCTEXT_NAMESPACE

