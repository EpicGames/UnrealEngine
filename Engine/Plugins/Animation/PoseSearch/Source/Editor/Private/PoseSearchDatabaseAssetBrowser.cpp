// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDatabaseAssetBrowser.h"

#include "ContentBrowserDataSource.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimMontage.h"
#include "Animation/BlendSpace.h"
#include "Widgets/SBoxPanel.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "PoseSearchDatabaseViewModel.h"
#include "PoseSearch/MultiAnimAsset.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"
#include "PoseSearchDatabaseEditorUtils.h"
#include "Filters/SBasicFilterBar.h"

#define LOCTEXT_NAMESPACE "PoseSearchDatabaseAssetBrowser"

namespace UE::PoseSearch
{
	
void SPoseSearchDatabaseAssetBrowser::Construct(const FArguments& InArgs, TSharedPtr<FDatabaseViewModel> InViewModel)
{
	DatabaseViewModel = InViewModel;
	
	ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SAssignNew(AssetBrowserBox, SBox)
			]
		];

	// Register to be notified when properties are edited. We leverage this to refresh the browser in case the target schema changes.
	const FCoreUObjectDelegates::FOnObjectPropertyChanged::FDelegate OnPropertyChangedDelegate = FCoreUObjectDelegates::FOnObjectPropertyChanged::FDelegate::CreateRaw(this, &SPoseSearchDatabaseAssetBrowser::OnObjectPropertyChanged);
	OnPropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.Add(OnPropertyChangedDelegate);

	RefreshView();
}

void SPoseSearchDatabaseAssetBrowser::RefreshView()
{
	FAssetPickerConfig AssetPickerConfig;
	
	AssetPickerConfig.Filter.ClassPaths.Add(UAnimationAsset::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.ClassPaths.Add(UMultiAnimAsset::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = true;
	
	AssetPickerConfig.bAddFilterUI = true;
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
	AssetPickerConfig.bShowPathInColumnView = true;
	AssetPickerConfig.bShowTypeInColumnView = false;
	AssetPickerConfig.InitialThumbnailSize = EThumbnailSize::Small;
	AssetPickerConfig.RefreshAssetViewDelegates.Add(&RefreshAssetViewDelegate);
	AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &SPoseSearchDatabaseAssetBrowser::OnShouldFilterAsset);
	AssetPickerConfig.OnAssetDoubleClicked = FOnAssetSelected::CreateSP(this, &SPoseSearchDatabaseAssetBrowser::OnAssetDoubleClicked);
	AssetPickerConfig.AssetShowWarningText = LOCTEXT("NoAssets_Warning", "No Assets found. No compatible assets with the database's schema where found. Ensure your assets' skeleton matches a skeleton from the database's schema.");
	AssetPickerConfig.bCanShowDevelopersFolder = true;

	// Hide all asset registry columns by default (we only really want the name and path)
	const UObject* AnimSequenceDefaultObject = UAnimSequence::StaticClass()->GetDefaultObject();
	FAssetRegistryTagsContextData TagsContext(AnimSequenceDefaultObject, EAssetRegistryTagsCaller::Uncategorized);
	AnimSequenceDefaultObject->GetAssetRegistryTags(TagsContext);
	for (const TPair<FName, UObject::FAssetRegistryTag>& TagPair : TagsContext.Tags)
	{
		AssetPickerConfig.HiddenColumnNames.Add(TagPair.Key.ToString());
	}

	// Also hide the type column by default (but allow users to enable it, so don't use bShowTypeInColumnView)
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("Class"));
	AssetPickerConfig.HiddenColumnNames.Add(ContentBrowserItemAttributes::ItemDiskSize.ToString());
	AssetPickerConfig.HiddenColumnNames.Add(ContentBrowserItemAttributes::VirtualizedData.ToString());

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	AssetBrowserBox->SetContent(ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig));
}

SPoseSearchDatabaseAssetBrowser::~SPoseSearchDatabaseAssetBrowser()
{
	// Unregister the property modification handler
	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnPropertyChangedHandle);
}

void SPoseSearchDatabaseAssetBrowser::OnAssetDoubleClicked(const FAssetData& AssetData)
{
	if (const UObject* Asset = AssetData.GetAsset())
	{
		if (Cast<UAnimationAsset>(Asset) || Cast<UMultiAnimAsset>(Asset))
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Asset);
		}
	}
}

bool SPoseSearchDatabaseAssetBrowser::OnShouldFilterAsset(const FAssetData& AssetData)
{
	const bool bIsAssetCompatibleWithDatabase = DatabaseViewModel && FPoseSearchEditorUtils::IsAssetCompatibleWithDatabase(DatabaseViewModel->GetPoseSearchDatabase(), AssetData);
	return !bIsAssetCompatibleWithDatabase;
}
	
void SPoseSearchDatabaseAssetBrowser::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent) const
{
	if (DatabaseViewModel && InObject)
	{
		const UPoseSearchDatabase* Database = DatabaseViewModel->GetPoseSearchDatabase();
		if ((Database != nullptr && (Database == InObject || Database->Schema == InObject)) ||
			InObject->GetClass()->IsChildOf(UAnimationAsset::StaticClass()) ||
			InObject->GetClass()->IsChildOf(UMultiAnimAsset::StaticClass()))
		{
			RefreshAssetViewDelegate.ExecuteIfBound(true);
		}
	}
}
}

#undef LOCTEXT_NAMESPACE