// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ContentBrowserDelegates.h"
#include "Widgets/Layout/SBox.h"

struct FAssetData;

namespace UE::PoseSearch
{
	class FDatabaseViewModel;
	
	class SPoseSearchDatabaseAssetBrowser : public SBox
	{
	public:
	
		SLATE_BEGIN_ARGS(SPoseSearchDatabaseAssetBrowser) {}
		SLATE_END_ARGS();

		void Construct(const FArguments& InArgs, TSharedPtr<FDatabaseViewModel> InViewModel);
		
		void RefreshView();
		
		virtual ~SPoseSearchDatabaseAssetBrowser() override;
	private:

		TSharedPtr<SBox> AssetBrowserBox;
		TSharedPtr<FDatabaseViewModel> DatabaseViewModel;
		
		void OnAssetDoubleClicked(const FAssetData& AssetData);
		bool OnShouldFilterAsset(const FAssetData& AssetData);

		/* We need to be able to refresh the asset list if requested (i.e. Schema changes) */
		FRefreshAssetViewDelegate RefreshAssetViewDelegate;

		/* Keep track of delegate used to listen for pose search schema changes */
		FDelegateHandle OnPropertyChangedHandle;
		
		void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent) const;
	};
}