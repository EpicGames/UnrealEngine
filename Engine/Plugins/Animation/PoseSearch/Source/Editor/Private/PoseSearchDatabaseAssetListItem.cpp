// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDatabaseAssetListItem.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AnimPreviewInstance.h"
#include "AssetSelection.h"
#include "AssetToolsModule.h"
#include "ClassIconFinder.h"
#include "DetailColumnSizeData.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Editor.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IAnimationEditor.h"
#include "IMultiAnimAssetEditor.h"
#include "IPersonaToolkit.h"
#include "Misc/FeedbackContext.h"
#include "Misc/TransactionObjectEvent.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearchDatabaseAssetTree.h"
#include "PoseSearchDatabaseEditorUtils.h"
#include "PoseSearchDatabaseViewModel.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "SPositiveActionButton.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "SDatabaseAssetListItem"

namespace UE::PoseSearch
{
	static constexpr FLinearColor DisabledColor = FLinearColor(1.f, 1.f, 1.f, 0.25f);

	static FText GetWarningsForDatabaseAsset(const FPoseSearchDatabaseAnimationAssetBase& InDatabaseAsset, const UPoseSearchDatabase& InDatabase)
	{
		if (InDatabaseAsset.GetAnimationAsset() == nullptr)
		{
			return LOCTEXT("ErrorNoAsset", "No asset has been selected.");
		}
		
		if (!InDatabaseAsset.IsSkeletonCompatible(InDatabase.Schema))
		{
			return FText::Format(LOCTEXT("ErrorIncompatibleSkeleton", "{0}'s skeleton is not compatible with the schema's skeleton(s)."), FText::FromString(InDatabaseAsset.GetName()));
		}
		
		if (!InDatabaseAsset.IsMirrorOptionCompatible(InDatabase.Schema))
		{
			return FText::Format(LOCTEXT("ErrorIncompatibleMirrorOption", "{0}'s MirrorOption is not compatible with the schema's MirrorDataTable(s)."), FText::FromString(InDatabaseAsset.GetName()));
		}
		
		if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(InDatabaseAsset.GetAnimationAsset()))
		{
			if (!BlendSpace->bShouldMatchSyncPhases)
			{
				return FText::Format(LOCTEXT("ErrorMissingSyncPhaseMatch", "{0}'s bShouldMatchSyncPhases flag is not enabled. This is required for properly pose matching blendspaces."), FText::FromString(InDatabaseAsset.GetName()));
			}
			
			const TConstArrayView<FBlendSample> BlendSamples = BlendSpace->GetBlendSamples();
			for (int i = 0; i < BlendSamples.Num() - 1; ++i)
			{
				const FBlendSample& CurrSample = BlendSamples[i];
				const FBlendSample& NextSample = BlendSamples[i + 1];

				if (CurrSample.Animation && NextSample.Animation)
				{
					bool bWarning = false;

					if (CurrSample.Animation->AuthoredSyncMarkers.Num() == NextSample.Animation->AuthoredSyncMarkers.Num())
					{
						for (int j = 0; j < CurrSample.Animation->AuthoredSyncMarkers.Num(); ++j)
						{
							if (CurrSample.Animation->AuthoredSyncMarkers[j].MarkerName != NextSample.Animation->AuthoredSyncMarkers[j].MarkerName)
							{
								bWarning = true;
								break;
							}
						}
					}
					else
					{
						bWarning = true;
					}

					if (bWarning)
					{
						return FText::Format(LOCTEXT("ErrorDifferentNumOfSyncMarkers", "{0}'s samples don't share the same layout of sync markers. This is required for properly pose matching blendspaces."), FText::FromString(InDatabaseAsset.GetName()));
					}
				}
			}
		}

		if (const UObject* ThisAnimAsset = InDatabaseAsset.GetAnimationAsset())
		{
			const EPoseSearchMirrorOption ThisMirrorOption = InDatabaseAsset.GetMirrorOption();
			const FFloatInterval ThisSamplingRange = InDatabaseAsset.GetSamplingRange();
			const int32 NumAssets = InDatabase.GetNumAnimationAssets();
			for (int32 OtherIdx = 0; OtherIdx < NumAssets; ++OtherIdx)
			{
				const FPoseSearchDatabaseAnimationAsset* Other = InDatabase.GetDatabaseAnimationAsset(OtherIdx);
				if (Other 
					&& static_cast<const FPoseSearchDatabaseAnimationAssetBase*>(Other) != &InDatabaseAsset
					&& Other->GetAnimationAsset() == ThisAnimAsset
					&& Other->GetMirrorOption() == ThisMirrorOption
					&& Other->GetSamplingRange() == ThisSamplingRange)
				{
					return FText::Format(LOCTEXT("ErrorDuplicateAnimationAsset", "{0} duplicates another entry with the same animation asset, mirror option, and sampling range."), FText::FromString(InDatabaseAsset.GetName()));
				}
			}
		}

		return FText::GetEmpty();
	}
	
	/* We need a custom widget to be able to consume the "DoubleClick" event so we can cycle through the mirror options but not open the asset. */
	class SMirrorTypeWidget : public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SMirrorTypeWidget){}
		SLATE_END_ARGS()
	public:
		
		void Construct(const FArguments& InArgs, const TWeakPtr<FDatabaseAssetTreeNode>& InAssetTreeNode, const TWeakPtr<SDatabaseAssetTree>& InAssetTree, const TWeakPtr<FDatabaseViewModel>& InViewModel)
		{
			WeakAssetTreeNode = InAssetTreeNode;
			SkeletonView = InAssetTree;
			EditorViewModel = InViewModel;
			
			ChildSlot
			[
				SNew(SOverlay)
				+SOverlay::Slot()
				[
					SNew(SImage)
					.Image(this, &SMirrorTypeWidget::GetBackgroundImage)
				]
				+SOverlay::Slot()
				[
					SNew(SImage)
					.Image(this, &SMirrorTypeWidget::GetMirrorOptionSlateBrush)
					.ToolTipText(this, &SMirrorTypeWidget::GetMirrorOptionToolTip)
				]
			];
		}

		virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
		{
			return OnMouseButtonDown(InMyGeometry, InMouseEvent);
		}

		virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& InMouseEvent) override
		{
			const TSharedPtr<FDatabaseAssetTreeNode> ThisNode = WeakAssetTreeNode.Pin();
			const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin();
			const TSharedPtr<SDatabaseAssetTree> SkeletonViewPtr = SkeletonView.Pin();

			if (ThisNode && ViewModel && SkeletonViewPtr && InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
			{
				TArray<TSharedPtr<FDatabaseAssetTreeNode>> SelectedNodes = SkeletonViewPtr->GetSelectedNodes();
				if (!SelectedNodes.Contains(ThisNode))
				{
					SelectedNodes = { ThisNode };
				}

				const FScopedTransaction Transaction(LOCTEXT("OnClickEditMirrorOptionPoseSearchDatabase", "Edit Mirror Option"));

				// Next mirror option is driven by the clicked item's current state
				static const TArray<EPoseSearchMirrorOption> OptionArray = { EPoseSearchMirrorOption::UnmirroredOnly, EPoseSearchMirrorOption::MirroredOnly, EPoseSearchMirrorOption::UnmirroredAndMirrored };
				const EPoseSearchMirrorOption NextOption = OptionArray[(static_cast<int32>(ViewModel->GetMirrorOption(ThisNode->SourceAssetIdx)) + 1) % OptionArray.Num()];

				for (const TSharedPtr<FDatabaseAssetTreeNode>& Node : SelectedNodes)
				{
					if (Node->SourceAssetIdx != INDEX_NONE)
					{
						ViewModel->SetMirrorOption(Node->SourceAssetIdx, NextOption);
					}
				}

				SkeletonViewPtr->RefreshTreeView(false, true);
				ViewModel->BuildSearchIndex();

				return FReply::Handled();
			}

			return FReply::Unhandled();
		}

		const FSlateBrush * GetBackgroundImage() const
		{
			const FCheckBoxStyle& Style = FCoreStyle::Get().GetWidgetStyle<FCheckBoxStyle>("Checkbox");
							
			return IsHovered() ? &Style.BackgroundHoveredImage : &Style.BackgroundImage; 
		}
		
		FText GetMirrorOptionToolTip() const
		{
			const TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin();

			FString TooltipString;
						
			TooltipString.Append(LOCTEXT("ToolTipMirrorOption", "Mirror Option: ").ToString());
			TooltipString.Append(AssetTreeNode ? UEnum::GetDisplayValueAsText(AssetTreeNode->GetMirrorOption()).ToString() : LOCTEXT("ToolTipMirrorOption_Invalid", "Invalid").ToString());
						
			return FText::FromString(TooltipString);
		}
		
		const FSlateBrush* GetMirrorOptionSlateBrush() const
		{
			if (const TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin())
			{
				switch (AssetTreeNode->GetMirrorOption())
				{
				case EPoseSearchMirrorOption::UnmirroredOnly:
					return FAppStyle::Get().GetBrush("GraphEditor.AlignNodesRight");

				case EPoseSearchMirrorOption::MirroredOnly:
					return FAppStyle::Get().GetBrush("GraphEditor.AlignNodesLeft");

				case EPoseSearchMirrorOption::UnmirroredAndMirrored:
					return FAppStyle::Get().GetBrush("GraphEditor.AlignNodesCenter");
				}
			}
		
			return nullptr;
		}
		
		TWeakPtr<FDatabaseAssetTreeNode> WeakAssetTreeNode;
		TWeakPtr<FDatabaseViewModel> EditorViewModel;
		TWeakPtr<SDatabaseAssetTree> SkeletonView;
	};
	
	void SDatabaseAssetListItem::Construct(
		const FArguments& InArgs,
		const TSharedRef<FDatabaseViewModel>& InEditorViewModel,
		const TSharedRef<STableViewBase>& OwnerTable,
		TSharedRef<FDatabaseAssetTreeNode> InAssetTreeNode,
		TSharedRef<FUICommandList> InCommandList,
		TSharedPtr<SDatabaseAssetTree> InHierarchy)
	{
		WeakAssetTreeNode = InAssetTreeNode;
		EditorViewModel = InEditorViewModel;
		SkeletonView = InHierarchy;

		AssetTypeColor = FColor::White;
		if (UPoseSearchDatabase* Database = InEditorViewModel->GetPoseSearchDatabase())
		{
			if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(WeakAssetTreeNode.Pin()->SourceAssetIdx))
			{
				static FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
				if (TSharedPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(DatabaseAnimationAsset->GetAnimationAssetStaticClass()).Pin())
				{
					AssetTypeColor = AssetTypeActions->GetTypeColor();
				}
			}
		}
		
		if (InAssetTreeNode->SourceAssetIdx == INDEX_NONE)
		{
			ConstructGroupItem(OwnerTable);
		}
		else
		{
			ConstructAssetItem(OwnerTable);
		}
	}

	void SDatabaseAssetListItem::ConstructGroupItem(const TSharedRef<STableViewBase>& OwnerTable)
	{
		STableRow<TSharedPtr<FDatabaseAssetTreeNode>>::ChildSlot
		.Padding(0.0f, 2.0f, 0.0f, 0.0f)
		[
			GenerateItemWidget()
		];

		STableRow<TSharedPtr<FDatabaseAssetTreeNode>>::ConstructInternal(
			STableRow<TSharedPtr<FDatabaseAssetTreeNode>>::FArguments()
			.Style(FAppStyle::Get(), "DetailsView.TreeView.TableRow")
			.OnCanAcceptDrop(SkeletonView.Pin().Get(), &SDatabaseAssetTree::OnCanAcceptDrop)
			.OnAcceptDrop(SkeletonView.Pin().Get(), &SDatabaseAssetTree::OnAcceptDrop)
			.ShowSelection(true),
			OwnerTable);
	}

	void SDatabaseAssetListItem::ConstructAssetItem(const TSharedRef<STableViewBase>& OwnerTable)
	{
		STableRow<TSharedPtr<FDatabaseAssetTreeNode>>::Construct(
			STableRow<TSharedPtr<FDatabaseAssetTreeNode>>::FArguments()
			.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row"))
			.OnCanAcceptDrop(SkeletonView.Pin().Get(), &SDatabaseAssetTree::OnCanAcceptDrop)
			.OnAcceptDrop(SkeletonView.Pin().Get(), &SDatabaseAssetTree::OnAcceptDrop)
			.ShowWires(false)
			.Content()
			[
				GenerateItemWidget()
			], OwnerTable);
	}

	FReply SDatabaseAssetListItem::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
	{
		if (TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin())
		{
			TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin();
			if (const UPoseSearchDatabase* Database = ViewModel->GetPoseSearchDatabase())
			{
				if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(AssetTreeNode->SourceAssetIdx))
				{
					if (UAssetEditorSubsystem* AssetEditorSS = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
					{
						if (UObject* AnimationAsset = DatabaseAnimationAsset->GetAnimationAsset())
						{
							AssetEditorSS->OpenEditorForAsset(AnimationAsset);

							if (IAssetEditorInstance* Editor = AssetEditorSS->FindEditorForAsset(AnimationAsset, true))
							{
								float AnimationAssetTime = 0.f;
								FVector AnimationAssetBlendParameters = FVector::ZeroVector;
								ViewModel->GetAnimationTime(AssetTreeNode->SourceAssetIdx, AnimationAssetTime, AnimationAssetBlendParameters);

								if (Editor->GetEditorName() == "AnimationEditor")
								{
									IAnimationEditor* AnimationEditor = static_cast<IAnimationEditor*>(Editor);
									UDebugSkelMeshComponent* PreviewComponent = AnimationEditor->GetPersonaToolkit()->GetPreviewMeshComponent();

									// Open asset paused and at specific time as seen on the pose search debugger.
									PreviewComponent->PreviewInstance->SetPosition(AnimationAssetTime);
									PreviewComponent->PreviewInstance->SetPlaying(false);
									PreviewComponent->PreviewInstance->SetBlendSpacePosition(AnimationAssetBlendParameters);
								}
								else if (Editor->GetEditorName() == "PoseSearchInteractionAssetEditor")
								{
									IMultiAnimAssetEditor* MultiAnimAssetEditor = static_cast<IMultiAnimAssetEditor*>(Editor);

									// Open asset paused and at specific time as seen on the pose search debugger.
									MultiAnimAssetEditor->SetPreviewProperties(AnimationAssetTime, AnimationAssetBlendParameters, false);
								}
							}
						}
					}
				}
			}
		}
		return STableRow<TSharedPtr<FDatabaseAssetTreeNode>>::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
	}

	FText SDatabaseAssetListItem::GetName() const
	{
		TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin();

		if (const UPoseSearchDatabase* Database = ViewModel->GetPoseSearchDatabase())
		{
			if (TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin())
			{
				if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(AssetTreeNode->SourceAssetIdx))
				{
					return FText::FromString(DatabaseAnimationAsset->GetName());
				}
			}
			return FText::FromString(Database->GetName());
		}

		return LOCTEXT("None", "None");
	}

	TSharedRef<SWidget> SDatabaseAssetListItem::GenerateItemWidget()
	{
		int32 SourceAssetIdx = INDEX_NONE;
		if (TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin())
		{
			SourceAssetIdx = AssetTreeNode->SourceAssetIdx;
		}

		TSharedPtr<SWidget> ItemWidget;
		
		if (SourceAssetIdx == INDEX_NONE)
		{
			// it's a group
			SAssignNew(ItemWidget, SBorder)
			.BorderImage(this, &SDatabaseAssetListItem::GetGroupBackgroundImage)
			.Padding(FMargin(3.0f, 5.0f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(5.0f)
				.AutoWidth()
				[
					SNew(SExpanderArrow, STableRow<TSharedPtr<FDatabaseAssetTreeNode>>::SharedThis(this))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SRichTextBlock)
					.Text(this, &SDatabaseAssetListItem::GetName)
					.TransformPolicy(ETextTransformPolicy::ToUpper)
					.DecoratorStyleSet(&FAppStyle::Get())
					.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
				]
			];
		}
		else
		{
			// Branch in
			TSharedPtr<SImage> BranchInIconWidget;
			{
				SAssignNew(BranchInIconWidget, SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.ArrowRight"))
				.ColorAndOpacity(FColor::Turquoise)
				.ToolTipText(LOCTEXT("NodeBranchInTooltip", "This database item is synchronize with an external depedency and is sampled via a BranchIn notify."))
				.Visibility_Lambda([this]()
				{
				   const TSharedPtr<FDatabaseViewModel> ViewModelPtr = EditorViewModel.Pin();
    
				   if (const UPoseSearchDatabase* Database = ViewModelPtr->GetPoseSearchDatabase())
				   {
					  if (const TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin())
					  {
						 if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAssetBase = Database->GetDatabaseAnimationAsset(AssetTreeNode->SourceAssetIdx))
						 {
							if (DatabaseAnimationAssetBase->IsSynchronizedWithExternalDependency())
							{
							   return EVisibility::Visible;
							}
						 }
					  }
				   }
                      
				   return EVisibility::Hidden;
				})
				// @note: Works under the assumption there are not hierarchy in databases, done this way to avoid having to change the TreeView to a ListView in case its needed in the future.
				.RenderTransform(FSlateRenderTransform(1.0f, FVector2d(-8.0f, 0.0f))) 
				.Clipping(EWidgetClipping::ClipToBoundsWithoutIntersecting);
			}
			
			// Item Thumbnail
			{
				// Get item Icon
				TSharedPtr<SImage> ItemIconWidget;
				{
					SAssignNew(ItemIconWidget, SImage)
						.Image_Lambda([this]
							{
								UClass* AnimationAssetStaticClass = nullptr;
								TSharedPtr<FDatabaseViewModel> ViewModelPtr = EditorViewModel.Pin();
								if (const UPoseSearchDatabase* Database = ViewModelPtr->GetPoseSearchDatabase())
								{
									if (const TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin())
									{
										if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(AssetTreeNode->SourceAssetIdx))
										{
											AnimationAssetStaticClass = DatabaseAnimationAsset->GetAnimationAssetStaticClass();
										}
									}
								}
								return FSlateIconFinder::FindIconBrushForClass(AnimationAssetStaticClass);
							});
				}
				
				SAssignNew(AssetThumbnailOverlay, SOverlay)
				
				// Item Icon
				+ SOverlay::Slot()
				.Padding(1.0f)
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					[
						SNew(SBorder)
						.Padding(0.0f)
						.VAlign(VAlign_Fill)
						.HAlign(HAlign_Fill)
						.BorderImage(FAppStyle::GetBrush("AssetThumbnail.AssetBackground"))
						[
							SNew(SBorder)
							.Padding(3.0f)
							.BorderImage(FStyleDefaults::GetNoBrush())
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Center)
							[
								ItemIconWidget.ToSharedRef()
							]
						]
					]

					// Color strip
					+ SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Bottom )
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
						.BorderBackgroundColor(AssetTypeColor)
						.Padding(FMargin(0, 2, 0, 0))
					]
				]

				// Square border
				+ SOverlay::Slot()
				[
					SNew(SImage)
					.Image_Lambda([this]() -> const FSlateBrush *
					{
						static const FName HoveredBorderName("PropertyEditor.AssetThumbnailBorderHovered");
						static const FName RegularBorderName("PropertyEditor.AssetThumbnailBorder");
						
						if (AssetThumbnailOverlay)
						{
							return AssetThumbnailOverlay->IsHovered() ? FAppStyle::Get().GetBrush(HoveredBorderName) : FAppStyle::Get().GetBrush(RegularBorderName);
						}
						
						return nullptr;
					})
					.Visibility(EVisibility::SelfHitTestInvisible)
				];
			}
			
			// Picker
			TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin();
			TSharedPtr<SObjectPropertyEntryBox> AssetPickerWidget;
			if (UPoseSearchDatabase* Database = ViewModel->GetPoseSearchDatabase())
			{
				if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(SourceAssetIdx))
				{
					SAssignNew(AssetPickerWidget, SObjectPropertyEntryBox)
					.AllowClear(false)
					// @todo: fix the AllowedClass(es). for now we filter inefficiently in OnShouldFilterAsset_Lambda
					//.AllowedClass(DatabaseAnimationAsset->GetAnimationAssetStaticClass())
					.DisplayThumbnail(false)
					.IsEnabled(this, &SDatabaseAssetListItem::GetAssetPickerIsEnabled)
					.ObjectPath(this, &SDatabaseAssetListItem::GetAssetPickerObjectPath)
					.OnObjectChanged(this, &SDatabaseAssetListItem::OnAssetPickerObjectChanged)
					.OnShouldFilterAsset_Lambda([this](const FAssetData& InAssetData)
					{
						if (EditorViewModel.IsValid())
						{
							return !FPoseSearchEditorUtils::IsAssetCompatibleWithDatabase(EditorViewModel.Pin()->GetPoseSearchDatabase(), InAssetData);
						}
						
						return true;
					})
					.CustomContentSlot()
					[
						// Display warning below picked asset.
						SNew(STextBlock)
						.Margin(FMargin(2,0))
						.Justification(ETextJustify::Left)
						.Font(FSlateFontInfo(FCoreStyle::GetDefaultFont(),8, "Regular"))
						.Text(this, &SDatabaseAssetListItem::GetAssetPickerText)
						.ColorAndOpacity(this, &SDatabaseAssetListItem::GetAssetPickerCustomContentSlotTextColor)
						.Visibility(this, &SDatabaseAssetListItem::GetAssetPickerCustomContentSlotVisibility)
					];
				}
			}

			// Info icons
			TSharedPtr<SHorizontalBox> InfoIconsHorizontalBox;
			{
				SAssignNew(InfoIconsHorizontalBox, SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.0f, 1.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Graph.Node.Loop"))
					.ColorAndOpacity(this, &SDatabaseAssetListItem::GetLoopingColorAndOpacity)
					.ToolTipText(this, &SDatabaseAssetListItem::GetLoopingToolTip)
				]

				// Root Motion
				+ SHorizontalBox::Slot()
				.Padding(1.0f, 2.0f)
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("AnimGraph.Attribute.RootMotionDelta.Icon"))
					.DesiredSizeOverride(FVector2D{16.f, 16.f})
					.ColorAndOpacity(this, &SDatabaseAssetListItem::GetRootMotionColorAndOpacity)
					.ToolTipText(this, &SDatabaseAssetListItem::GetRootMotionOptionToolTip)
				]

				// Mirror type
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Padding(4.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(SMirrorTypeWidget, WeakAssetTreeNode, SkeletonView, EditorViewModel)
				]

				// Disable Reselection
				+ SHorizontalBox::Slot()
				.Padding(4.0f, 1.0f)
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked(this, &SDatabaseAssetListItem::GetDisableReselectionChecked)
					.OnCheckStateChanged(const_cast<SDatabaseAssetListItem*>(this), &SDatabaseAssetListItem::OnDisableReselectionChanged)
					.ToolTipText(this, &SDatabaseAssetListItem::GetDisableReselectionToolTip)
					.Padding(FMargin(0.0f, 2.0f, 0.0f, 0.0f))
					.CheckedImage(FAppStyle::Get().GetBrush("MotionMatchingEditor.DisablePoseReselection"))
					.CheckedHoveredImage(FAppStyle::Get().GetBrush("MotionMatchingEditor.DisablePoseReselection"))
					.CheckedPressedImage(FAppStyle::Get().GetBrush("MotionMatchingEditor.DisablePoseReselection"))
					.UncheckedImage(FAppStyle::Get().GetBrush("MotionMatchingEditor.EnablePoseReselection"))
					.UncheckedHoveredImage(FAppStyle::Get().GetBrush("MotionMatchingEditor.EnablePoseReselection"))
					.UncheckedPressedImage(FAppStyle::Get().GetBrush("MotionMatchingEditor.EnablePoseReselection"))
				]
				
				// Disable/Enable
				+ SHorizontalBox::Slot()
				.MaxWidth(16)
				.Padding(4.0f, 0.0f)
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked(this, &SDatabaseAssetListItem::GetAssetEnabledChecked)
					.OnCheckStateChanged(const_cast<SDatabaseAssetListItem*>(this), &SDatabaseAssetListItem::OnAssetIsEnabledChanged)
					.ToolTipText(this, &SDatabaseAssetListItem::GetAssetEnabledToolTip)
					.CheckedImage(FAppStyle::Get().GetBrush("Icons.Visible"))
					.CheckedHoveredImage(FAppStyle::Get().GetBrush("Icons.Visible"))
					.CheckedPressedImage(FAppStyle::Get().GetBrush("Icons.Visible"))
					.UncheckedImage(FAppStyle::Get().GetBrush("Icons.Hidden"))
					.UncheckedHoveredImage(FAppStyle::Get().GetBrush("Icons.Hidden"))
					.UncheckedPressedImage(FAppStyle::Get().GetBrush("Icons.Hidden"))
				]

				// Is this the picked item?
				+ SHorizontalBox::Slot()
				.MaxWidth(18)
				.Padding(4.0f, 0.0f, 4.0f, 0.0f)
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.EyeDropper"))
					.Visibility_Raw(this, &SDatabaseAssetListItem::GetSelectedActorIconVisibility)
				];
			}
			
			// Setup table row to display database item
			SAssignNew(ItemWidget, SHorizontalBox)
			.Clipping(EWidgetClipping::ClipToBounds)
			+ SHorizontalBox::Slot()
			.Padding(0, 0.0, 0.0, 0.0)
			.FillWidth(1.0f)
			[
				SNew(SSplitter)
				.Style(FAppStyle::Get(), "FoliageEditMode.Splitter")
				.PhysicalSplitterHandleSize(0.0f)
				.HitDetectionSplitterHandleSize(0.0f)
				.MinimumSlotHeight(0.5f)
					
				// Asset Name with type icon
				+ SSplitter::Slot()
				.SizeRule(SSplitter::FractionOfParent)
				[
					SNew(SBorder)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.BorderImage(FStyleDefaults::GetNoBrush())
					[
						SNew(SHorizontalBox)
						.Clipping(EWidgetClipping::ClipToBounds)
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.Padding(0.0f, 0.0f, 0.0f, 0.0f)
						.AutoWidth()
						[
						   BranchInIconWidget.ToSharedRef()
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0.0f, 0.0f, 10.0f, 0.0f)
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						[
							AssetThumbnailOverlay.ToSharedRef()
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.FillWidth(0.7f)
						.Padding(0.0f, 0.0f, 30.0f, 0.0f)
						.HAlign(HAlign_Fill)
						[
							AssetPickerWidget.ToSharedRef()
						]
					]
				]
					
				// Display information via icons
				+SSplitter::Slot()
				.SizeRule(SSplitter::SizeToContent)
				[
					InfoIconsHorizontalBox.ToSharedRef()
				]
			];
		}

		return ItemWidget.ToSharedRef();
	}

	const FSlateBrush* SDatabaseAssetListItem::GetGroupBackgroundImage() const
	{
		if (STableRow<TSharedPtr<FDatabaseAssetTreeNode>>::IsHovered())
		{
			return FAppStyle::Get().GetBrush("Brushes.Secondary");
		}
		else
		{
			return FAppStyle::Get().GetBrush("Brushes.Header");
		}
	}

	EVisibility SDatabaseAssetListItem::GetSelectedActorIconVisibility() const
	{
		TSharedPtr<FDatabaseViewModel> ViewModelPtr = EditorViewModel.Pin();
		if (TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin())
		{
			if (const FSearchIndexAsset* SelectedIndexAsset = ViewModelPtr->GetSelectedActorIndexAsset())
			{
				if (AssetTreeNode->SourceAssetIdx == SelectedIndexAsset->GetSourceAssetIdx())
				{
					return EVisibility::Visible;
				}
			}
		}
		return EVisibility::Hidden;
	}

	void SDatabaseAssetListItem::OnAssetPickerObjectChanged(const FAssetData& AssetData)
	{
		const TSharedPtr<FDatabaseViewModel> ViewModelPtr = EditorViewModel.Pin();

		if (const TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin())
		{
			const FScopedTransaction Transaction(LOCTEXT("Edit Asset", "Edit Asset"));

			ViewModelPtr->SetAnimationAsset(AssetTreeNode->SourceAssetIdx, AssetData.GetAsset());
		}
	}

	FString SDatabaseAssetListItem::GetAssetPickerObjectPath() const
	{
		const TSharedPtr<FDatabaseViewModel> ViewModelPtr = EditorViewModel.Pin();

		if (const TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin())
		{
			if (const UPoseSearchDatabase* Database = ViewModelPtr->GetPoseSearchDatabase())
			{
				if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(AssetTreeNode->SourceAssetIdx))
				{
					if (const UObject* AnimAsset = DatabaseAnimationAsset->GetAnimationAsset())
					{
						return AnimAsset->GetPathName();
					}
				}
			}
		}
		
		return FString("");
	}

	bool SDatabaseAssetListItem::GetAssetPickerIsEnabled() const
	{
		const TSharedPtr<FDatabaseViewModel> ViewModelPtr = EditorViewModel.Pin();

		if (const TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin())
		{
			if (const UPoseSearchDatabase* Database = ViewModelPtr->GetPoseSearchDatabase())
			{
				if (Database->GetDatabaseAnimationAsset(AssetTreeNode->SourceAssetIdx))
				{
					return ViewModelPtr->IsEnabled(AssetTreeNode->SourceAssetIdx);
				}
			}
		}
		
		return false;
	}

	EVisibility SDatabaseAssetListItem::GetAssetPickerCustomContentSlotVisibility() const
	{
		const TSharedPtr<FDatabaseViewModel> ViewModelPtr = EditorViewModel.Pin();
							
		if (const UPoseSearchDatabase* Database = ViewModelPtr->GetPoseSearchDatabase())
		{
			if (const TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin())
			{
				if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAssetBase = Database->GetDatabaseAnimationAsset(AssetTreeNode->SourceAssetIdx))
				{
					if (DatabaseAnimationAssetBase->IsEnabled())
					{
						if (!GetWarningsForDatabaseAsset(*DatabaseAnimationAssetBase, *Database).IsEmpty())
						{
							return EVisibility::Visible;
						}
					}
				}
			}
		}
							
		return EVisibility::Collapsed;
	}

	FText SDatabaseAssetListItem::GetAssetPickerText() const
	{
		const TSharedPtr<FDatabaseViewModel> ViewModelPtr = EditorViewModel.Pin();
							
		if (const UPoseSearchDatabase* Database = ViewModelPtr->GetPoseSearchDatabase())
		{
			if (const TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin())
			{
				if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAssetBase = Database->GetDatabaseAnimationAsset(AssetTreeNode->SourceAssetIdx))
				{
					if (DatabaseAnimationAssetBase->IsEnabled())
					{
						return GetWarningsForDatabaseAsset(*DatabaseAnimationAssetBase, *Database);
					}
				}
			}
		}

		return FText::GetEmpty();
	}

	FText SDatabaseAssetListItem::GetDisableReselectionToolTip() const
	{
		if (GetDisableReselectionChecked() == ECheckBoxState::Checked)
		{
			return LOCTEXT("EnableReselectionToolTip", "Reselection of poses from the same asset is disabled.");
		}
		
		return LOCTEXT("DisableReselectionToolTip", "Reselection of poses from the same asset is enabled.");
	}

	ECheckBoxState SDatabaseAssetListItem::GetDisableReselectionChecked() const
	{
		TSharedPtr<FDatabaseViewModel> ViewModelPtr = EditorViewModel.Pin();
		if (const UPoseSearchDatabase* Database = ViewModelPtr->GetPoseSearchDatabase())
		{
			if (TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin())
			{
				if (Database->GetDatabaseAnimationAsset(AssetTreeNode->SourceAssetIdx))
				{
					if (ViewModelPtr->IsDisableReselection(AssetTreeNode->SourceAssetIdx))
					{
						return ECheckBoxState::Checked;
					}
				}
			}
		}

		return ECheckBoxState::Unchecked;
	}

	void SDatabaseAssetListItem::OnDisableReselectionChanged(ECheckBoxState NewCheckboxState)
	{
		const TSharedPtr<FDatabaseViewModel> ViewModelPtr = EditorViewModel.Pin();
		const TSharedPtr<SDatabaseAssetTree> SkeletonViewPtr = SkeletonView.Pin();
		if (!ViewModelPtr || !SkeletonViewPtr)
		{
			return;
		}

		if (UPoseSearchDatabase* PoseSearchDatabase = ViewModelPtr->GetPoseSearchDatabase())
		{
			TArray<TSharedPtr<FDatabaseAssetTreeNode>> SelectedNodes = SkeletonViewPtr->TreeView->GetSelectedItems();

			const TSharedPtr<FDatabaseAssetTreeNode> ThisNode = WeakAssetTreeNode.Pin();
			if (!ThisNode)
			{
				return;
			}
			if (!SelectedNodes.Contains(ThisNode))
			{
				SelectedNodes = { ThisNode };
			}

			const FScopedTransaction Transaction(LOCTEXT("DisableReselectionChangedForAssetInPoseSearchDatabase", "Update disable reselection flag for item from Pose Search Database"));
			PoseSearchDatabase->Modify();

			for (const TSharedPtr<FDatabaseAssetTreeNode>& Node : SelectedNodes)
			{
				if (Node->SourceAssetIdx != INDEX_NONE)
				{
					ViewModelPtr->SetDisableReselection(Node->SourceAssetIdx, NewCheckboxState == ECheckBoxState::Checked);
				}
			}

			PoseSearchDatabase->PostEditChange();
			SkeletonViewPtr->RefreshTreeView(false, true);
		}
	}

	ECheckBoxState SDatabaseAssetListItem::GetAssetEnabledChecked() const
	{
		TSharedPtr<FDatabaseViewModel> ViewModelPtr = EditorViewModel.Pin();
		if (const UPoseSearchDatabase* Database = ViewModelPtr->GetPoseSearchDatabase())
		{
			if (TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin())
			{
				if (Database->GetDatabaseAnimationAsset(AssetTreeNode->SourceAssetIdx))
				{
					if (ViewModelPtr->IsEnabled(AssetTreeNode->SourceAssetIdx))
					{
						return ECheckBoxState::Checked;
					}
				}
			}
		}
		return ECheckBoxState::Unchecked;
	}

	void SDatabaseAssetListItem::OnAssetIsEnabledChanged(ECheckBoxState NewCheckboxState)
	{
		const TSharedPtr<FDatabaseViewModel> ViewModelPtr = EditorViewModel.Pin();
		const TSharedPtr<SDatabaseAssetTree> SkeletonViewPtr = SkeletonView.Pin();
		if (!ViewModelPtr || !SkeletonViewPtr)
		{
			return;
		}

		if (UPoseSearchDatabase* PoseSearchDatabase = ViewModelPtr->GetPoseSearchDatabase())
		{
			TArray<TSharedPtr<FDatabaseAssetTreeNode>> SelectedNodes = SkeletonViewPtr->TreeView->GetSelectedItems();

			// If the clicked item is not part of the selection, apply only to it
			const TSharedPtr<FDatabaseAssetTreeNode> ThisNode = WeakAssetTreeNode.Pin();
			if (!ThisNode)
			{
				return;
			}
			if (!SelectedNodes.Contains(ThisNode))
			{
				SelectedNodes = { ThisNode };
			}

			const FScopedTransaction Transaction(LOCTEXT("EnableChangedForAssetInPoseSearchDatabase", "Update enabled flag for item from Pose Search Database"));
			PoseSearchDatabase->Modify();

			for (const TSharedPtr<FDatabaseAssetTreeNode>& Node : SelectedNodes)
			{
				if (Node->SourceAssetIdx != INDEX_NONE)
				{
					ViewModelPtr->SetIsEnabled(Node->SourceAssetIdx, NewCheckboxState == ECheckBoxState::Checked);
				}
			}

			PoseSearchDatabase->PostEditChange();
			SkeletonViewPtr->RefreshTreeView(false, true);
			ViewModelPtr->BuildSearchIndex();
		}
	}

	FSlateColor SDatabaseAssetListItem::GetAssetPickerCustomContentSlotTextColor() const
	{
		const TSharedPtr<FDatabaseViewModel> ViewModelPtr = EditorViewModel.Pin();

		if (const UPoseSearchDatabase* Database = ViewModelPtr->GetPoseSearchDatabase())
		{
			if (const TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin())
			{
				if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAssetBase = Database->GetDatabaseAnimationAsset(AssetTreeNode->SourceAssetIdx))
				{
					if (DatabaseAnimationAssetBase->IsEnabled())
					{
						if (!GetWarningsForDatabaseAsset(*DatabaseAnimationAssetBase, *Database).IsEmpty())
						{
							return FColor::Red;
						}
					}
				}
			}
		}
		
		return DisabledColor;
	}

	FSlateColor SDatabaseAssetListItem::GetLoopingColorAndOpacity() const
	{
		if (const TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin())
		{
			if (AssetTreeNode->IsLooping())
			{
				return FLinearColor::White;
			}
		}
		
		return DisabledColor;
	}

	FText SDatabaseAssetListItem::GetLoopingToolTip() const
	{
		if (const TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin())
		{
			if (AssetTreeNode->IsLooping())
			{
				return LOCTEXT("NodeLoopEnabledToolTip", "Looping (Read only)");
			}
		}

		return LOCTEXT("NodeLoopDisabledToolTip", "Not looping (Read only)");
	}

	FSlateColor SDatabaseAssetListItem::GetRootMotionColorAndOpacity() const
	{
		if (const TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin())
		{
			if (AssetTreeNode->IsRootMotionEnabled())
			{
				return FLinearColor::White;
			}
		}
		
		return DisabledColor;
	}

	FText SDatabaseAssetListItem::GetRootMotionOptionToolTip() const
	{
		if (const TSharedPtr<FDatabaseAssetTreeNode> AssetTreeNode = WeakAssetTreeNode.Pin())
		{
			if (AssetTreeNode->IsRootMotionEnabled())
			{
				return LOCTEXT("NodeRootMotionEnabledToolTip", "Root motion enabled (Read only)");
			}
		}
		
		return LOCTEXT("NodeRootMotionDisabledToolTip", "No root motion enabled (Read only)");
	}

	FText SDatabaseAssetListItem::GetAssetEnabledToolTip() const
	{
		if (GetAssetEnabledChecked() == ECheckBoxState::Checked)
		{
			return LOCTEXT("DisableAssetTooltip", "Disable this asset in the Pose Search Database.");
		}
		
		return LOCTEXT("EnableAssetTooltip", "Enable this asset in the Pose Search Database.");
	}
}

#undef LOCTEXT_NAMESPACE
