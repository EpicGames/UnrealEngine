// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDatabaseAssetTree.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "AnimationBlueprintLibrary.h"
#include "AssetSelection.h"
#include "ClassIconFinder.h"
#include "DetailColumnSizeData.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/FeedbackContext.h"
#include "Misc/TransactionObjectEvent.h"
#include "PoseSearch/MultiAnimAsset.h"
#include "PoseSearch/PoseSearchAnimNotifies.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearchDatabaseEditorClipboard.h"
#include "PoseSearchDatabaseViewModel.h"
#include "PoseSearchEditor.h"
#include "SPositiveActionButton.h"
#include "Styling/AppStyle.h"
#include "ScopedTransaction.h"
#include "Styling/StyleColors.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"

#define LOCTEXT_NAMESPACE "SDatabaseAssetTree"

namespace UE::PoseSearch
{
	void SDatabaseAssetTree::Construct(
		const FArguments& InArgs, 
		TSharedRef<FDatabaseViewModel> InEditorViewModel)
	{
		EditorViewModel = InEditorViewModel;

		CreateCommandList();

		ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0, 0, 4, 0)
				[
					SNew(SPositiveActionButton)
					.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
					.Text(LOCTEXT("AddNew", "Add"))
					.ToolTipText(LOCTEXT("AddNewAssetToolTip", "Add a new asset to the database"))
					.OnClicked_Lambda([this]()
					{
						OnAddAsset(true);
						return FReply::Handled();
					})
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.Padding(2, 0, 0, 0)
				[
					GenerateFilterBoxWidget()
				]
			]
			+SVerticalBox::Slot()
			.Padding(0.0f, 0.0f)
			[
				SNew(SBorder)
				.Padding(2.0f)
				.BorderImage(FAppStyle::GetBrush("SCSEditor.TreePanel"))
				[
					SNew(SOverlay)
					+SOverlay::Slot()
					[
						SAssignNew(TreeView, STreeView<TSharedPtr<FDatabaseAssetTreeNode>>)
						.TreeItemsSource(&RootNodes)
						.SelectionMode(ESelectionMode::Multi)
						.OnGenerateRow(this, &SDatabaseAssetTree::MakeTableRowWidget)
						.OnGetChildren(this, &SDatabaseAssetTree::HandleGetChildrenForTree)
						.OnContextMenuOpening(this, &SDatabaseAssetTree::CreateContextMenu)
						.HighlightParentNodesForSelection(false)
						.OnSelectionChanged_Lambda([this](TSharedPtr<FDatabaseAssetTreeNode> Item, ESelectInfo::Type Type)
							{
								TArray<TSharedPtr<FDatabaseAssetTreeNode>> SelectedItems = TreeView->GetSelectedItems();
								OnSelectionChanged.Broadcast(SelectedItems, Type);
							})
					]
					+SOverlay::Slot()
					[
						SAssignNew(TreeViewDragAndDropSuggestion, SVerticalBox)
						+ SVerticalBox::Slot()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("DragDropHint", "Drag and drop Animation Sequences, Anim Composites, Blendspaces, or Anim Montages"))
							.Font(FAppStyle::Get().GetFontStyle("DetailsView.CategoryFontStyle"))
						]
					]
				]
			]
		];

		RefreshTreeView(true);
	}

	bool SDatabaseAssetTree::SupportsKeyboardFocus() const
	{
		return true;
	}

	FReply SDatabaseAssetTree::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		// When the database is empty the STreeView has no rows, so STableRow::OnMouseButtonDown never fires
		// and focus is never acquired. Claim focus here so that keyboard shortcuts (e.g. Ctrl+V) work even
		// on an empty database. This override is only reached when no deeper widget handled the click, which
		// is exactly the empty-tree case; non-empty clicks are consumed by STableRow before bubbling here.
		return FReply::Handled().SetUserFocus(SharedThis(this), EFocusCause::Mouse);
	}

	FReply SDatabaseAssetTree::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		FReply Reply = FReply::Unhandled();

		TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();

		const bool bValidOperation =
			Operation.IsValid() &&
			(Operation->IsOfType<FExternalDragOperation>() || Operation->IsOfType<FAssetDragDropOp>());
		if (bValidOperation)
		{
			Reply = AssetUtil::CanHandleAssetDrag(DragDropEvent);

			if (!Reply.IsEventHandled())
			{
				if (Operation->IsOfType<FAssetDragDropOp>())
				{
					const TSharedPtr<FAssetDragDropOp> AssetDragDropOp = StaticCastSharedPtr<FAssetDragDropOp>(Operation);

					for (const FAssetData& AssetData : AssetDragDropOp->GetAssets())
					{
						if (UClass* AssetClass = AssetData.GetClass())
						{
							if (AssetClass->IsChildOf(UAnimSequence::StaticClass()) ||
								AssetClass->IsChildOf(UAnimComposite::StaticClass()) ||
								AssetClass->IsChildOf(UBlendSpace::StaticClass()) ||
								AssetClass->IsChildOf(UAnimMontage::StaticClass()) ||
								AssetClass->IsChildOf(UMultiAnimAsset::StaticClass()))
							{
								Reply = FReply::Handled();
								break;
							}
						}
					}
				}
			}
		}

		return Reply;
	}

	FReply SDatabaseAssetTree::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		return OnAcceptDrop(DragDropEvent, EItemDropZone::OntoItem, nullptr);
	}

	FReply SDatabaseAssetTree::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		if (CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}
		return FReply::Unhandled();
	}

	bool SDatabaseAssetTree::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const
	{
		// Ensure that we only react to modifications to the UPosesSearchDatabase.
		if (const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin())
		{
			if (const UPoseSearchDatabase* Database = ViewModel->GetPoseSearchDatabase())
			{
				for (const TPair<UObject*, FTransactionObjectEvent>& TransactionObjectPair : TransactionObjectContexts)
				{
					const UObject* Object = TransactionObjectPair.Key;
					while (Object != nullptr)
					{
						if (Object == Database)
						{
							return true;
						}

						Object = Object->GetOuter();
					}
				}
			}
		}
		
		return false;
	}

	void SDatabaseAssetTree::PostUndo(bool bSuccess)
	{
		if (bSuccess)
		{
			FinalizeTreeChanges();
		}
	}

	void SDatabaseAssetTree::PostRedo(bool bSuccess)
	{
		if (bSuccess)
		{
			FinalizeTreeChanges();
		}
	}

	void SDatabaseAssetTree::RefreshTreeView(bool bIsInitialSetup, bool bRecoverSelection)
	{
		const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin();
		if (!ViewModel.IsValid())
		{
			return;
		}

		const TSharedRef<FDatabaseViewModel> ViewModelRef = ViewModel.ToSharedRef();

		// Empty node data.
		RootNodes.Reset();
		//AllNodes.Reset();

		const UPoseSearchDatabase* Database = ViewModel->GetPoseSearchDatabase();
		if (!IsValid(Database))
		{
			TreeView->RequestTreeRefresh();
			return;
		}

		// Store selection so we can recover it afterwards (if possible)
		TArray<TSharedPtr<FDatabaseAssetTreeNode>> PreviouslySelectedNodes = TreeView->GetSelectedItems();

		// Rebuild node hierarchy
		{
			// Build an index based off of alphabetical order than iterate the index instead
			TArray<uint32> IndexArray;
			IndexArray.SetNumUninitialized(Database->GetNumAnimationAssets());
			for (int32 AnimationAssetIdx = 0; AnimationAssetIdx < Database->GetNumAnimationAssets(); ++AnimationAssetIdx)
			{
				IndexArray[AnimationAssetIdx] = AnimationAssetIdx;
			}

			IndexArray.Sort([Database](int32 SequenceIdxA, int32 SequenceIdxB)
			{
				const FPoseSearchDatabaseAnimationAssetBase* A = Database->GetDatabaseAnimationAsset(SequenceIdxA);
				if (!A)
				{
					return false;
				}

				const FPoseSearchDatabaseAnimationAssetBase* B = Database->GetDatabaseAnimationAsset(SequenceIdxB);
				if (!B)
				{
					return true;
				}

				//If its null add it to the end of the list 
				if (!B->GetAnimationAsset())
				{
					return true;
				}

				if (!A->GetAnimationAsset())
				{
					return false;
				}

				const int32 Comparison = A->GetName().Compare(B->GetName());
				return Comparison < 0;
			});

			// create all nodes
			for (int32 AnimationAssetIdx = 0; AnimationAssetIdx < Database->GetNumAnimationAssets(); ++AnimationAssetIdx)
			{
				const int32 MappedId = IndexArray[AnimationAssetIdx];

				if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(MappedId))
				{
					const bool bFiltered = (DatabaseAnimationAsset->GetAnimationAsset() == nullptr || GetAssetFilterString().IsEmpty()) ? false : !DatabaseAnimationAsset->GetName().Contains(GetAssetFilterString());

					if (!bFiltered)
					{
						// Create sequence node
						const TSharedPtr<FDatabaseAssetTreeNode> SequenceGroupNode = MakeShared<FDatabaseAssetTreeNode>(MappedId, ViewModelRef);

						// Keep track of node
						RootNodes.Add(SequenceGroupNode);
						//AllNodes.Add(SequenceGroupNode);
					}
				}
			}

			// Show drag and drop suggestion if tree is empty
			TreeViewDragAndDropSuggestion->SetVisibility(IndexArray.IsEmpty() ? EVisibility::Visible : EVisibility::Hidden);
		}

		// Update tree view
		TreeView->RequestTreeRefresh();

		for (TSharedPtr<FDatabaseAssetTreeNode>& RootNode : RootNodes)
		{
			TreeView->SetItemExpansion(RootNode, true);
		}

		// Handle selection
		if (bRecoverSelection)
		{
			RecoverSelection(PreviouslySelectedNodes);
		}
		else
		{
			TreeView->SetItemSelection(PreviouslySelectedNodes, false, ESelectInfo::Direct);
		}
	}

	TSharedRef<ITableRow> SDatabaseAssetTree::MakeTableRowWidget(
		TSharedPtr<FDatabaseAssetTreeNode> InItem,
		const TSharedRef<STableViewBase>& OwnerTable)
	{
		return InItem->MakeTreeRowWidget(OwnerTable, InItem.ToSharedRef(), CommandList.ToSharedRef(), SharedThis(this));
	}

	void SDatabaseAssetTree::HandleGetChildrenForTree(
		TSharedPtr<FDatabaseAssetTreeNode> InNode,
		TArray<TSharedPtr<FDatabaseAssetTreeNode>>& OutChildren)
	{
		OutChildren = InNode.Get()->Children;
	}

	TOptional<EItemDropZone> SDatabaseAssetTree::OnCanAcceptDrop(
		const FDragDropEvent& DragDropEvent,
		EItemDropZone DropZone,
		TSharedPtr<FDatabaseAssetTreeNode> TargetItem)
	{
		TOptional<EItemDropZone> ReturnedDropZone;

		TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();

		const bool bValidOperation = Operation.IsValid() && Operation->IsOfType<FAssetDragDropOp>();
		if (bValidOperation)
		{
			const TSharedPtr<FAssetDragDropOp> AssetDragDropOp = StaticCastSharedPtr<FAssetDragDropOp>(Operation);

			for (const FAssetData& AssetData : AssetDragDropOp->GetAssets())
			{
				if (UClass* AssetClass = AssetData.GetClass())
				{
					if (AssetClass->IsChildOf(UAnimSequence::StaticClass()) ||
						AssetClass->IsChildOf(UAnimComposite::StaticClass()) ||
						AssetClass->IsChildOf(UBlendSpace::StaticClass()) ||
						AssetClass->IsChildOf(UAnimMontage::StaticClass()) ||
						AssetClass->IsChildOf(UMultiAnimAsset::StaticClass()))
					{
						ReturnedDropZone = EItemDropZone::OntoItem;
						break;
					}
				}
			}
		}

		return ReturnedDropZone;
	}

	FReply SDatabaseAssetTree::OnAcceptDrop(
		const FDragDropEvent& DragDropEvent,
		EItemDropZone DropZone,
		TSharedPtr<FDatabaseAssetTreeNode> TargetItem)
	{
		const TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();

		const bool bValidOperation = Operation.IsValid() && Operation->IsOfType<FAssetDragDropOp>();
		if (!bValidOperation)
		{
			return FReply::Unhandled();
		}

		const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin();
		if (!ViewModel.IsValid())
		{
			return FReply::Unhandled();
		}

		TArray<FAssetData> DroppedAssetData = AssetUtil::ExtractAssetDataFromDrag(Operation);
		const int32 NumAssets = DroppedAssetData.Num();

		int32 AddedAssets = 0;
		UPoseSearchDatabase* PoseSearchDatabase = ViewModel->GetPoseSearchDatabase();
		if (PoseSearchDatabase && NumAssets > 0)
		{
			GWarn->BeginSlowTask(LOCTEXT("LoadingAssets", "Loading Asset(s)"), true);

			const FScopedTransaction Transaction(LOCTEXT("AddAssetsOnDrop", "Add Animation Asset(s) to Pose Search Database"));
			PoseSearchDatabase->Modify();
			
			for (int32 DroppedAssetIdx = 0; DroppedAssetIdx < NumAssets; ++DroppedAssetIdx)
			{
				const FAssetData& AssetData = DroppedAssetData[DroppedAssetIdx];

				if (!AssetData.IsAssetLoaded())
				{
					GWarn->StatusUpdate(
						DroppedAssetIdx,
						NumAssets,
						FText::Format(
							LOCTEXT("LoadingAsset", "Loading Asset {0}"),
							FText::FromName(AssetData.AssetName)));
				}

				UClass* AssetClass = AssetData.GetClass();
				UObject* Asset = AssetData.GetAsset();
					
				ViewModel->AddAnimationAssetToDatabase(Asset);
				++AddedAssets;
			}
			
			PoseSearchDatabase->PostEditChange();

			GWarn->EndSlowTask();
		}

		if (AddedAssets == 0)
		{
			return FReply::Unhandled();
		}

		FinalizeTreeChanges(false);
		return FReply::Handled();
	}

	TSharedPtr<SWidget> SDatabaseAssetTree::CreateContextMenu()
	{
		const bool CloseAfterSelection = true;
		FMenuBuilder MenuBuilder(CloseAfterSelection, CommandList);

		const TArray<TSharedPtr<FDatabaseAssetTreeNode>> SelectedNodes = TreeView->GetSelectedItems();
		if (!SelectedNodes.IsEmpty())
		{
			MenuBuilder.BeginSection("SelectedAssetsEdit", LOCTEXT("SelectedAssetEdit", "Asset Actions"));
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("ConvertToBranchIn", "Convert selected assets to sample via BranchIn notify"),
					LOCTEXT("ConvertToBranchInToolTip", "Creates PoseSearchBranchIn notify state for the asset sampling range"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &SDatabaseAssetTree::OnConvertToBranchIn)),
					NAME_None,
					EUserInterfaceActionType::Button);	
			}
			MenuBuilder.EndSection();

			MenuBuilder.BeginSection("SelectionAssetsClipboardEdit", LOCTEXT("SelectionAssetsClipboardEdit", "Edit"));
			{
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
			}
			MenuBuilder.EndSection();
		}
		else
		{
			// Asset actions
			MenuBuilder.BeginSection("Edit", LOCTEXT("EditSection", "Edit"));
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("AddAssetOption", "Add New Asset"),
					LOCTEXT("AddAssetToDatabaseTooltip", "Add new asset to the database"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &SDatabaseAssetTree::OnAddAsset, true)),
					NAME_None,
					EUserInterfaceActionType::Button);
			}
			MenuBuilder.EndSection();
			
			// Edit / Clipboard actions
			MenuBuilder.BeginSection("Clipboard", LOCTEXT("ClipboardSection", "Clipboard"));
			{
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
			}
			
			MenuBuilder.EndSection();
		}

		return MenuBuilder.MakeWidget();
	}

	TSharedRef<SWidget> SDatabaseAssetTree::GenerateFilterBoxWidget()
	{
		TSharedPtr<SSearchBox> SearchBox;
		SAssignNew(SearchBox, SSearchBox)
			.MinDesiredWidth(300.0f)
			.InitialText(this, &SDatabaseAssetTree::GetFilterText)
			.ToolTipText(LOCTEXT("AssetFilterTooltip", "Enter Asset Filter..."))
			.OnTextChanged(this, &SDatabaseAssetTree::OnAssetFilterTextCommitted, ETextCommit::Default)
			.OnTextCommitted(this, &SDatabaseAssetTree::OnAssetFilterTextCommitted);

		return SearchBox.ToSharedRef();
	}


	FText SDatabaseAssetTree::GetFilterText() const
	{
		return FText::FromString(GetAssetFilterString());
	}

	void SDatabaseAssetTree::OnAssetFilterTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
	{
		SetAssetFilterString(InText.ToString());
		RefreshTreeView(false);
	}


	void SDatabaseAssetTree::OnAddAsset(bool bFinalizeChanges)
	{
		FScopedTransaction Transaction(LOCTEXT("AddAsset", "Add Asset"));

		EditorViewModel.Pin()->AddAnimationAssetToDatabase(nullptr);

		if (bFinalizeChanges)
		{
			FinalizeTreeChanges();
		}
		
	}

	void SDatabaseAssetTree::OnDeleteAsset(TSharedPtr<FDatabaseAssetTreeNode> Node, bool bFinalizeChanges)
	{
		FScopedTransaction Transaction(LOCTEXT("DeleteAsset", "Delete Asset"));
		const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin();
		
		if (ViewModel->DeleteFromDatabase(Node->SourceAssetIdx) && bFinalizeChanges)
		{
			FinalizeTreeChanges();
		}
	}

	void SDatabaseAssetTree::RegisterOnSelectionChanged(const FOnSelectionChanged& Delegate)
	{
		OnSelectionChanged.Add(Delegate);
	}

	void SDatabaseAssetTree::UnregisterOnSelectionChanged(FDelegateUserObject Unregister)
	{
		OnSelectionChanged.RemoveAll(Unregister);
	}

	void SDatabaseAssetTree::RecoverSelection(const TArray<TSharedPtr<FDatabaseAssetTreeNode>>& PreviouslySelectedNodes)
	{
		TArray<TSharedPtr<FDatabaseAssetTreeNode>> NewSelectedNodes;

		for (const TSharedPtr<FDatabaseAssetTreeNode>& Node : RootNodes)
		{
			const bool bFoundNode = PreviouslySelectedNodes.ContainsByPredicate([Node](const TSharedPtr<FDatabaseAssetTreeNode>& PrevSelectedNode) { return PrevSelectedNode->SourceAssetIdx == Node->SourceAssetIdx; });
			if (bFoundNode)
			{
				NewSelectedNodes.Add(Node);
			}
		}

		// @todo: investigate if we should call a TreeView->ClearSelection() before TreeView->SetItemSelection
		TreeView->SetItemSelection(NewSelectedNodes, true, ESelectInfo::Direct);
	}

	void SDatabaseAssetTree::CreateCommandList()
	{
		CommandList = MakeShared<FUICommandList>();
		
		CommandList->MapAction(
			FGenericCommands::Get().Delete,
			FUIAction(
				FExecuteAction::CreateSP(this, &SDatabaseAssetTree::OnDeleteNodes),
				FCanExecuteAction::CreateSP(this, &SDatabaseAssetTree::CanDeleteNodes)));
		
		CommandList->MapAction(
		FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SDatabaseAssetTree::OnCopySelectedNodesToClipboard),
		FCanExecuteAction::CreateSP(this, &SDatabaseAssetTree::CanCopyToClipboard));

		CommandList->MapAction(
			FGenericCommands::Get().Paste,
			FExecuteAction::CreateSP(this, &SDatabaseAssetTree::OnPasteNodesFromClipboard),
			FCanExecuteAction::CreateSP(this, &SDatabaseAssetTree::CanPasteFromClipboard));
	
		CommandList->MapAction(
			FGenericCommands::Get().Cut,
			FExecuteAction::CreateSP(this, &SDatabaseAssetTree::OnCutSelectedNodesToClipboard),
			FCanExecuteAction::CreateSP(this, &SDatabaseAssetTree::CanCutToClipboard));
	}

	bool SDatabaseAssetTree::CanDeleteNodes() const
	{
		TArray<TSharedPtr<FDatabaseAssetTreeNode>> SelectedNodes = TreeView->GetSelectedItems();
		for (const TSharedPtr<FDatabaseAssetTreeNode>& SelectedNode : SelectedNodes)
		{
			if (SelectedNode->SourceAssetIdx != INDEX_NONE)
			{
				return true;
			}
		}

		return false;
	}

	void SDatabaseAssetTree::OnDeleteNodes()
	{
		TArray<TSharedPtr<FDatabaseAssetTreeNode>> SelectedNodes = TreeView->GetSelectedItems();
		if (!SelectedNodes.IsEmpty())
		{
			const FScopedTransaction Transaction(LOCTEXT("DeletePoseSearchDatabaseNodes", "Delete selected item(s) from Pose Search Database"));
			const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin();

			SelectedNodes.Sort([](const TSharedPtr<FDatabaseAssetTreeNode>& A, const TSharedPtr<FDatabaseAssetTreeNode>& B)
				{
					return B->SourceAssetIdx < A->SourceAssetIdx;
				});

			for (const TSharedPtr<FDatabaseAssetTreeNode>& SelectedNode : SelectedNodes)
			{
				if (SelectedNode->SourceAssetIdx != INDEX_NONE)
				{
					OnDeleteAsset(SelectedNode, false);
				}
			}

			ViewModel->RemovePreviewActors();
			FinalizeTreeChanges();
		}
	}

	void SDatabaseAssetTree::OnCopySelectedNodesToClipboard() const
	{
		TArray<TSharedPtr<FDatabaseAssetTreeNode>> SelectedNodes = TreeView->GetSelectedItems();
		
		if (!SelectedNodes.IsEmpty())
		{
			if (UPoseSearchDatabaseEditorClipboardContent* ClipboardContent = UPoseSearchDatabaseEditorClipboardContent::Create())
			{
				const FScopedTransaction Transaction(LOCTEXT("CopyPoseSearchDatabaseNodes", "Copy selected item(s) from Pose Search Database"));
				const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin();
				
				for (const TSharedPtr<FDatabaseAssetTreeNode>& SelectedNode : SelectedNodes)
				{
					if (SelectedNode->SourceAssetIdx != INDEX_NONE)
					{
						if (UPoseSearchDatabase* Database = ViewModel->GetPoseSearchDatabase())
						{
							if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(SelectedNode->SourceAssetIdx))
							{
								// @todo: Support copying assets added via BranchIn notifies. 
								if (!DatabaseAnimationAsset->IsSynchronizedWithExternalDependency())
								{
									ClipboardContent->CopyDatabaseItem(DatabaseAnimationAsset);
								}
								else
								{
									UE_LOGF(LogPoseSearchEditor, Log, "Failed to copy %ls. Asset(s) with BranchIn notifies do not have clipboard support.", *DatabaseAnimationAsset->GetName())
								}
							}
						}
					}
				}
			
				ClipboardContent->CopyToClipboard();
			}
			else
			{
				UE_LOGF(LogPoseSearchEditor, Warning, "Failed create clipboard object while attempting to copy data");
			}
		}
	}

	bool SDatabaseAssetTree::CanCopyToClipboard() const
	{
		return !TreeView->GetSelectedItems().IsEmpty();
	}

	void SDatabaseAssetTree::OnPasteNodesFromClipboard()
	{
		const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin();
		
		if (const UPoseSearchDatabaseEditorClipboardContent* ClipboardContent = UPoseSearchDatabaseEditorClipboardContent::CreateFromClipboard())
		{
			if (UPoseSearchDatabase* Database = ViewModel->GetPoseSearchDatabase())
			{
				const FScopedTransaction Transaction(LOCTEXT("PastePoseSearchDatabaseNodes", "Paste item(s) to Pose Search Database"));
			
				ClipboardContent->PasteToDatabase(Database);

				FinalizeTreeChanges();
			}
		}
		else
		{
			UE_LOGF(LogPoseSearchEditor, Warning, "Failed to get valid clipboard data while attempting to paste data");
		}
	}

	bool SDatabaseAssetTree::CanPasteFromClipboard()
	{
		const UPoseSearchDatabaseEditorClipboardContent* ClipboardContent = UPoseSearchDatabaseEditorClipboardContent::CreateFromClipboard();
		return ClipboardContent && !ClipboardContent->DatabaseItems.IsEmpty();
	}

	void SDatabaseAssetTree::OnCutSelectedNodesToClipboard()
	{
		const FScopedTransaction Transaction(LOCTEXT("CutPoseSearchDatabaseNodes", "Cut selected item(s) from Pose Search Database"));
		
		OnCopySelectedNodesToClipboard();

		// @todo: Following code can be replaced with OnDeleteNodes() call once assets with external dependencies support copying/pasting.
		TArray<TSharedPtr<FDatabaseAssetTreeNode>> SelectedNodes = TreeView->GetSelectedItems();
		if (!SelectedNodes.IsEmpty())
		{
			const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin();

			SelectedNodes.Sort([](const TSharedPtr<FDatabaseAssetTreeNode>& A, const TSharedPtr<FDatabaseAssetTreeNode>& B)
				{
					return B->SourceAssetIdx < A->SourceAssetIdx;
				});

			for (const TSharedPtr<FDatabaseAssetTreeNode>& SelectedNode : SelectedNodes)
			{
				if (SelectedNode->SourceAssetIdx != INDEX_NONE)
				{
					if (UPoseSearchDatabase* Database = ViewModel->GetPoseSearchDatabase())
					{
						const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(SelectedNode->SourceAssetIdx);
						
						if (DatabaseAnimationAsset && !DatabaseAnimationAsset->IsSynchronizedWithExternalDependency())
						{
							OnDeleteAsset(SelectedNode, false);
						}
					}
				}
			}

			ViewModel->RemovePreviewActors();
			FinalizeTreeChanges();
		}
	}

	bool SDatabaseAssetTree::CanCutToClipboard() const
	{
		return CanCopyToClipboard() && CanDeleteNodes();
	}

	void SDatabaseAssetTree::EnableSelectedNodes(bool bIsEnabled)
	{
		const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin();
		if (UPoseSearchDatabase* PoseSearchDatabase = ViewModel->GetPoseSearchDatabase())
		{
			TArray<TSharedPtr<FDatabaseAssetTreeNode>> SelectedNodes = TreeView->GetSelectedItems();
			if (!SelectedNodes.IsEmpty())
			{
				const FText TransactionName = bIsEnabled ? LOCTEXT("EnablePoseSearchDatabaseNodes", "Enable selected items from Pose Search Database") : LOCTEXT("DisablePoseSearchDatabaseNodes", "Disable selected items from Pose Search Database");
				const FScopedTransaction Transaction(TransactionName);

				PoseSearchDatabase->Modify();

				for (const TSharedPtr<FDatabaseAssetTreeNode>& SelectedNode : SelectedNodes)
				{
					ViewModel->SetIsEnabled(SelectedNode->SourceAssetIdx, bIsEnabled);
				}

				PoseSearchDatabase->PostEditChange();

				FinalizeTreeChanges();
			}
		}
	}

	void SDatabaseAssetTree::OnSetMirrorOptionForSelectedNodes(EPoseSearchMirrorOption InMirrorOption)
	{
		const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin();
		if (UPoseSearchDatabase* PoseSearchDatabase = ViewModel->GetPoseSearchDatabase())
		{
			TArray<TSharedPtr<FDatabaseAssetTreeNode>> SelectedNodes = TreeView->GetSelectedItems();
			if (!SelectedNodes.IsEmpty())
			{
				const FScopedTransaction Transaction(LOCTEXT("OnClickEditMirrorOptionPoseSearchDatabase", "Edit Mirror Option on selected items"));
				
				PoseSearchDatabase->Modify();

				for (const TSharedPtr<FDatabaseAssetTreeNode>& SelectedNode : SelectedNodes)
				{
					ViewModel->SetMirrorOption(SelectedNode->SourceAssetIdx, InMirrorOption);
				}

				PoseSearchDatabase->PostEditChange();

				FinalizeTreeChanges();
			}
		}
	}

	void SDatabaseAssetTree::OnSetPoseReselectionForSelectedNodes(bool bIsEnabled)
	{
		const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin();
		if (UPoseSearchDatabase* PoseSearchDatabase = ViewModel->GetPoseSearchDatabase())
		{
			TArray<TSharedPtr<FDatabaseAssetTreeNode>> SelectedNodes = TreeView->GetSelectedItems();
			if (!SelectedNodes.IsEmpty())
			{
				const FScopedTransaction Transaction(LOCTEXT("OnClickEditPoseReselection", "Set pose reselection for selected items"));
				
				PoseSearchDatabase->Modify();

				for (const TSharedPtr<FDatabaseAssetTreeNode>& SelectedNode : SelectedNodes)
				{
					ViewModel->SetDisableReselection(SelectedNode->SourceAssetIdx, !bIsEnabled);
				}

				PoseSearchDatabase->PostEditChange();

				RefreshTreeView(false, true);
			}
		}
	}

	void SDatabaseAssetTree::OnConvertToBranchIn()
	{
		const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin();
		if (UPoseSearchDatabase* PoseSearchDatabase = ViewModel->GetPoseSearchDatabase())
		{
			TArray<TSharedPtr<FDatabaseAssetTreeNode>> SelectedNodes = TreeView->GetSelectedItems();
			if (!SelectedNodes.IsEmpty())
			{
				const FScopedTransaction Transaction(LOCTEXT("ConvertToBranchInTransaction", "Create PoseSearchBranchIn notify state for assets in Pose Search Database"));

				bool bModified = false;
				
				for (const TSharedPtr<FDatabaseAssetTreeNode>& SelectedNode : SelectedNodes)
				{
					if (const FPoseSearchDatabaseAnimationAsset* DatabaseAnimationAsset = PoseSearchDatabase->GetDatabaseAnimationAsset(SelectedNode->SourceAssetIdx))
					{
						if (UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(DatabaseAnimationAsset->GetAnimationAsset()))
						{
							if (!bModified)
							{
								PoseSearchDatabase->Modify();
								bModified = true;
							}
							
							AnimSequenceBase->Modify();

							const FFloatInterval SamplingRange = DatabaseAnimationAsset->GetEffectiveSamplingRange(FVector::ZeroVector);
							const float StartTime = SamplingRange.Min;
							const float Duration = SamplingRange.Max - SamplingRange.Min;
							const FName TrackName = "PoseSearch";
							
							if (!UAnimationBlueprintLibrary::IsValidAnimNotifyTrackName(AnimSequenceBase, TrackName))
							{
								UAnimationBlueprintLibrary::AddAnimationNotifyTrack(AnimSequenceBase, TrackName, FColor::Turquoise);
							}

							UAnimNotifyState_PoseSearchBranchIn* PoseSearchBranchIn = CastChecked<UAnimNotifyState_PoseSearchBranchIn>(UAnimationBlueprintLibrary::AddAnimationNotifyStateEvent(AnimSequenceBase, TrackName, StartTime, Duration, UAnimNotifyState_PoseSearchBranchIn::StaticClass()));
							PoseSearchBranchIn->Database = PoseSearchDatabase;
						
							AnimSequenceBase->PostEditChange();

							FPoseSearchDatabaseAnimationAsset NewDatabaseAnimationAsset = *DatabaseAnimationAsset;
							NewDatabaseAnimationAsset.BranchInId = PoseSearchBranchIn->GetBranchInId();

							PoseSearchDatabase->SetAnimationAssetAt(NewDatabaseAnimationAsset, SelectedNode->SourceAssetIdx);
						}
					}
				}

				if (bModified)
				{
					PoseSearchDatabase->PostEditChange();
				}

				PoseSearchDatabase->SynchronizeWithExternalDependencies();

				FinalizeTreeChanges();
			}
		}
	}

	void SDatabaseAssetTree::FinalizeTreeChanges(bool bRecoverSelection, bool bRefreshView)
	{
		if (bRefreshView)
		{
			RefreshTreeView(false, bRecoverSelection);
		}

		EditorViewModel.Pin()->BuildSearchIndex();
	}

	void SDatabaseAssetTree::SetSelectedItem(int32 SourceAssetIdx, bool bClearSelection)
	{
		if (bClearSelection)
		{
			TreeView->ClearSelection();
		}

		if (SourceAssetIdx >= 0)
		{
			for (TSharedPtr<FDatabaseAssetTreeNode>& Node : RootNodes)
			{
				if (Node->SourceAssetIdx == SourceAssetIdx)
				{
					TreeView->SetItemSelection(Node, true);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
