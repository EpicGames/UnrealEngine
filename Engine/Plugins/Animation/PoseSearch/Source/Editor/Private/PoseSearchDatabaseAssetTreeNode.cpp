// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDatabaseAssetTreeNode.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "AssetSelection.h"
#include "ClassIconFinder.h"
#include "DetailColumnSizeData.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/FeedbackContext.h"
#include "Misc/TransactionObjectEvent.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearchDatabaseAssetListItem.h"
#include "PoseSearchDatabaseViewModel.h"
#include "ScopedTransaction.h"
#include "SPositiveActionButton.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Views/SListView.h"

namespace UE::PoseSearch
{
	FDatabaseAssetTreeNode::FDatabaseAssetTreeNode(int32 InSourceAssetIdx, const TSharedRef<FDatabaseViewModel>& InEditorViewModel) 
		: SourceAssetIdx(InSourceAssetIdx)
		, EditorViewModel(InEditorViewModel)
	{
	}

	TSharedRef<ITableRow> FDatabaseAssetTreeNode::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FDatabaseAssetTreeNode> InDatabaseAssetNode, TSharedRef<FUICommandList> InCommandList, TSharedPtr<SDatabaseAssetTree> InHierarchy)
	{
		return SNew(SDatabaseAssetListItem, EditorViewModel.Pin().ToSharedRef(), InOwnerTable, InDatabaseAssetNode, InCommandList, InHierarchy);
	}

	bool FDatabaseAssetTreeNode::IsRootMotionEnabled() const
	{
		if (const UPoseSearchDatabase* Database = EditorViewModel.Pin()->GetPoseSearchDatabase())
		{
			if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(SourceAssetIdx))
			{
				return DatabaseAnimationAsset->IsRootMotionEnabled();
			}
		}

		return false;
	}

	bool FDatabaseAssetTreeNode::IsLooping() const
	{
		if (const UPoseSearchDatabase* Database = EditorViewModel.Pin()->GetPoseSearchDatabase())
		{
			if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(SourceAssetIdx))
			{
				return DatabaseAnimationAsset->IsLooping();
			}
		}

		return false;
	}

	EPoseSearchMirrorOption FDatabaseAssetTreeNode::GetMirrorOption() const
	{
		if (const UPoseSearchDatabase* Database = EditorViewModel.Pin()->GetPoseSearchDatabase())
		{
			if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(SourceAssetIdx))
			{
				return DatabaseAnimationAsset->GetMirrorOption();
			}
		}

		return EPoseSearchMirrorOption::UnmirroredOnly;
	}
}
