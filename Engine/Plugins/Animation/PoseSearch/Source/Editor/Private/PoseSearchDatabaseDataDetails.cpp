// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDatabaseDataDetails.h"
#include "Animation/AnimationAsset.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "PoseSearch/PoseSearchDerivedData.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearch/PoseSearchSettings.h"
#include "PoseSearchDatabaseViewModel.h"
#include "Widgets/Layout/SScrollBox.h"

#define LOCTEXT_NAMESPACE "PoseSearchDatabaseDataDetails"

namespace UE::PoseSearch
{

class FChannelItem
{
public:
	FChannelItem(const UPoseSearchFeatureChannel* Channel, int32 ChannelComponentIdx, const FSearchIndex& SearchIndex)
	: FullLabel(ComputeLabel(Channel, ChannelComponentIdx, true))
	, CompactLabel(ComputeLabel(Channel, ChannelComponentIdx, false))
	, DataOffset(ComputeDataOffset(Channel, ChannelComponentIdx))
	, Cardinality(ComputeCardinality(Channel, ChannelComponentIdx))
	, Weight(ComputeWeight(Channel, ChannelComponentIdx, SearchIndex))
	{
	}

	FChannelItem(const FString& InFullLabel, const FString& InCompactLabel)
	: FullLabel(InFullLabel)
	, CompactLabel(InCompactLabel)
	, DataOffset(INDEX_NONE)
	, Cardinality(INDEX_NONE)
	, Weight(-1.0f)
	{
	}

	TArray<FChannelItemPtr>& GetChannelItems()
	{
		return ChannelItems;
	}

	FString GetFullLabel() const
	{
		return FullLabel;
	}

	FString GetCompactLabel() const
	{
		return CompactLabel;
	}

	int32 GetDataOffset() const
	{
		return DataOffset;
	}

	int32 GetCardinality() const
	{
		return Cardinality;
	}

	float GetWeight() const
	{
		return Weight;
	}

	bool IsExpanded() const
	{
		return bExpanded;
	}

	void SetExpanded(bool bValue)
	{
		bExpanded = bValue;
	}

private:
	static FString ComputeLabel(const UPoseSearchFeatureChannel* Channel, int32 ChannelComponentIdx, bool bFullLabel)
	{
		if (ChannelComponentIdx >= 0)
		{
			switch (ChannelComponentIdx)
			{
			case 0: return FString("x");
			case 1: return FString("y");
			case 2: return FString("z");
			case 3: return FString("w");
			default: return FString::Printf(TEXT("%d"), ChannelComponentIdx);
			}
		}

		if (Channel != nullptr)
		{
			TLabelBuilder LabelBuilder;
			return Channel->GetLabel(LabelBuilder, bFullLabel ? ELabelFormat::Full_Horizontal : ELabelFormat::Compact_Horizontal).ToString();
		}

		return FString();
	}

	static int32 ComputeDataOffset(const UPoseSearchFeatureChannel* Channel, int32 ChannelComponentIdx)
	{
		int32 DataOffset = 0;

		if (Channel != nullptr)
		{
			DataOffset = Channel->GetChannelDataOffset();

			if (ChannelComponentIdx > 0)
			{
				DataOffset += ChannelComponentIdx;
			}
		}

		return DataOffset;
	}

	static int32 ComputeCardinality(const UPoseSearchFeatureChannel* Channel, int32 ChannelComponentIdx)
	{
		int32 Cardinality = 0;

		if (ChannelComponentIdx >= 0)
		{
			Cardinality = 1;
		}
		else if (Channel != nullptr)
		{
			Cardinality = Channel->GetChannelCardinality();
		}

		return Cardinality;
	}

	static float ComputeWeight(const UPoseSearchFeatureChannel* Channel, int32 ChannelComponentIdx, const FSearchIndex& SearchIndex)
	{
		// Using double precision for accumulation of component weights
		float Weight = 0.0f;

		const int32 DataOffset = ComputeDataOffset(Channel, ChannelComponentIdx);
		const int32 Cardinality = ComputeCardinality(Channel, ChannelComponentIdx);

		for (int32 Component = 0; Component != Cardinality; ++Component)
		{
			// Weights are pre-multiplied by inverse variance in PreprocessSearchIndexWeights, so we undo that
			// here to reconstitute the original weights
			const float WeightSqrt = SearchIndex.WeightsSqrt[DataOffset + Component];
			const float Deviation = SearchIndex.DeviationEditorOnly[DataOffset + Component];
			Weight += FMath::Square(WeightSqrt * Deviation);
		}

		return Weight;
	}

	const FString FullLabel;
	const FString CompactLabel;
	const int32 DataOffset = 0;
	const int32 Cardinality = 0;
	const float Weight = 0.f;
	bool bExpanded = false;
	TArray<FChannelItemPtr> ChannelItems;
};

class SDatabaseDataDetailsTableRow : public SMultiColumnTableRow<FChannelItemPtr>
{
	FChannelItemPtr ChannelItem;
	TWeakPtr<FDatabaseViewModel> EditorViewModel;

public:
	void Construct(const FTableRowArgs& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, FChannelItemPtr InChannelItem, TSharedRef<FDatabaseViewModel> InEditorViewModel)
	{
		EditorViewModel = InEditorViewModel;
		ChannelItem = InChannelItem;
		SMultiColumnTableRow<FChannelItemPtr>::Construct(InArgs, InOwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == FName("ChannelName"))
		{
			// Rows in a TreeView need an expander button and some indentation
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Fill)
				[
					SNew(SExpanderArrow, SharedThis(this))
						.StyleSet(ExpanderStyleSet)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(FText::FromString(ChannelItem->GetCompactLabel()))
				];
		}

		if (ColumnName == FName("DataOffset"))
		{
			if (ChannelItem->GetDataOffset() == INDEX_NONE)
			{
				return SNew(STextBlock);
			}
			return SNew(STextBlock)
				.Text(FText::FromString(FString::Printf(TEXT("%d"), ChannelItem->GetDataOffset())));
		}

		if (ColumnName == FName("Weight"))
		{
			if (ChannelItem->GetDataOffset() == INDEX_NONE)
			{
				return SNew(STextBlock);
			}
			return SNew(STextBlock)
				.Text(FText::FromString(FString::Printf(TEXT("%.3f"), ChannelItem->GetWeight())));
		}

		if (ColumnName == FName("Query"))
		{
			if (ChannelItem->GetDataOffset() == INDEX_NONE)
			{
				return SNew(STextBlock);
			}

			return SNew(STextBlock)
				.Text_Lambda([this, ColumnName]() -> FText
					{
						TStringBuilder<256> StringBuilder;
						if (!ChannelItem->IsExpanded())
						{
							if (const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin())
							{
								const int32 DataOffset = ChannelItem->GetDataOffset();
								const int32 Cardinality = ChannelItem->GetCardinality();

								const TConstArrayView<float> QueryValues = ViewModel->GetQueryVector();
								if (DataOffset + Cardinality <= QueryValues.Num())
								{
									if (Cardinality > 1)
									{
										// using only one decimal to keep the string compact
										for (int32 i = 0; i < Cardinality; ++i)
										{
											if (i != 0)
											{
												StringBuilder.Append(TEXT(", "));
											}
											const float Value = QueryValues[i + DataOffset];
											StringBuilder.Appendf(TEXT("%.1f"), Value);
										}
									}
									else
									{
										// using all the float digits 
										const float Value = QueryValues[DataOffset];
										StringBuilder.Appendf(TEXT("%f"), Value);
									}
								}
							}
						}
						return FText::FromString(StringBuilder.ToString());
					});
		}

		if (ChannelItem->GetDataOffset() == INDEX_NONE)
		{
			if (ChannelItem->GetCompactLabel() == "[Stats]")
			{
				return SNew(STextBlock);
			}

			if (ChannelItem->GetCompactLabel() == "PoseIndex")
			{
				return SNew(STextBlock)
					.Margin(FMargin(1.f, 1.f, 1.f, 1.f))
					.Text_Lambda([this, ColumnName]() -> FText
						{
							if (!ChannelItem->IsExpanded())
							{
								if (const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin())
								{
									const UPoseSearchDatabase* PoseSearchDatabase = ViewModel->GetPoseSearchDatabase();
									if (EAsyncBuildIndexResult::Success == FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(PoseSearchDatabase, ERequestAsyncBuildFlag::ContinueRequest))
									{
										for (const TArray<FDatabasePreviewActor>& PreviewActorGroup : ViewModel->GetPreviewActors())
										{
											for (const FDatabasePreviewActor& PreviewActor : PreviewActorGroup)
											{
												if (*GetNameSafe(PreviewActor.GetActor()) == ColumnName)
												{
													const int32 PoseIdx = PreviewActor.GetCurrentPoseIndex();
													FString PoseIdxString;
													PoseIdxString.AppendInt(PoseIdx);
													return FText::FromString(PoseIdxString);
												}
											}
										}
									}
								}
							}
							return FText::GetEmpty();
						});
			}

			if (ChannelItem->GetCompactLabel() == "SearchAssetIndex")
			{
				return SNew(STextBlock)
					.Margin(FMargin(1.f, 1.f, 1.f, 1.f))
					.Text_Lambda([this, ColumnName]() -> FText
						{
							if (!ChannelItem->IsExpanded())
							{
								if (const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin())
								{
									const UPoseSearchDatabase* PoseSearchDatabase = ViewModel->GetPoseSearchDatabase();
									if (EAsyncBuildIndexResult::Success == FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(PoseSearchDatabase, ERequestAsyncBuildFlag::ContinueRequest))
									{
										for (const TArray<FDatabasePreviewActor>& PreviewActorGroup : ViewModel->GetPreviewActors())
										{
											for (const FDatabasePreviewActor& PreviewActor : PreviewActorGroup)
											{
												if (*GetNameSafe(PreviewActor.GetActor()) == ColumnName)
												{
													const int32 AssetIdx = PreviewActor.GetIndexAssetIndex();
													FString AssetIdxString;
													AssetIdxString.AppendInt(AssetIdx);
													return FText::FromString(AssetIdxString);
												}
											}
										}
									}
								}
							}
							return FText::GetEmpty();
						});
			}

			if (ChannelItem->GetCompactLabel() == "SourceAssetIndex")
			{
				return SNew(STextBlock)
					.Margin(FMargin(1.f, 1.f, 1.f, 1.f))
					.Text_Lambda([this, ColumnName]() -> FText
						{
							if (!ChannelItem->IsExpanded())
							{
								if (const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin())
								{
									const UPoseSearchDatabase* PoseSearchDatabase = ViewModel->GetPoseSearchDatabase();
									if (EAsyncBuildIndexResult::Success == FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(PoseSearchDatabase, ERequestAsyncBuildFlag::ContinueRequest))
									{
										for (const TArray<FDatabasePreviewActor>& PreviewActorGroup : ViewModel->GetPreviewActors())
										{
											for (const FDatabasePreviewActor& PreviewActor : PreviewActorGroup)
											{
												if (*GetNameSafe(PreviewActor.GetActor()) == ColumnName)
												{
													const TConstArrayView<FSearchIndexAsset> Assets = PoseSearchDatabase->GetSearchIndex().Assets;
													const int32 AssetIdx = PreviewActor.GetIndexAssetIndex();
													if (Assets.IsValidIndex(AssetIdx))
													{
														FString SourceAssetIndexString;
														SourceAssetIndexString.AppendInt(Assets[AssetIdx].GetSourceAssetIdx());
														return FText::FromString(SourceAssetIndexString);
													}
												}
											}
										}
									}
								}
							}
							return FText::GetEmpty();
						});
			}

			if (ChannelItem->GetCompactLabel() == "BlendParams")
			{
				return SNew(STextBlock)
					.Margin(FMargin(1.f, 1.f, 1.f, 1.f))
					.Text_Lambda([this, ColumnName]() -> FText
						{
							if (!ChannelItem->IsExpanded())
							{
								if (const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin())
								{
									const UPoseSearchDatabase* PoseSearchDatabase = ViewModel->GetPoseSearchDatabase();
									if (EAsyncBuildIndexResult::Success == FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(PoseSearchDatabase, ERequestAsyncBuildFlag::ContinueRequest))
									{
										for (const TArray<FDatabasePreviewActor>& PreviewActorGroup : ViewModel->GetPreviewActors())
										{
											for (const FDatabasePreviewActor& PreviewActor : PreviewActorGroup)
											{
												if (*GetNameSafe(PreviewActor.GetActor()) == ColumnName)
												{
													const TConstArrayView<FSearchIndexAsset> Assets = PoseSearchDatabase->GetSearchIndex().Assets;
													const int32 AssetIdx = PreviewActor.GetIndexAssetIndex();
													if (Assets.IsValidIndex(AssetIdx))
													{
														const FVector BlendParameters = Assets[AssetIdx].GetBlendParameters();
														FString BlendParamsString = FString::Printf(TEXT("%f, %f"), BlendParameters.X, BlendParameters.Y);
														return FText::FromString(BlendParamsString);
													}
												}
											}
										}
									}
								}
							}
							return FText::GetEmpty();
						});
			}
		}

		return SNew(STextBlock)
			.Margin(FMargin(1.f, 1.f, 1.f, 1.f))
			.Text_Lambda([this, ColumnName]() -> FText
				{
					TStringBuilder<256> StringBuilder;

					const int32 DataOffset = ChannelItem->GetDataOffset();
					const int32 Cardinality = ChannelItem->GetCardinality();

					if (!ChannelItem->IsExpanded() && DataOffset != INDEX_NONE && Cardinality != INDEX_NONE)
					{
						if (const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin())
						{
							const UPoseSearchDatabase* PoseSearchDatabase = ViewModel->GetPoseSearchDatabase();
							if (EAsyncBuildIndexResult::Success == FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(PoseSearchDatabase, ERequestAsyncBuildFlag::ContinueRequest))
							{
								TArray<float> BufferUsedForReconstruction;
								for (const TArray<FDatabasePreviewActor>& PreviewActorGroup : ViewModel->GetPreviewActors())
								{
									for (const FDatabasePreviewActor& PreviewActor : PreviewActorGroup)
									{
										if (*GetNameSafe(PreviewActor.GetActor()) == ColumnName)
										{
											const int32 PoseIdx = PreviewActor.GetCurrentPoseIndex();

											const TConstArrayView<float> PoseValues = PoseSearchDatabase->GetSearchIndex().GetPoseValuesSafe(PoseIdx, BufferUsedForReconstruction);

											if (Cardinality > 1)
											{
												// using only one decimal to keep the string compact
												for (int32 i = 0; i < Cardinality; ++i)
												{
													if (i != 0)
													{
														StringBuilder.Append(TEXT(", "));
													}

													const int32 PoseValueIndex = i + DataOffset;
													if (PoseValueIndex < PoseValues.Num())
													{
														const float Value = PoseValues[PoseValueIndex];
														StringBuilder.Appendf(TEXT("%.1f"), Value);
													}
													else
													{
														StringBuilder.Append(TEXT("---"));
													}
												}
											}
											else
											{
												// using all the float digits 
												if (DataOffset < PoseValues.Num())
												{
													const float Value = PoseValues[DataOffset];
													StringBuilder.Appendf(TEXT("%f"), Value);
												}
												else
												{
													StringBuilder.Append(TEXT("---"));
												}
											}
											return FText::FromString(StringBuilder.ToString());
										}
									}
								}
							}
						}
					}
					return FText::GetEmpty();
				});
	}
};

void SDatabaseDataDetails::Construct(const FArguments& Args, TSharedRef<FDatabaseViewModel> InEditorViewModel)
{
	EditorViewModel = InEditorViewModel;
}

void SDatabaseDataDetails::TrackExpandedItems(const TArray<FChannelItemPtr>& ChannelItems, TMap<const FString, bool>& ExpandedItems)
{
	for (const FChannelItemPtr& ChannelItem : ChannelItems)
	{
		TrackExpandedItems(ChannelItem->GetChannelItems(), ExpandedItems);
		ExpandedItems.FindOrAdd(ChannelItem->GetFullLabel()) = ChannelItem->IsExpanded();
	}
}

void SDatabaseDataDetails::SetExpandedItems(TArray<FChannelItemPtr>& ChannelItems, const TMap<const FString, bool>& ExpandedItems, SChannelItemsTreeView* ChannelItemsTreeView)
{
	for (const FChannelItemPtr& ChannelItem : ChannelItems)
	{
		SetExpandedItems(ChannelItem->GetChannelItems(), ExpandedItems, ChannelItemsTreeView);
		
		bool bIsExpanded = false;
		if (const bool* IsExpandedPtr = ExpandedItems.Find(ChannelItem->GetFullLabel()))
		{
			bIsExpanded = *IsExpandedPtr;
		}

		ChannelItemsTreeView->SetItemExpansion(ChannelItem, bIsExpanded);
	}
};

void SDatabaseDataDetails::Reconstruct()
{
	const int32 MaxPreviewActors = UPoseSearchSettings::Get().MaxDatabaseEditorPreviewActors;
	const TSharedPtr<FDatabaseViewModel> ViewModel = EditorViewModel.Pin();
	
	if (!ViewModel)
	{
		ChannelItems.Reset();
		return;
	}
	
	const UPoseSearchDatabase* PoseSearchDatabase = ViewModel->GetPoseSearchDatabase();
	if (EAsyncBuildIndexResult::Success != FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(PoseSearchDatabase, ERequestAsyncBuildFlag::ContinueRequest))
	{
		ChannelItems.Reset();
		return;
	}

	TMap<const FString, bool> ExpandedItems;
	TrackExpandedItems(ChannelItems, ExpandedItems);

	ChannelItems.Reset();

	RebuildChannelItemsTreeRecursively(ChannelItems, PoseSearchDatabase->Schema->GetChannels(), PoseSearchDatabase->GetSearchIndex());
	RebuildChannelItemsStats(ChannelItems);

	// generating the header
	TSharedPtr<SHeaderRow> HeaderRow = SNew(SHeaderRow);
	HeaderRow->AddColumn(
		SHeaderRow::Column(TEXT("ChannelName"))
			.DefaultLabel(LOCTEXT("ChannelName_Header", "Channel Name"))
			.DefaultTooltip(LOCTEXT("ChannelName_ToolTip", "Channel Name"))
			.FillWidth(0.2f));

	HeaderRow->AddColumn(
		SHeaderRow::Column(TEXT("DataOffset"))
		.DefaultLabel(LOCTEXT("DataOffset_Header", "Data Offset"))
		.DefaultTooltip(LOCTEXT("DataOffset_ToolTip", "Offset from the beginning of the features data"))
		.FillWidth(0.1f));

	HeaderRow->AddColumn(
		SHeaderRow::Column(TEXT("Weight"))
		.DefaultLabel(LOCTEXT("Weight_Header", "Weight"))
		.DefaultTooltip(LOCTEXT("Weight_ToolTip", "Normalized weight of feature's contribution to pose comparisons.\nWeights are set on channels in the schema, and channels distribute weights over their features.\nWeights of all individual features are then normalized together so they sum to one."))
		.FillWidth(0.1f));

	if (ViewModel->ShouldDrawQueryVector())
	{
		HeaderRow->AddColumn(
			SHeaderRow::Column(TEXT("Query"))
			.DefaultLabel(LOCTEXT("Query_Header", "Query"))
			.DefaultTooltip(LOCTEXT("Query_ToolTip", "Query Values")));
	}

	int32 AddedColumns = 0;
	for (const TArray<FDatabasePreviewActor>& PreviewActorGroup : ViewModel->GetPreviewActors())
	{
		for (const FDatabasePreviewActor& PreviewActor : PreviewActorGroup)
		{
			HeaderRow->AddColumn(SHeaderRow::Column(*GetNameSafe(PreviewActor.GetActor())) 
				.DefaultLabel(FText::FromString(GetNameSafe(PreviewActor.GetSampler().GetAsset()))));

			++AddedColumns;
			if (AddedColumns >= MaxPreviewActors)
			{
				break;
			}
		}

		if (AddedColumns >= MaxPreviewActors)
		{
			break;
		}
	}

	ChannelItemsTreeView = SNew(SChannelItemsTreeView)
		.TreeItemsSource(&ChannelItems)
		.HeaderRow(HeaderRow)
		.OnGenerateRow_Lambda([this](FChannelItemPtr ChannelItem, const TSharedRef<STableViewBase>& OwnerTable)
		{
			return SNew(SDatabaseDataDetailsTableRow, OwnerTable, ChannelItem, EditorViewModel.Pin().ToSharedRef());
		})
		.OnGetChildren_Lambda([](FChannelItemPtr ChannelItem, TArray<FChannelItemPtr>& OutChildren)
		{
			OutChildren.Append(ChannelItem->GetChannelItems());
		})
		.OnExpansionChanged_Lambda([](FChannelItemPtr ChannelItem, bool bExpanded)
		{
			ChannelItem->SetExpanded(bExpanded);
		});

	ChildSlot
	[
		SNew(SScrollBox)
		.Orientation(Orient_Horizontal)
		+ SScrollBox::Slot()
		.FillSize(1.f)
		[
			ChannelItemsTreeView.ToSharedRef()
		]
	];

	SetExpandedItems(ChannelItems, ExpandedItems, ChannelItemsTreeView.Get());
}

void SDatabaseDataDetails::RebuildChannelItemsTreeRecursively(TArray<FChannelItemPtr>& ChannelItems, TConstArrayView<TObjectPtr<UPoseSearchFeatureChannel>> Channels, const FSearchIndex& SearchIndex)
{
	for (const TObjectPtr<UPoseSearchFeatureChannel>& ChannelPtr : Channels)
	{
		if (ChannelPtr)
		{
			TSharedRef<FChannelItem> ChannelItem = MakeShareable(new FChannelItem(ChannelPtr, INDEX_NONE, SearchIndex));
			ChannelItems.Add(ChannelItem);

			if (ChannelPtr->GetSubChannels().IsEmpty())
			{
				for (int32 ChannelComponentIdx = 0; ChannelComponentIdx < ChannelPtr->GetChannelCardinality(); ++ChannelComponentIdx)
				{
					TSharedRef<FChannelItem> SubChannelItem = MakeShareable(new FChannelItem(ChannelPtr, ChannelComponentIdx, SearchIndex));
					ChannelItem->GetChannelItems().Add(SubChannelItem);
				}
			}
			else
			{
				RebuildChannelItemsTreeRecursively(ChannelItem->GetChannelItems(), ChannelPtr->GetSubChannels(), SearchIndex);
			}
		}
	}
}

void SDatabaseDataDetails::RebuildChannelItemsStats(TArray<FChannelItemPtr>& ChannelItems)
{
	TSharedRef<FChannelItem> ChannelItem = MakeShareable(new FChannelItem("SDatabaseDataDetailsStats", "[Stats]"));
	ChannelItems.Add(ChannelItem);

	ChannelItem->GetChannelItems().Add(MakeShareable(new FChannelItem("SDatabaseDataDetailsPoseIndex", "PoseIndex")));
	ChannelItem->GetChannelItems().Add(MakeShareable(new FChannelItem("SDatabaseDataDetailsPoseIndex", "SearchAssetIndex")));
	ChannelItem->GetChannelItems().Add(MakeShareable(new FChannelItem("SDatabaseDataDetailsPoseIndex", "SourceAssetIndex")));
	ChannelItem->GetChannelItems().Add(MakeShareable(new FChannelItem("SDatabaseDataDetailsPoseIndex", "BlendParams")));
	// @todo: add support for additional stats
	//ChannelItem->GetChannelItems().Add(MakeShareable(new FChannelItem("SDatabaseDataDetailsValueVectorIndex", "ValueVectorIndex")));
	//ChannelItem->GetChannelItems().Add(MakeShareable(new FChannelItem("SDatabaseDataDetailsPCAValueVectorIndex", "PCAValueVectorIndex")));
	//ChannelItem->GetChannelItems().Add(MakeShareable(new FChannelItem("SDatabaseDataDetailsSpeed", "Speed")));
}

} // namespace UE::PoseSearch

#undef LOCTEXT_NAMESPACE
