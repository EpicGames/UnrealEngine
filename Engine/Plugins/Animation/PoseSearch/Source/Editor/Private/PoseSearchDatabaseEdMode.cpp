// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDatabaseEdMode.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "EngineUtils.h"
#include "PoseSearchDatabaseEditor.h"
#include "PoseSearchDatabaseViewModel.h"
#include "PoseSearchDatabaseViewportClient.h"
#include "Preferences/PersonaOptions.h"
#include "SkeletalDebugRendering.h"

namespace UE::PoseSearch
{
	const FEditorModeID FDatabaseEdMode::EdModeId = TEXT("PoseSearchDatabaseEdMode");

	FDatabaseEdMode::FDatabaseEdMode()
	{
	}

	FDatabaseEdMode::~FDatabaseEdMode()
	{
	}

	void FDatabaseEdMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
	{
		FEdMode::Tick(ViewportClient, DeltaTime);

		FDatabaseViewportClient* PoseSearchDbViewportClient =
			static_cast<FDatabaseViewportClient*>(ViewportClient);
		if (PoseSearchDbViewportClient)
		{
			// ensure we redraw even if PIE is active
			PoseSearchDbViewportClient->Invalidate();

			if (!ViewModel)
			{
				ViewModel = PoseSearchDbViewportClient->GetAssetEditor()->GetViewModel();
			}
		}

		if (ViewModel)
		{
			ViewModel->Tick(DeltaTime);
		}
	}

	void FDatabaseEdMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
	{
		if (ViewModel->IsShowBonesChecked())
		{
			const UPersonaOptions* PersonaOptions = GetDefault<UPersonaOptions>();

			TArray<FTransform> WorldTransforms;
			TArray<FLinearColor> BoneColors;
			for (const TArray<FDatabasePreviewActor>& PreviewActorGroup : ViewModel->GetPreviewActors())
			{
				for (const FDatabasePreviewActor& PreviewActor : PreviewActorGroup)
				{
					const UDebugSkelMeshComponent* MeshComponent = PreviewActor.GetDebugSkelMeshComponent();
					if (MeshComponent && MeshComponent->GetSkeletalMeshAsset() && MeshComponent->GetNumDrawTransform() > 0 && MeshComponent->SkeletonDrawMode != ESkeletonDrawMode::Hidden)
					{
						WorldTransforms.SetNumUninitialized(MeshComponent->GetNumDrawTransform());
						BoneColors.SetNumUninitialized(MeshComponent->GetNumDrawTransform());
						
						const TArray<FBoneIndexType>& DrawBoneIndices = MeshComponent->GetDrawBoneIndices();
						for (int32 Index = 0; Index < DrawBoneIndices.Num(); ++Index)
						{
							const int32 BoneIndex = DrawBoneIndices[Index];
							WorldTransforms[BoneIndex] = MeshComponent->GetDrawTransform(BoneIndex) * MeshComponent->GetComponentTransform();
							BoneColors[BoneIndex] = MeshComponent->GetBoneColor(BoneIndex);
						}

						// color virtual bones
						const FLinearColor VirtualBoneColor = MeshComponent->SkeletonDrawMode == ESkeletonDrawMode::GreyedOut ? PersonaOptions->DisabledBoneColor : PersonaOptions->VirtualBoneColor;
						for (int16 VirtualBoneIndex : MeshComponent->GetReferenceSkeleton().GetRequiredVirtualBones())
						{
							BoneColors[VirtualBoneIndex] = VirtualBoneColor;
						}

						FSkelDebugDrawConfig DrawConfig;
						DrawConfig.BoneDrawMode = (EBoneDrawMode::Type)PersonaOptions->DefaultBoneDrawSelection;
						DrawConfig.BoneDrawSize = 0.2f;
						DrawConfig.bAddHitProxy = false;
						DrawConfig.bForceDraw = false;
						DrawConfig.bUseMultiColorAsDefaultColor = GetDefault<UPersonaOptions>()->bShowBoneColors;
						DrawConfig.DefaultBoneColor = PersonaOptions->DefaultBoneColor;
						DrawConfig.AffectedBoneColor = PersonaOptions->AffectedBoneColor;
						DrawConfig.SelectedBoneColor = PersonaOptions->SelectedBoneColor;
						DrawConfig.ParentOfSelectedBoneColor = PersonaOptions->ParentOfSelectedBoneColor;

						SkeletalDebugRendering::DrawBones(PDI, MeshComponent->GetComponentLocation(), DrawBoneIndices, MeshComponent->GetReferenceSkeleton(),
							WorldTransforms, MeshComponent->BonesOfInterest, BoneColors, TArray<TRefCountPtr<HHitProxy>>(), DrawConfig);
					}
				}
			}
		}

		FEdMode::Render(View, Viewport, PDI);
	}

	bool FDatabaseEdMode::HandleClick(
		FEditorViewportClient* InViewportClient,
		HHitProxy* HitProxy,
		const FViewportClick& Click)
	{
		if (HActor* ActorHitProxy = HitProxyCast<HActor>(HitProxy))
		{
			if (ViewModel && IsValid(ActorHitProxy->Actor))
			{
				ViewModel->ProcessSelectedActor(ActorHitProxy->Actor);
				return true;
			}
		}

		ViewModel->ProcessSelectedActor(nullptr);

		return false; // unhandled
	}


	bool FDatabaseEdMode::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
	{
		return FEdMode::StartTracking(InViewportClient, InViewport);
	}

	bool FDatabaseEdMode::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
	{
		return FEdMode::EndTracking(InViewportClient, InViewport);
	}

	bool FDatabaseEdMode::InputDelta(
		FEditorViewportClient* InViewportClient, 
		FViewport* InViewport, 
		FVector& InDrag, 
		FRotator& InRot, 
		FVector& InScale)
	{
		return FEdMode::InputDelta(InViewportClient, InViewport, InDrag, InRot, InScale);
	}

	bool FDatabaseEdMode::InputKey(
		FEditorViewportClient* ViewportClient, 
		FViewport* Viewport, 
		FKey Key, 
		EInputEvent Event)
	{
		return FEdMode::InputKey(ViewportClient, Viewport, Key, Event);
	}

	bool FDatabaseEdMode::AllowWidgetMove()
	{
		return FEdMode::ShouldDrawWidget();
	}

	bool FDatabaseEdMode::ShouldDrawWidget() const
	{
		return FEdMode::ShouldDrawWidget();
	}

	bool FDatabaseEdMode::GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData)
	{
		return FEdMode::GetCustomDrawingCoordinateSystem(InMatrix, InData);
	}

	bool FDatabaseEdMode::GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData)
	{
		return FEdMode::GetCustomDrawingCoordinateSystem(InMatrix, InData);
	}

	FVector FDatabaseEdMode::GetWidgetLocation() const
	{
		return FEdMode::GetWidgetLocation();
	}
}
