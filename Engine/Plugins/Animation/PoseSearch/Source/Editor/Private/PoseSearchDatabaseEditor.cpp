// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDatabaseEditor.h"
#include "AdvancedPreviewSceneModule.h"
#include "Animation/AnimSequenceBase.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/WorldSettings.h"
#include "IStructureDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PoseSearch/PoseSearchDerivedData.h"
#include "PoseSearchDatabaseAssetBrowser.h"
#include "PoseSearchDatabaseAssetTree.h"
#include "PoseSearchDatabaseDataDetails.h"
#include "PoseSearchDatabaseEditorCommands.h"
#include "PoseSearchDatabaseEditorReflection.h"
#include "PoseSearchDatabasePreviewScene.h"
#include "PoseSearchDatabaseViewModel.h"
#include "PoseSearchEditor.h"
#include "PropertyEditorModule.h"
#include "SPoseSearchDatabaseViewport.h"
#include "StructUtils/InstancedStruct.h"
#include "Styling/AppStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"

#define LOCTEXT_NAMESPACE "PoseSearchDatabaseEditor"

namespace UE::PoseSearch
{
	const FName PoseSearchDatabaseEditorAppName = FName(TEXT("PoseSearchDatabaseEditorApp"));

	// Tab identifiers
	struct FDatabaseEditorTabs
	{
		static const FName AssetDetailsID;
		static const FName ViewportID;
		static const FName PreviewSettingsID;
		static const FName AssetTreeViewID;
		static const FName SelectionDetailsID;
		static const FName StatisticsOverview;
		static const FName DataDetailsID;
		static const FName AssetBrowserID;
	};

	const FName FDatabaseEditorTabs::AssetDetailsID(TEXT("PoseSearchDatabaseEditorAssetDetailsTabID"));
	const FName FDatabaseEditorTabs::ViewportID(TEXT("PoseSearchDatabaseEditorViewportTabID"));
	const FName FDatabaseEditorTabs::PreviewSettingsID(TEXT("PoseSearchDatabaseEditorPreviewSettingsTabID"));
	const FName FDatabaseEditorTabs::AssetTreeViewID(TEXT("PoseSearchDatabaseEditorAssetTreeViewTabID"));
	const FName FDatabaseEditorTabs::SelectionDetailsID(TEXT("PoseSearchDatabaseEditorSelectionDetailsID"));
	const FName FDatabaseEditorTabs::StatisticsOverview(TEXT("PoseSearchDatabaseEditorStatisticsOverviewID"));
	const FName FDatabaseEditorTabs::DataDetailsID(TEXT("PoseSearchDatabaseEditorDataDetailsTabID"));
	const FName FDatabaseEditorTabs::AssetBrowserID(TEXT("PoseSearchDatabaseEditorAssetBrowserID"));

	const UPoseSearchDatabase* FDatabaseEditor::GetPoseSearchDatabase() const
	{
		return ViewModel.IsValid() ? ViewModel->GetPoseSearchDatabase() : nullptr;
	}

	UPoseSearchDatabase* FDatabaseEditor::GetPoseSearchDatabase()
	{
		return ViewModel.IsValid() ? ViewModel->GetPoseSearchDatabase() : nullptr;
	}

	void FDatabaseEditor::SetSelectedPoseIdx(int32 PoseIdx, bool bDrawQuery, TConstArrayView<float> InQueryVector)
	{
		if (ViewModel.IsValid())
		{
			const bool bClearSelection = !FSlateApplication::Get().GetModifierKeys().IsControlDown() || ViewModel->IsEditorSelection();
			const int32 SourceAssetIdx = ViewModel->SetSelectedNode(PoseIdx, bClearSelection, bDrawQuery, InQueryVector);
			AssetTreeWidget->SetSelectedItem(SourceAssetIdx, bClearSelection);
		}
	}

	void FDatabaseEditor::SetDrawQueryVector(bool bValue)
	{
		if (ViewModel.IsValid())
		{
			ViewModel->SetDrawQueryVector(bValue);
		}
	}

	void FDatabaseEditor::PreviewBackwardEnd()
	{
		ViewModel->PreviewBackwardEnd();
	}

	void FDatabaseEditor::PreviewBackwardStep()
	{
		ViewModel->PreviewBackwardStep();
	}

	void FDatabaseEditor::PreviewBackward()
	{
		ViewModel->PreviewBackward();
	}

	void FDatabaseEditor::PreviewPause()
	{
		ViewModel->PreviewPause();
	}

	void FDatabaseEditor::PreviewForward()
	{
		ViewModel->PreviewForward();
	}

	void FDatabaseEditor::PreviewForwardStep()
	{
		ViewModel->PreviewForwardStep();
	}

	void FDatabaseEditor::PreviewForwardEnd()
	{
		ViewModel->PreviewForwardEnd();
	}

	void FDatabaseEditor::InitAssetEditor(
		const EToolkitMode::Type Mode,
		const TSharedPtr<IToolkitHost>& InitToolkitHost,
		UPoseSearchDatabase* DatabaseAsset)
	{
		// Create Preview Scene
		if (!PreviewScene.IsValid())
		{
			PreviewScene = MakeShareable(
				new FDatabasePreviewScene(
					FPreviewScene::ConstructionValues()
					.SetCreatePhysicsScene(false)
					.SetTransactional(false)
					.ForceUseMovementComponentInNonGameWorld(true),
					StaticCastSharedRef<FDatabaseEditor>(AsShared())));

			//Temporary fix for missing attached assets - MDW (Copied from FPersonaToolkit::CreatePreviewScene)
			PreviewScene->GetWorld()->GetWorldSettings()->SetIsTemporarilyHiddenInEditor(false);
		}

		// Create view model
		ViewModel = MakeShared<FDatabaseViewModel>();

		// Create Data Details widget
		DataDetails = SNew(SDatabaseDataDetails, ViewModel.ToSharedRef());

		// Initialize view model
		ViewModel->Initialize(DatabaseAsset, PreviewScene.ToSharedRef(), DataDetails.ToSharedRef());

		// Initialize DataDetails
		DataDetails->Reconstruct();

		// Create viewport widget
		{
			FDatabasePreviewRequiredArgs PreviewArgs(
				StaticCastSharedRef<FDatabaseEditor>(AsShared()),
				PreviewScene.ToSharedRef());
			
			PreviewWidget = SNew(SDatabasePreview, PreviewArgs)
				.SliderColor_Lambda([this]()
					{
						return ViewModel->IsEditorSelection() ? FLinearColor::Red.CopyWithNewOpacity(0.5f) : FLinearColor::Blue.CopyWithNewOpacity(0.5f);
					})
				.SliderScrubTime_Lambda([this]()
					{
						return ViewModel->GetPlayTime();
					})
				.SliderViewRange_Lambda([this]() 
					{ 
						return ViewModel->GetPreviewPlayRange();
					})
				.OnSliderScrubPositionChanged_Lambda([this](float NewScrubPosition, bool bScrubbing)
					{
						ViewModel->SetPlayTime(NewScrubPosition, !bScrubbing);
					})
				.OnBackwardEnd_Raw(this, &FDatabaseEditor::PreviewBackwardEnd)
				.OnBackwardStep_Raw(this, &FDatabaseEditor::PreviewBackwardStep)
				.OnBackward_Raw(this, &FDatabaseEditor::PreviewBackward)
				.OnPause_Raw(this, &FDatabaseEditor::PreviewPause)
				.OnForward_Raw(this, &FDatabaseEditor::PreviewForward)
				.OnForwardStep_Raw(this, &FDatabaseEditor::PreviewForwardStep)
				.OnForwardEnd_Raw(this, &FDatabaseEditor::PreviewForwardEnd);
		}

		// Create asset tree widget
		AssetTreeWidget = SNew(SDatabaseAssetTree, ViewModel.ToSharedRef());
		AssetTreeWidget->RegisterOnSelectionChanged(
			SDatabaseAssetTree::FOnSelectionChanged::CreateSP(
				this,
				&FDatabaseEditor::OnAssetTreeSelectionChanged));
		
		// Create Asset Details widget
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		// Database details widget
		{
			FDetailsViewArgs DatabaseDetailsArgs;
			{
				DatabaseDetailsArgs.bHideSelectionTip = true;
				DatabaseDetailsArgs.NotifyHook = this;
			}

			EditingAssetWidget = PropertyModule.CreateDetailView(DatabaseDetailsArgs);
			EditingAssetWidget->SetObject(DatabaseAsset);
		}

		// Statistics details widgets
		{
			FDetailsViewArgs StatisticsOverviewDetailsArgs;
			{
				StatisticsOverviewDetailsArgs.bHideSelectionTip = true;
				StatisticsOverviewDetailsArgs.NotifyHook = this;
				StatisticsOverviewDetailsArgs.bAllowSearch = false;
			}
			
			StatisticsOverviewWidget = PropertyModule.CreateDetailView(StatisticsOverviewDetailsArgs);
			StatisticsOverviewWidget->SetObject(nullptr);

			// Ensure statistics information get updated/populated
			if (DatabaseAsset)
			{
				// Init statistics
				if (EAsyncBuildIndexResult::Success == FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(GetPoseSearchDatabase(), ERequestAsyncBuildFlag::ContinueRequest))
				{
					RefreshStatisticsWidgetInformation();
				}

				// Ensure any database changes are reflected
				DatabaseAsset->UnregisterOnDerivedDataRebuild(this);
				DatabaseAsset->RegisterOnDerivedDataRebuild(UPoseSearchDatabase::FOnDerivedDataRebuild::CreateSP(this, &FDatabaseEditor::RefreshStatisticsWidgetInformation));

				DatabaseAsset->UnregisterOnSynchronizeWithExternalDependencies(this);
				DatabaseAsset->RegisterOnSynchronizeWithExternalDependencies(UPoseSearchDatabase::FOnDerivedDataRebuild::CreateSP(this, &FDatabaseEditor::RefreshEditor));
			}
		}

		// Create asset browser widget
		AssetBrowserWidget = SNew(SPoseSearchDatabaseAssetBrowser, ViewModel);
		
		// Define Editor Layout
		const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout =
		FTabManager::NewLayout("Standalone_PoseSearchDatabaseEditor_Layout_v0.14")
			->AddArea
			(
				// Main application area
				FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Horizontal)
					->Split
					(
						FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.2f)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.6f)
							->AddTab(FDatabaseEditorTabs::AssetTreeViewID, ETabState::OpenedTab)
							->SetHideTabWell(false)
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.4f)
							->AddTab(FDatabaseEditorTabs::AssetDetailsID, ETabState::OpenedTab)
							->SetHideTabWell(false)
						)
					)
					->Split
					(
						FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.8f)
						->Split
						(
							FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
							->Split(
								FTabManager::NewStack()
								->SetSizeCoefficient(0.75f)
								->AddTab(FDatabaseEditorTabs::ViewportID, ETabState::OpenedTab)
								->SetHideTabWell(true)
									
							)
							->Split(
								FTabManager::NewStack()
								->SetSizeCoefficient(0.25f)
								->AddTab(FDatabaseEditorTabs::AssetBrowserID, ETabState::OpenedTab)
							)
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.4f)
							->AddTab(FDatabaseEditorTabs::StatisticsOverview, ETabState::OpenedTab)
							->AddTab(FDatabaseEditorTabs::PreviewSettingsID, ETabState::OpenedTab)
							->AddTab(FDatabaseEditorTabs::DataDetailsID, ETabState::OpenedTab)
							->AddTab(FDatabaseEditorTabs::SelectionDetailsID, ETabState::OpenedTab)
						)
					)
				)
			);

		const bool bCreateDefaultStandaloneMenuParam = true;
		const bool bCreateDefaultToolbarParam = true;
		const bool bIsToolbarFocusableParam = false;
		FAssetEditorToolkit::InitAssetEditor(
			Mode,
			InitToolkitHost,
			PoseSearchDatabaseEditorAppName,
			StandaloneDefaultLayout,
			bCreateDefaultStandaloneMenuParam,
			bCreateDefaultToolbarParam,
			DatabaseAsset,
			bIsToolbarFocusableParam);

		RegenerateMenusAndToolbars();
	}

	FDatabaseEditor::~FDatabaseEditor()
	{
		if (UObjectInitialized())
		{
			for (TWeakObjectPtr<UObject>& Reflection : SelectionWidget.SelectedReflections)
			{
				if (Reflection.IsValid())
				{
					Reflection->RemoveFromRoot();
				}
			}
		}
		
		if (CachedStatistics && UObjectInitialized())
		{
			CachedStatistics->RemoveFromRoot();
			CachedStatistics = nullptr;
		}
		
		if (ViewModel)
		{
			if (UPoseSearchDatabase* DatabaseAsset = ViewModel->GetPoseSearchDatabase())
			{
				DatabaseAsset->UnregisterOnDerivedDataRebuild(this);
				DatabaseAsset->UnregisterOnSynchronizeWithExternalDependencies(this);
			}
		}
	}

	void FDatabaseEditor::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(
			LOCTEXT("WorkspaceMenu_PoseSearchDbEditor", "Pose Search Database Editor"));
		TSharedRef<FWorkspaceItem> WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

		FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

		InTabManager->RegisterTabSpawner(
			FDatabaseEditorTabs::ViewportID,
			FOnSpawnTab::CreateSP(this, &FDatabaseEditor::SpawnTab_Viewport))
			.SetDisplayName(LOCTEXT("ViewportTab", "Viewport"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));

		InTabManager->RegisterTabSpawner(
			FDatabaseEditorTabs::AssetDetailsID,
			FOnSpawnTab::CreateSP(this, &FDatabaseEditor::SpawnTab_AssetDetails))
			.SetDisplayName(LOCTEXT("DatabaseDetailsTab", "Database Details"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		InTabManager->RegisterTabSpawner(
			FDatabaseEditorTabs::PreviewSettingsID,
			FOnSpawnTab::CreateSP(this, &FDatabaseEditor::SpawnTab_PreviewSettings))
			.SetDisplayName(LOCTEXT("PreviewSceneSettingsTab", "Preview Scene Settings"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		InTabManager->RegisterTabSpawner(
			FDatabaseEditorTabs::AssetTreeViewID,
			FOnSpawnTab::CreateSP(this, &FDatabaseEditor::SpawnTab_AssetTreeView))
			.SetDisplayName(LOCTEXT("TreeViewTab", "Tree View"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));

		InTabManager->RegisterTabSpawner(
			FDatabaseEditorTabs::SelectionDetailsID,
			FOnSpawnTab::CreateSP(this, &FDatabaseEditor::SpawnTab_SelectionDetails))
			.SetDisplayName(LOCTEXT("SelectionDetailsTab", "Selection Details"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));

		InTabManager->RegisterTabSpawner(
			FDatabaseEditorTabs::StatisticsOverview,
			FOnSpawnTab::CreateSP(this, &FDatabaseEditor::SpawnTab_StatisticsOverview))
			.SetDisplayName(LOCTEXT("StatisticsOverviewTab", "Statistics Overview"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));

		InTabManager->RegisterTabSpawner(
			FDatabaseEditorTabs::DataDetailsID,
			FOnSpawnTab::CreateSP(this, &FDatabaseEditor::SpawnTab_DataDetails))
			.SetDisplayName(LOCTEXT("DataDetailsTab", "Data Details"))
			.SetGroup(WorkspaceMenuCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));
		
		InTabManager->RegisterTabSpawner(
			FDatabaseEditorTabs::AssetBrowserID,
			FOnSpawnTab::CreateSP(this, &FDatabaseEditor::SpawnTab_AssetBrowser))
			.SetDisplayName(LOCTEXT("AssetBrowserTab", "Asset Browser"))
			.SetGroup(WorkspaceMenuCategoryRef);
			// .SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));
	}

	void FDatabaseEditor::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

		InTabManager->UnregisterTabSpawner(FDatabaseEditorTabs::ViewportID);
		InTabManager->UnregisterTabSpawner(FDatabaseEditorTabs::AssetDetailsID);
		InTabManager->UnregisterTabSpawner(FDatabaseEditorTabs::PreviewSettingsID);
		InTabManager->UnregisterTabSpawner(FDatabaseEditorTabs::AssetTreeViewID);
		InTabManager->UnregisterTabSpawner(FDatabaseEditorTabs::SelectionDetailsID);
		InTabManager->UnregisterTabSpawner(FDatabaseEditorTabs::StatisticsOverview);
		InTabManager->UnregisterTabSpawner(FDatabaseEditorTabs::DataDetailsID);
		InTabManager->UnregisterTabSpawner(FDatabaseEditorTabs::AssetBrowserID);
	}

	FName FDatabaseEditor::GetToolkitFName() const
	{
		return FName("PoseSearchDatabaseEditor");
	}

	FText FDatabaseEditor::GetBaseToolkitName() const
	{
		return LOCTEXT("PoseSearchDatabaseEditorAppLabel", "Pose Search Database Editor");
	}

	FText FDatabaseEditor::GetToolkitName() const
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("AssetName"), FText::FromString(GetNameSafe(GetPoseSearchDatabase())));
		return FText::Format(LOCTEXT("PoseSearchDatabaseEditorToolkitName", "{AssetName}"), Args);
	}

	FLinearColor FDatabaseEditor::GetWorldCentricTabColorScale() const
	{
		return FLinearColor::White;
	}

	FString FDatabaseEditor::GetWorldCentricTabPrefix() const
	{
		return TEXT("PoseSearchDatabaseEditor");
	}

	TSharedRef<SDockTab> FDatabaseEditor::SpawnTab_Viewport(const FSpawnTabArgs& Args)
	{
		check(Args.GetTabId() == FDatabaseEditorTabs::ViewportID);

		TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab).Label(LOCTEXT("ViewportTab_Title", "Viewport"));

		if (PreviewWidget.IsValid())
		{
			SpawnedTab->SetContent(PreviewWidget.ToSharedRef());
		}

		return SpawnedTab;
	}

	TSharedRef<SDockTab> FDatabaseEditor::SpawnTab_AssetDetails(const FSpawnTabArgs& Args)
	{
		check(Args.GetTabId() == FDatabaseEditorTabs::AssetDetailsID);

		return SNew(SDockTab)
			.Label(LOCTEXT("Database_Details_Title", "Database Details"))
			[
				EditingAssetWidget.ToSharedRef()
			];
	}

	TSharedRef<SDockTab> FDatabaseEditor::SpawnTab_PreviewSettings(const FSpawnTabArgs& Args)
	{
		check(Args.GetTabId() == FDatabaseEditorTabs::PreviewSettingsID);

		FAdvancedPreviewSceneModule& AdvancedPreviewSceneModule = 
			FModuleManager::LoadModuleChecked<FAdvancedPreviewSceneModule>("AdvancedPreviewScene");
		TSharedRef<SWidget> InWidget = 
			AdvancedPreviewSceneModule.CreateAdvancedPreviewSceneSettingsWidget(PreviewScene.ToSharedRef());

		TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
			.Label(LOCTEXT("PreviewSceneSettingsTab", "Preview Scene Settings"))
			[
				InWidget
			];

		return SpawnedTab;
	}

	TSharedRef<SDockTab> FDatabaseEditor::SpawnTab_AssetTreeView(const FSpawnTabArgs& Args)
	{
		check(Args.GetTabId() == FDatabaseEditorTabs::AssetTreeViewID);

		return SNew(SDockTab)
			.Label(LOCTEXT("AssetTreeView_Title", "Asset List"))
			[
				AssetTreeWidget.ToSharedRef()
			];
	}

	TSharedRef<SDockTab> FDatabaseEditor::SpawnTab_SelectionDetails(const FSpawnTabArgs& Args)
	{
		check(Args.GetTabId() == FDatabaseEditorTabs::SelectionDetailsID);

		SAssignNew(DetailsContainer, SVerticalBox);

		return SNew(SDockTab)
			.Label(LOCTEXT("SelectionDetails_Title", "Selection Details"))
			[
				SNew(SScrollBox)
				+SScrollBox::Slot()
				[
					DetailsContainer.ToSharedRef()
				]
			];
	}

	TSharedRef<SDockTab> FDatabaseEditor::SpawnTab_StatisticsOverview(const FSpawnTabArgs& Args) const
	{
		check(Args.GetTabId() == FDatabaseEditorTabs::StatisticsOverview);
		
		return SNew(SDockTab)
		.Label(LOCTEXT("StatisticsOverview_Title", "Statistics Overview"))
		[
			SNew(SScrollBox)
			+SScrollBox::Slot()
			[
				StatisticsOverviewWidget.ToSharedRef()
			]
		];
	}

	TSharedRef<SDockTab> FDatabaseEditor::SpawnTab_DataDetails(const FSpawnTabArgs& Args) const
	{
		check(Args.GetTabId() == FDatabaseEditorTabs::DataDetailsID);
		
		return SNew(SDockTab)
			.Label(LOCTEXT("DataDetails_Title", "Data Details"))
			[
				DataDetails.ToSharedRef()
			];
	}
	
	TSharedRef<SDockTab> FDatabaseEditor::SpawnTab_AssetBrowser(const FSpawnTabArgs& Args) const
	{
		check(Args.GetTabId() == FDatabaseEditorTabs::AssetBrowserID);
		
		return SNew(SDockTab)
			.Label(LOCTEXT("AssetBrowser_Title", "Asset Browser"))
			[
				AssetBrowserWidget.ToSharedRef()
			];
	}
	
	void FDatabaseEditor::OnFinishedChangingSelectionProperties(const FPropertyChangedEvent& PropertyChangedEvent)
	{
		if (UPoseSearchDatabase* Database = GetPoseSearchDatabase())
		{
			if (IsValid(Database))
			{
				bool bModified = false;
				for (TWeakObjectPtr<UObject>& SelectedReflectionPtr : SelectionWidget.SelectedReflections)
				{
					if (UPoseSearchDatabaseAnimationAssetReflection* SelectionReflection = Cast<UPoseSearchDatabaseAnimationAssetReflection>(SelectedReflectionPtr.Pin().Get()))
					{
						// adding some edit constraints to SelectionReflection
						if (UObject* AnimationAsset = SelectionReflection->DatabaseAnimationAsset.GetAnimationAsset())
						{
							if (AnimationAsset->IsA<UAnimSequenceBase>())
							{
								const float PlayLength = SelectionReflection->DatabaseAnimationAsset.GetPlayLength(FVector::ZeroVector);
								const FFloatInterval ClampedSamplingRange(FMath::Clamp(SelectionReflection->DatabaseAnimationAsset.SamplingRange.Min, 0.0f, PlayLength), FMath::Clamp(SelectionReflection->DatabaseAnimationAsset.SamplingRange.Max, 0.0f, PlayLength));
								if (ClampedSamplingRange != SelectionReflection->DatabaseAnimationAsset.SamplingRange)
								{
									SelectionReflection->DatabaseAnimationAsset.SamplingRange = ClampedSamplingRange;
								}
							}
						}

						if (const FPoseSearchDatabaseAnimationAsset* DatabaseAnimationAssetPtr = Database->GetDatabaseAnimationAsset(SelectionReflection->SourceAssetIdx))
						{
							if (*DatabaseAnimationAssetPtr != SelectionReflection->DatabaseAnimationAsset)
							{
								if (!bModified)
								{
									Database->Modify();
									bModified = true;
								}
								Database->SetAnimationAssetAt(SelectionReflection->DatabaseAnimationAsset, SelectionReflection->SourceAssetIdx);
							}
						}
					}
				}

				if (bModified)
				{
					Database->PostEditChange();
				}
			}
		}
	}

	void FDatabaseEditor::OnAssetTreeSelectionChanged(
		const TArrayView<TSharedPtr<FDatabaseAssetTreeNode>>& SelectedItems,
		ESelectInfo::Type SelectionType)
	{
		// Reset selected objects on the selection widget
		for (TWeakObjectPtr<UObject>& Reflection : SelectionWidget.SelectedReflections)
		{
			if (Reflection.IsValid())
			{
				Reflection->RemoveFromRoot();
			}
		}
		SelectionWidget.SelectedReflections.Reset();

		const UPoseSearchDatabase* PoseSearchDatabase = GetPoseSearchDatabase();
		if (PoseSearchDatabase && SelectedItems.Num() > 0)
		{
			for (TSharedPtr<FDatabaseAssetTreeNode>& SelectedItem : SelectedItems)
			{
				if (SelectedItem.IsValid())
				{
					if (const FPoseSearchDatabaseAnimationAsset* DatabaseAnimationAsset = PoseSearchDatabase->GetDatabaseAnimationAsset(SelectedItem->SourceAssetIdx))
					{
						UPoseSearchDatabaseAnimationAssetReflection* NewSelectionReflection = NewObject<UPoseSearchDatabaseAnimationAssetReflection>();
						NewSelectionReflection->AddToRoot();
						NewSelectionReflection->DatabaseAnimationAsset = *DatabaseAnimationAsset;
						NewSelectionReflection->SourceAssetIdx = SelectedItem->SourceAssetIdx;
						NewSelectionReflection->SetFlags(RF_Transactional);

						GetInitializedSelectionWidget().SelectedReflections.Add(NewSelectionReflection);
					}
				}
			}
		}

		if (SelectionWidget.DetailView.IsValid())
		{
			if (SelectionWidget.SelectedReflections.IsEmpty())
			{
				SelectionWidget.DetailView->SetObject(nullptr, true);
			}
			else
			{
				SelectionWidget.DetailView->SetObjects(SelectionWidget.SelectedReflections);
			}
		}

		if (SelectionType != ESelectInfo::Direct)
		{
			ViewModel->SetSelectedNodes(SelectedItems);
		}
	}

	FDatabaseEditor::FSelectionWidget& FDatabaseEditor::GetInitializedSelectionWidget()
	{
		if (SelectionWidget.DetailView.IsValid())
		{
			return SelectionWidget;
		}

		FDetailsViewArgs SelectionDetailsArgs;
		{
			SelectionDetailsArgs.bHideSelectionTip = true;
			SelectionDetailsArgs.NotifyHook = this;
			SelectionDetailsArgs.bShowScrollBar = false;
		}

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		SelectionWidget.DetailView = PropertyModule.CreateDetailView(SelectionDetailsArgs);
		SelectionWidget.DetailView->SetObject(nullptr);
		SelectionWidget.DetailView->OnFinishedChangingProperties().AddSP(
			this,
			&FDatabaseEditor::OnFinishedChangingSelectionProperties);

		if (DetailsContainer)
		{
			DetailsContainer->AddSlot()
				.AutoHeight()
				[
					SelectionWidget.DetailView.ToSharedRef()
				];
		}

		return SelectionWidget;
	}

	void FDatabaseEditor::RefreshStatisticsWidgetInformation()
	{
		if (CachedStatistics)
		{
			CachedStatistics->RemoveFromRoot();
		}
		
		CachedStatistics = NewObject<UPoseSearchDatabaseStatistics>();
		CachedStatistics->AddToRoot();
		CachedStatistics->Initialize(GetPoseSearchDatabase());
		StatisticsOverviewWidget->SetObject(CachedStatistics);
	}

	void FDatabaseEditor::RefreshEditor()
	{
		AssetTreeWidget->RefreshTreeView(false, true);
	}
}

#undef LOCTEXT_NAMESPACE
