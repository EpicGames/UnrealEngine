// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDatabaseEditorClipboard.h"

#include "Factories.h"
#include "StructUtils/InstancedStruct.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimMontage.h"
#include "Animation/BlendSpace.h"
#include "PoseSearch/MultiAnimAsset.h"
#include "PoseSearch/PoseSearchDatabase.h"

#include "UnrealExporter.h"
#include "Exporters/Exporter.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/StringOutputDevice.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchDatabaseEditorClipboard)

void UPoseSearchDatabaseEditorClipboardContent::CopyDatabaseItem(const FPoseSearchDatabaseAnimationAssetBase* InItem)
{
	check(InItem != nullptr)
	
	UPoseSearchDatabaseItemCopyObject* CopyObj = NewObject<UPoseSearchDatabaseItemCopyObject>((UObject*)GetTransientPackage(), UPoseSearchDatabaseItemCopyObject::StaticClass(), NAME_None, RF_Transient);
	
	UClass* AnimationAssetStaticClass = InItem->GetAnimationAssetStaticClass();
	check(AnimationAssetStaticClass);
	CopyObj->ClassName = AnimationAssetStaticClass->GetFName();
	FPoseSearchDatabaseAnimationAsset::StaticStruct()->ExportText(CopyObj->Content, InItem, nullptr, nullptr, PPF_None, nullptr);


	DatabaseItems.Add(CopyObj);
}

void UPoseSearchDatabaseEditorClipboardContent::CopyToClipboard()
{
	// Clear the mark state for saving.
	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));
	
	// Export the clipboard to text.
	FStringOutputDevice Archive;
	const FExportObjectInnerContext Context;
	UExporter::ExportToOutputDevice(&Context, this, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, this->GetOuter());
	FPlatformApplicationMisc::ClipboardCopy(*Archive);
}

void UPoseSearchDatabaseEditorClipboardContent::PasteToDatabase(UPoseSearchDatabase* InTargetDatabase) const
{
	check(InTargetDatabase != nullptr)
	
	InTargetDatabase->Modify();
	
	for (TObjectPtr<UPoseSearchDatabaseItemCopyObject> Item : DatabaseItems)
	{
		FPoseSearchDatabaseAnimationAsset NewAsset;
		FPoseSearchDatabaseAnimationAsset::StaticStruct()->ImportText(*Item->Content, &NewAsset, nullptr, PPF_None, GLog, FPoseSearchDatabaseAnimationAsset::StaticStruct()->GetName());
		InTargetDatabase->AddAnimationAsset(NewAsset);
	}

	InTargetDatabase->PostEditChange();
}

class FPoseSearchDatabaseEditorClipboardContentTextFactory : public FCustomizableTextObjectFactory
{
public:
	
	FPoseSearchDatabaseEditorClipboardContentTextFactory()
		: FCustomizableTextObjectFactory(GWarn)
		, ClipboardContent(nullptr) 
	{
	}

	UPoseSearchDatabaseEditorClipboardContent* ClipboardContent;

protected:

	virtual bool CanCreateClass(UClass* InObjectClass, bool& bOmitSubObjs) const override
	{
		if (InObjectClass->IsChildOf(UPoseSearchDatabaseEditorClipboardContent::StaticClass()))
		{
			return true;
		}
		return false;
	}
	
	virtual void ProcessConstructedObject(UObject* CreatedObject) override
	{
		if (CreatedObject->IsA<UPoseSearchDatabaseEditorClipboardContent>())
		{
			ClipboardContent = CastChecked<UPoseSearchDatabaseEditorClipboardContent>(CreatedObject);
		}
	}
};

bool UPoseSearchDatabaseEditorClipboardContent::CanPasteContentFromClipboard(const FString& InTextToImport)
{
	const FPoseSearchDatabaseEditorClipboardContentTextFactory ClipboardContentFactory;
	return ClipboardContentFactory.CanCreateObjectsFromText(InTextToImport);
}

UPoseSearchDatabaseEditorClipboardContent* UPoseSearchDatabaseEditorClipboardContent::CreateFromClipboard()
{
	// Get the text from the clipboard.
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

	// Try to create clipboard content from text.
	if (CanPasteContentFromClipboard(ClipboardText))
	{
		FPoseSearchDatabaseEditorClipboardContentTextFactory ClipboardContentFactory;
		ClipboardContentFactory.ProcessBuffer((UObject*)GetTransientPackage(), RF_Transactional, ClipboardText);
		return ClipboardContentFactory.ClipboardContent;
	}

	return nullptr;
}

UPoseSearchDatabaseEditorClipboardContent* UPoseSearchDatabaseEditorClipboardContent::Create()
{
	return NewObject<UPoseSearchDatabaseEditorClipboardContent>((UObject*)GetTransientPackage());
}
