// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearchDatabaseEditorClipboard.generated.h"

/** Object used to export FPoseSearchDatabaseAnimationAssetBase and derived classes to clipboard */
UCLASS(Transient)
class UPoseSearchDatabaseItemCopyObject : public UObject
{
	GENERATED_BODY()

public:
	
	UPROPERTY()
	FName ClassName = NAME_None;

	/** Store's exported database asset as a string */
	UPROPERTY()
	FString Content;
};

UCLASS(Transient)
class UPoseSearchDatabaseEditorClipboardContent : public UObject
{
	GENERATED_BODY()

public:
	
	UPROPERTY(Export)
	TArray<TObjectPtr<UPoseSearchDatabaseItemCopyObject>> DatabaseItems;

	// Utility functions

	/** Store copy of database item. */
	void CopyDatabaseItem(const FPoseSearchDatabaseAnimationAssetBase* InItem);

	/** Copy contents to system's clipboard */
	void CopyToClipboard();

	/** Paste all items from clipboard to a given pose search database. Note that this will mark the target database as dirty. */
	void PasteToDatabase(UPoseSearchDatabase* InTargetDatabase) const;

	/** Determine if a pose search database clipboard content can be constructed from system's clipboard. */
	static bool CanPasteContentFromClipboard(const FString & InTextToImport);

	/** Attempt to construct a pose search database clipboard content from system's clipboard. Will return nullptr if it fails to do so. */
	static UPoseSearchDatabaseEditorClipboardContent* CreateFromClipboard();

	/** Create new transient clipboard content object  */
	static UPoseSearchDatabaseEditorClipboardContent* Create();
};