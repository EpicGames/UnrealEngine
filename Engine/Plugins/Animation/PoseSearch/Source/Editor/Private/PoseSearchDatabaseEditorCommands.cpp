// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDatabaseEditorCommands.h"

#define LOCTEXT_NAMESPACE "PoseSearchDatabaseEditorCommands"

namespace UE::PoseSearch
{
	void FDatabaseEditorCommands::RegisterCommands()
	{
		UI_COMMAND(ShowDisplayRootMotionSpeed, "Display Root Motion Speed", "Show root motion speed for the selected item", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(ShowQuantizeAnimationToPoseData, "Quantize Animation To Pose Data", "Animations will not playing continuously, only using the associated pose quantized time", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(ShowBones, "Show All Bones", "Debug Draw All Bones", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(ShowDisplayBlockTransition, "Display Block Transition", "Show root motion samples (green) with block transition (red)", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(ShowDisplayEventData, "Display Events", "Show Events (blue)", EUserInterfaceActionType::ToggleButton, FInputChord());
	}
}

#undef LOCTEXT_NAMESPACE
