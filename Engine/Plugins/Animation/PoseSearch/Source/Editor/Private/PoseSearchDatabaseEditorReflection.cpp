// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDatabaseEditorReflection.h"
#include "PoseSearch/PoseSearchDerivedData.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearchDatabaseAssetTree.h"
#include "PoseSearchDatabaseAssetTreeNode.h"
#include "PoseSearchDatabaseViewModel.h"
#include "StructUtils/InstancedStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchDatabaseEditorReflection)

#define LOCTEXT_NAMESPACE "UPoseSearchDatabaseReflection"

void UPoseSearchDatabaseStatistics::Initialize(const UPoseSearchDatabase* PoseSearchDatabase)
{
	static FText TimeFormat = LOCTEXT("TimeFormat", "{0} {0}|plural(one=Second,other=Seconds)");
	
	if (IsValid(PoseSearchDatabase))
	{
		const UE::PoseSearch::FSearchIndex& SearchIndex = PoseSearchDatabase->GetSearchIndex();
		// General Information
	
		AnimationSequences = PoseSearchDatabase->GetNumAnimationAssets();
			
		const int32 SampleRate = FMath::Max(1, PoseSearchDatabase->Schema->SampleRate);
		TotalAnimationPosesInFrames = SearchIndex.GetNumPoses();
		TotalAnimationPosesInTime = FText::Format(TimeFormat, static_cast<double>(TotalAnimationPosesInFrames) / SampleRate);
			
		uint32 NumOfSearchablePoses = 0;
		for (const UE::PoseSearch::FPoseMetadata& PoseMetadata : SearchIndex.PoseMetadata)
		{
			if (!PoseMetadata.IsBlockTransition())
			{
				++NumOfSearchablePoses;
			}
		}
		SearchableFrames = NumOfSearchablePoses;
		SearchableTime = FText::Format(TimeFormat, static_cast<double>(NumOfSearchablePoses) / SampleRate);

		SchemaCardinality = PoseSearchDatabase->Schema->SchemaCardinality;

		if (SchemaCardinality > 0)
		{
			const int32 TotalAnimationFeatureVectors = SearchIndex.GetNumValuesVectors(SchemaCardinality);
			PrunedFrames = TotalAnimationPosesInFrames - TotalAnimationFeatureVectors;
		}
		else
		{
			PrunedFrames = 0;
		}

		if (PoseSearchDatabase->GetNumberOfPrincipalComponents() > 0)
		{
			const int32 TotalAnimationPCAFeatureVectors = SearchIndex.GetNumPCAValuesVectors(PoseSearchDatabase->GetNumberOfPrincipalComponents());
			PrunedPCAFrames = TotalAnimationPosesInFrames - TotalAnimationPCAFeatureVectors;
		}
		else
		{
			PrunedPCAFrames = 0;
		}

		// Kinematic Information
	
		// using FText instead of meta = (ForceUnits = "cm/s") to keep properties consistent
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		AverageSpeed = FText::Format(LOCTEXT("StatsAverageSpeed", "{0} cm/s"), SearchIndex.Stats.AverageSpeed);
		MaxSpeed = FText::Format(LOCTEXT("StatsMaxSpeed", "{0} cm/s"), SearchIndex.Stats.MaxSpeed);
		AverageAcceleration = FText::Format(LOCTEXT("StatsAverageAcceleration", "{0} cm/s²"), SearchIndex.Stats.AverageAcceleration);
		MaxAcceleration = FText::Format(LOCTEXT("StatsMaxAcceleration", "{0} cm/s²"), SearchIndex.Stats.MaxAcceleration);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		// Principal Component Analysis
			
		ExplainedVariance = SearchIndex.PCAExplainedVarianceEditorOnly * 100.f;
			
		// Memory Information
			
		{
			uint32 SourceAnimAssetsSizeCookedEstimateInBytes = 0;
			TSet<const UObject*> Analyzed;
			const int32 NumAnimationAssets = PoseSearchDatabase->GetNumAnimationAssets();
			Analyzed.Reserve(NumAnimationAssets);
			for (int32 AnimationAssetIndex = 0; AnimationAssetIndex < NumAnimationAssets; ++AnimationAssetIndex)
			{
				if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAssetBase = PoseSearchDatabase->GetDatabaseAnimationAsset(AnimationAssetIndex))
				{
					bool bAlreadyAnalyzed = false;
					Analyzed.Add(DatabaseAnimationAssetBase->GetAnimationAsset(), &bAlreadyAnalyzed);
					if (!bAlreadyAnalyzed && DatabaseAnimationAssetBase->GetAnimationAsset())
					{
						SourceAnimAssetsSizeCookedEstimateInBytes += DatabaseAnimationAssetBase->GetApproxCookedSize();
					}
				}
			}

			const uint32 ValuesBytesSize = SearchIndex.Values.GetAllocatedSize();
			const uint32 PCAValuesBytesSize = SearchIndex.PCAValues.GetAllocatedSize();
			const uint32 KDTreeBytesSize = SearchIndex.KDTree.GetAllocatedSize();
			const uint32 VPTreeBytesSize = SearchIndex.VPTree.GetAllocatedSize();
			const uint32 ValuesVectorToPoseIndexesBytesSize = SearchIndex.ValuesVectorToPoseIndexes.GetAllocatedSize();
			const uint32 PCAValuesVectorToPoseIndexesBytesSize = SearchIndex.PCAValuesVectorToPoseIndexes.GetAllocatedSize();

			const uint32 PoseMetadataBytesSize = SearchIndex.PoseMetadata.GetAllocatedSize();
			const uint32 AssetsBytesSize = SearchIndex.Assets.GetAllocatedSize();
			const uint32 EventDataBytesSize = SearchIndex.EventData.GetAllocatedSize();
			const uint32 OtherBytesSize = SearchIndex.PCAProjectionMatrix.GetAllocatedSize() + SearchIndex.Mean.GetAllocatedSize() + SearchIndex.WeightsSqrt.GetAllocatedSize();
			const uint32 EstimatedDatabaseBytesSize = ValuesBytesSize + PCAValuesBytesSize + KDTreeBytesSize + VPTreeBytesSize + ValuesVectorToPoseIndexesBytesSize + PCAValuesVectorToPoseIndexesBytesSize + PoseMetadataBytesSize + AssetsBytesSize + EventDataBytesSize + OtherBytesSize + SourceAnimAssetsSizeCookedEstimateInBytes;
				
			ValuesSize = FText::AsMemory(ValuesBytesSize);
			PCAValuesSize = FText::AsMemory(PCAValuesBytesSize);
			KDTreeSize = FText::AsMemory(KDTreeBytesSize);
			VPTreeSize = FText::AsMemory(VPTreeBytesSize);
			PoseMetadataSize = FText::AsMemory(PoseMetadataBytesSize);
			AssetsSize = FText::AsMemory(AssetsBytesSize);
			EventDataSize = FText::AsMemory(EventDataBytesSize);
			EstimatedDatabaseSize = FText::AsMemory(EstimatedDatabaseBytesSize);
			SourceAnimAssetsSizeCookedEstimate = FText::AsMemory(SourceAnimAssetsSizeCookedEstimateInBytes);
		}
	}
}

#undef LOCTEXT_NAMESPACE
