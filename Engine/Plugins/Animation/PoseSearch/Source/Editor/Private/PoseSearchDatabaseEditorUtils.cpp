// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDatabaseEditorUtils.h"

#include "Animation/AnimationAsset.h"
#include "PoseSearch/MultiAnimAsset.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchSchema.h"

namespace UE::PoseSearch
{
	bool FPoseSearchEditorUtils::IsAssetCompatibleWithDatabase(const UPoseSearchDatabase* InDatabase, const FAssetData& InAssetData)
	{
		if (!InDatabase || !InDatabase->Schema)
		{
			return false;
		}
			
		const TArray<FPoseSearchRoledSkeleton> RoledSkeletons = InDatabase->Schema->GetRoledSkeletons();
		if (RoledSkeletons.IsEmpty())
		{
			return false;
		}

		const UClass* InAssetDataClass = InAssetData.GetClass();
		if (!InAssetDataClass)
		{
			return false;
		}

		if (RoledSkeletons.Num() == 1)
		{
			if (RoledSkeletons[0].Skeleton && InAssetDataClass->IsChildOf(UAnimationAsset::StaticClass()) &&
				RoledSkeletons[0].Skeleton->IsCompatibleForEditor(InAssetData))
			{
				// We found a compatible skeleton in the schema.
				return true;
			}
			
			return false;
		}
				
		if (!InAssetDataClass->IsChildOf(UMultiAnimAsset::StaticClass()))
		{
			return false;
		}
		
		// loading the UMultiAnimAsset
		const UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(InAssetData.GetAsset());
		check(MultiAnimAsset)

		if (MultiAnimAsset->GetNumRoles() != RoledSkeletons.Num())
		{
			return false;
		}

		for (const FPoseSearchRoledSkeleton& RoledSkeleton : RoledSkeletons)
		{
			if (const UAnimationAsset* RoledAnimationAsset = MultiAnimAsset->GetAnimationAsset(RoledSkeleton.Role))
			{
				if (!RoledAnimationAsset->GetSkeleton()->IsCompatibleForEditor(RoledSkeleton.Skeleton))
				{
					return false;
				}
			}
			else
			{
				// couldn't find a necessary asset for the RoledSkeleton
				return false;
			}
		}

		// we passed all the compatibility requirements!
		return true;
	}
}
