// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDatabaseFactory.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "IContentBrowserSingleton.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "Widgets/Layout/SBorder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchDatabaseFactory)

#define LOCTEXT_NAMESPACE "PoseSearchEditor"

UPoseSearchDatabaseFactory::UPoseSearchDatabaseFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UPoseSearchDatabase::StaticClass();
}

void UPoseSearchDatabaseFactory::OnTargetSchemaSelected(const FAssetData& SelectedAsset)
{
	TargetSchema = Cast<UPoseSearchSchema>(SelectedAsset.GetAsset());
	PickerWindow->RequestDestroyWindow();
}

bool UPoseSearchDatabaseFactory::ConfigureProperties()
{
	// Load the content browser module to display an asset picker
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	FAssetPickerConfig AssetPickerConfig;
	
	/** The asset picker will only show UPoseSearchSchema(s) */
	AssetPickerConfig.Filter.ClassPaths.Add(UPoseSearchSchema::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = true;

	/** The delegate that fires when an asset was selected */
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateUObject(this, &UPoseSearchDatabaseFactory::OnTargetSchemaSelected);

	/** The default view mode should be a list view */
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

	PickerWindow = SNew(SWindow)
	.Title(LOCTEXT("CreatePoseSearchDatabaseOptions", "Pick Schema"))
	.ClientSize(FVector2D(500, 600))
	.SupportsMinimize(false)
	.SupportsMaximize(false)
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		]
	];

	GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
	PickerWindow.Reset();

	return TargetSchema != nullptr;
}

UObject* UPoseSearchDatabaseFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UPoseSearchDatabase* Database = nullptr;
	if (TargetSchema)
	{
		Database = NewObject<UPoseSearchDatabase>(InParent, Class, Name, Flags);
		Database->Schema = TargetSchema;
	}
	return Database;
}

FString UPoseSearchDatabaseFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("NewPoseSearchDatabase"));
}

#undef LOCTEXT_NAMESPACE
