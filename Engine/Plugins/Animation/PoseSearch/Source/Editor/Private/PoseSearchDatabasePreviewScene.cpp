// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDatabasePreviewScene.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AnimPreviewInstance.h"
#include "Components/StaticMeshComponent.h"
#include "EngineUtils.h"
#include "GameFramework/WorldSettings.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchDerivedData.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearchDatabaseEditor.h"
#include "PoseSearchDatabaseViewModel.h"

namespace UE::PoseSearch
{
	FDatabasePreviewScene::FDatabasePreviewScene(
		ConstructionValues CVs,
		const TSharedRef<FDatabaseEditor>& Editor)
		: FAdvancedPreviewScene(CVs)
		, EditorPtr(Editor)
	{
		// Disable killing actors outside of the world
		AWorldSettings* WorldSettings = GetWorld()->GetWorldSettings(true);
		WorldSettings->bEnableWorldBoundsChecks = false;

		// Spawn an owner for FloorMeshComponent so CharacterMovementComponent can detect it as a valid floor and slide 
		// along it
		{
			AActor* FloorActor = GetWorld()->SpawnActor<AActor>(AActor::StaticClass(), FTransform());
			check(FloorActor);

			static const FString NewName = FString(TEXT("FloorComponent"));
			FloorMeshComponent->Rename(*NewName, FloorActor);

			FloorActor->SetRootComponent(FloorMeshComponent);
		}
	}

	void FDatabasePreviewScene::Tick(float InDeltaTime)
	{
		FAdvancedPreviewScene::Tick(InDeltaTime);

		// Trigger Begin Play in this preview world.
		// This is needed for the CharacterMovementComponent to be able to switch to falling mode. 
		// See: UCharacterMovementComponent::StartFalling
		if (PreviewWorld && !PreviewWorld->GetBegunPlay())
		{
			for (FActorIterator It(PreviewWorld); It; ++It)
			{
				It->DispatchBeginPlay();
			}

			PreviewWorld->SetBegunPlay(true);
		}

		GetWorld()->Tick(LEVELTICK_All, InDeltaTime);
	}
}
