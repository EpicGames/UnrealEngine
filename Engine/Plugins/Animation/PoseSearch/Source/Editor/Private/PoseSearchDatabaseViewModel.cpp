// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDatabaseViewModel.h"
#include "AnimPreviewInstance.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/PreviewAssetAttachComponent.h"
#include "Animation/Skeleton.h"
#include "Animation/TrajectoryTypes.h"
#include "Animation/MirrorDataTable.h"
#include "Camera/CameraComponent.h"
#include "Components/CapsuleComponent.h"
#include "ComponentAssetBroker.h"
#include "EngineUtils.h"
#include "Particles/ParticleSystemComponent.h"
#include "GameFramework/PlayerController.h"
#include "StructUtils/InstancedStruct.h"
#include "Modules/ModuleManager.h"
#include "PoseSearch/MultiAnimAsset.h"
#include "PoseSearch/PoseSearchAnimNotifies.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "PoseSearch/PoseSearchDerivedData.h"
#include "PoseSearch/PoseSearchHistory.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearch/PoseSearchSettings.h"
#include "PoseSearchDatabaseAssetTreeNode.h"
#include "PoseSearchDatabaseDataDetails.h"
#include "PoseSearchDatabasePreviewScene.h"
#include "PoseSearchEditor.h"
#include "PropertyEditorModule.h"

namespace UE::PoseSearch
{
#if ENABLE_ANIM_DEBUG
static float GVarDatabasePreviewDebugDrawSamplerSize = 0.f;
static FAutoConsoleVariableRef CVarDatabasePreviewDebugDrawSamplerSize(TEXT("a.DatabasePreview.DebugDrawSamplerSize"), GVarDatabasePreviewDebugDrawSamplerSize, TEXT("Debug Draw Sampler Positions Size"));

static float GVarDatabasePreviewDebugDrawSamplerTimeOffset = 0.f;
static FAutoConsoleVariableRef CVarDatabasePreviewDebugDrawSamplerTimeOffset(TEXT("a.DatabasePreview.DebugDrawSamplerTimeOffset"), GVarDatabasePreviewDebugDrawSamplerTimeOffset, TEXT("Debug Draw Sampler Positions At Time Offset"));

static float GVarDatabasePreviewDebugDrawSamplerRootAxisLength = 0.f;
static FAutoConsoleVariableRef CVarDatabasePreviewDebugDrawSamplerRootAxisLength(TEXT("a.DatabasePreview.DebugDrawSamplerRootAxisLength"), GVarDatabasePreviewDebugDrawSamplerRootAxisLength, TEXT("Debug Draw Sampler Root Axis Length"));
#endif

namespace
{
	static USceneComponent* FindExistingPreviewAttachment(USceneComponent* Parent, UObject* Asset, FName AttachedTo)
	{
		if (!Parent || !Asset)
		{
			return nullptr;
		}

		for (USceneComponent* Child : Parent->GetAttachChildren())
		{
			if (Child && Child->GetAttachSocketName() == AttachedTo &&
				FComponentAssetBrokerage::GetAssetFromComponent(Child) == Asset)
			{
				return Child;
			}
		}
		return nullptr;
	}

	static void AttachPreviewAssetToMesh(USkeletalMeshComponent* Mesh, UObject* Asset, FName AttachedTo)
	{
		if (!Mesh || !Asset || FindExistingPreviewAttachment(Mesh, Asset, AttachedTo))
		{
			return;
		}

		TSubclassOf<UActorComponent> ComponentClass = FComponentAssetBrokerage::GetPrimaryComponentForAsset(Asset->GetClass());
		if (!*ComponentClass || !ComponentClass->IsChildOf(USceneComponent::StaticClass()))
		{
			return;
		}

		USceneComponent* SceneComponent = NewObject<USceneComponent>(Mesh->GetOwner() ? (UObject*)Mesh->GetOwner() : (UObject*)Mesh, ComponentClass, NAME_None, RF_Transient);
		FComponentAssetBrokerage::AssignAssetToComponent(SceneComponent, Asset);

		if (UParticleSystemComponent* PSysComp = Cast<UParticleSystemComponent>(SceneComponent))
		{
			PSysComp->SetTickGroup(TG_PostUpdateWork);
		}

		SceneComponent->SetupAttachment(Mesh, AttachedTo);
		SceneComponent->RegisterComponent();
	}

	static void RefreshPreviewAttachedAssets(UDebugSkelMeshComponent* Mesh)
	{
		if (!Mesh)
		{
			return;
		}

		// Detach any prior preview attachments so a mesh swap doesn't leave stale ones behind.
		// Iterate backwards because DestroyComponent mutates the AttachChildren array.
		const TArray<TObjectPtr<USceneComponent>> ChildrenSnapshot = Mesh->GetAttachChildren();
		for (int32 Index = ChildrenSnapshot.Num() - 1; Index >= 0; --Index)
		{
			USceneComponent* Child = ChildrenSnapshot[Index];
			if (Child && FComponentAssetBrokerage::GetAssetFromComponent(Child))
			{
				Child->DestroyComponent();
			}
		}

		// Reapply attachments declared on the skeletal mesh asset...
		if (USkeletalMesh* SkeletalMesh = Mesh->GetSkeletalMeshAsset())
		{
			FPreviewAssetAttachContainer& Container = SkeletalMesh->GetPreviewAttachedAssetContainer();
			for (int32 Index = 0; Index < Container.Num(); ++Index)
			{
				const FPreviewAttachedObjectPair& Pair = Container[Index];
				AttachPreviewAssetToMesh(Mesh, Pair.GetAttachedObject(), Pair.AttachedTo);
			}

			// ...and on the skeleton.
			if (const USkeleton* Skeleton = SkeletalMesh->GetSkeleton())
			{
				for (int32 Index = 0; Index < Skeleton->PreviewAttachedAssetContainer.Num(); ++Index)
				{
					const FPreviewAttachedObjectPair& Pair = Skeleton->PreviewAttachedAssetContainer[Index];
					AttachPreviewAssetToMesh(Mesh, Pair.GetAttachedObject(), Pair.AttachedTo);
				}
			}
		}
	}
}

// FDatabasePreviewActor
bool FDatabasePreviewActor::SpawnPreviewActor(UWorld* World, const UPoseSearchDatabase* PoseSearchDatabase, int32 IndexAssetIdx, const FRole& Role, const FTransform& SamplerRootTransformOrigin, int32 PoseIdxForTimeOffset)
{
	check(PoseSearchDatabase && PoseSearchDatabase->Schema);
	const FSearchIndex& SearchIndex = PoseSearchDatabase->GetSearchIndex();
	const FSearchIndexAsset& IndexAsset = SearchIndex.Assets[IndexAssetIdx];
	const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAsset = PoseSearchDatabase->GetDatabaseAnimationAsset(IndexAsset.GetSourceAssetIdx());
	check(DatabaseAnimationAsset);
		
	USkeleton* Skeleton = PoseSearchDatabase->Schema->GetSkeleton(Role);
	if (!Skeleton)
	{
		UE_LOGF(LogPoseSearchEditor, Log, "Couldn't spawn preview Actor for asset %ls because its Role '%ls' is missing in Schema '%ls'", *GetNameSafe(DatabaseAnimationAsset->GetAnimationAsset()), *Role.ToString(), *PoseSearchDatabase->Schema->GetName());
		return false;
	}

	UAnimationAsset* PreviewAsset = Cast<UAnimationAsset>(DatabaseAnimationAsset->GetAnimationAssetForRole(Role));
	if (!PreviewAsset)
	{
		return false;
	}

	ActorRole = Role;
	IndexAssetIndex = IndexAssetIdx;
	CurrentPoseIndex = INDEX_NONE;

	FActorSpawnParameters Params;
	Params.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButAlwaysSpawn;
	ActorPtr = World->SpawnActor<AActor>(AActor::StaticClass(), FTransform::Identity, Params);
	ActorPtr->SetFlags(RF_Transient);

	UDebugSkelMeshComponent* Mesh = NewObject<UDebugSkelMeshComponent>(ActorPtr.Get());
	Mesh->RegisterComponentWithWorld(World);

	UAnimPreviewInstance* AnimInstance = NewObject<UAnimPreviewInstance>(Mesh);
	Mesh->PreviewInstance = AnimInstance;
	AnimInstance->InitializeAnimation();

	USkeletalMesh* PreviewMesh = DatabaseAnimationAsset->GetPreviewMeshForRole(Role);
	if (!PreviewMesh)
	{
		PreviewMesh = PoseSearchDatabase->PreviewMesh;
		if (!PreviewMesh)
		{
			PreviewMesh = Skeleton->GetPreviewMesh(true);
		}
	}
	
	Mesh->SetSkeletalMesh(PreviewMesh);
	RefreshPreviewAttachedAssets(Mesh);
	Mesh->EnablePreview(true, PreviewAsset);
		
	AnimInstance->SetAnimationAsset(PreviewAsset, IndexAsset.IsLooping(), 0.f);
	AnimInstance->SetBlendSpacePosition(IndexAsset.GetBlendParameters());
		
	if (IndexAsset.IsMirrored())
	{
		const UMirrorDataTable* MirrorDataTable = PoseSearchDatabase->Schema->GetMirrorDataTable(Role);
		AnimInstance->SetMirrorDataTable(MirrorDataTable);
	}

	const FMirrorDataCache MirrorDataCache(AnimInstance->GetMirrorDataTable(), AnimInstance->GetRequiredBonesOnAnyThread());
	
	Sampler.Init(PreviewAsset, SamplerRootTransformOrigin, IndexAsset.GetBlendParameters(), FAnimationAssetSampler::DefaultRootTransformSamplingRate, true, false);

	PlayTimeOffset = 0.f;
	if (PoseIdxForTimeOffset >= 0)
	{
		PlayTimeOffset = PoseSearchDatabase->GetRealAssetTime(PoseIdxForTimeOffset) - IndexAsset.GetFirstSampleTime(PoseSearchDatabase->Schema->SampleRate);

		if (DatabaseAnimationAsset->GetNumRoles() > 1)
		{
			// @todo: implement support for UMultiAnimAsset. the transform should be centered to the origin of the multi character animation!
		}
		else
		{
			// centering the Sampler RootTransformOrigin at PlayTimeOffset time, to be able to "align" multiple actors from different animation frames when selected by the pose search debugger
			FTransform NewSamplerRootTransformOrigin = MirrorDataCache.MirrorTransform(Sampler.ExtractRootTransform(0.f));
			NewSamplerRootTransformOrigin.SetToRelativeTransform(MirrorDataCache.MirrorTransform(Sampler.ExtractRootTransform(PlayTimeOffset)));
			Sampler.SetRootTransformOrigin(NewSamplerRootTransformOrigin);
		}
	}

	AnimInstance->PlayAnim(IndexAsset.IsLooping(), 0.f);
	if (!ActorPtr->GetRootComponent())
	{
		ActorPtr->SetRootComponent(Mesh);
	}

	AnimInstance->SetPlayRate(0.f);

	// initializing Trajectory and TrajectorySpeed
	const int NumPoses = IndexAsset.GetNumPoses();
	Trajectory.Samples.SetNumUninitialized(NumPoses);
	TrajectorySpeed.SetNumUninitialized(NumPoses);

	for (int32 Index = 0; Index < NumPoses; ++Index)
	{
		const int32 IndexAssetPoseIdx = Index + IndexAsset.GetFirstPoseIdx();
		const float IndexAssetPoseTime = IndexAsset.GetTimeFromPoseIndex(IndexAssetPoseIdx, PoseSearchDatabase->Schema->SampleRate);
		const FTransform IndexAssetPoseTransform = MirrorDataCache.MirrorTransform(Sampler.ExtractRootTransform(IndexAssetPoseTime));

		Trajectory.Samples[Index].SetTransform(IndexAssetPoseTransform);
		Trajectory.Samples[Index].TimeInSeconds = IndexAssetPoseTime;
	}

	for (int32 Index = 1; Index < NumPoses; ++Index)
	{
		const float DeltaAccumulatedSeconds = FMath::Max(UE_KINDA_SMALL_NUMBER, Trajectory.Samples[Index].TimeInSeconds - Trajectory.Samples[Index - 1].TimeInSeconds);
		const FVector& Start = Trajectory.Samples[Index - 1].Position;
		const FVector& End = Trajectory.Samples[Index].Position;
		TrajectorySpeed[Index] = (Start - End).Length() / DeltaAccumulatedSeconds;
	}
	
	if (NumPoses > 0)
	{
		TrajectorySpeed[0] = NumPoses > 1 ? TrajectorySpeed[1] : 0.f;
	}

	return true;
}

void FDatabasePreviewActor::UpdatePreviewActor(const UPoseSearchDatabase* PoseSearchDatabase, float PlayTime, bool bQuantizeAnimationToPoseData)
{
	check(PoseSearchDatabase);

	const FSearchIndex& SearchIndex = PoseSearchDatabase->GetSearchIndex();

	UAnimPreviewInstance* AnimInstance = GetAnimPreviewInstanceInternal();
	if (!AnimInstance || !SearchIndex.Assets.IsValidIndex(IndexAssetIndex))
	{
		return;
	}

	const UAnimationAsset* PreviewAsset = AnimInstance->GetAnimationAsset();
	if (!PreviewAsset)
	{
		return;
	}

	UDebugSkelMeshComponent* Mesh = GetDebugSkelMeshComponent();
	if (!Mesh)
	{
		return;
	}

	const FSearchIndexAsset& IndexAsset = SearchIndex.Assets[IndexAssetIndex];
	const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAsset = PoseSearchDatabase->GetDatabaseAnimationAsset(IndexAsset.GetSourceAssetIdx());

	USkeletalMesh* PreviewMesh = DatabaseAnimationAsset->GetPreviewMeshForRole(ActorRole);
	if (!PreviewMesh)
	{
		PreviewMesh = PoseSearchDatabase->PreviewMesh;
		if (!PreviewMesh)
		{
			if (USkeleton* Skeleton = PoseSearchDatabase->Schema->GetSkeleton(ActorRole))
			{
				PreviewMesh = Skeleton->GetPreviewMesh(true);
			}
		}
	}

	if (Mesh->GetSkeletalMeshAsset() != PreviewMesh)
	{
		Mesh->SetSkeletalMesh(PreviewMesh);
		RefreshPreviewAttachedAssets(Mesh);
	}

	CurrentTime = 0.f;
	float CurrentPlayTime = PlayTime + IndexAsset.GetFirstSampleTime(PoseSearchDatabase->Schema->SampleRate) + PlayTimeOffset;
	FAnimationRuntime::AdvanceTime(false, CurrentPlayTime, CurrentTime, IndexAsset.GetLastSampleTime(PoseSearchDatabase->Schema->SampleRate));
			 
	// time to pose index
	CurrentPoseIndex = IndexAsset.GetPoseIndexFromTime(CurrentTime, PoseSearchDatabase->Schema->SampleRate);

	QuantizedTime = CurrentPoseIndex >= 0 ? PoseSearchDatabase->GetRealAssetTime(CurrentPoseIndex) : CurrentTime;
	if (bQuantizeAnimationToPoseData)
	{
		CurrentTime = QuantizedTime;
	}

	// SetPosition is in [0..1] range for blendspaces
	AnimInstance->SetPosition(Sampler.ToNormalizedTime(CurrentTime));
	AnimInstance->SetPlayRate(0.f);
	AnimInstance->SetBlendSpacePosition(IndexAsset.GetBlendParameters());

	check(ActorPtr != nullptr);
	ActorPtr->SetActorTransform(Trajectory.GetSampleAtTime(CurrentTime).GetTransform());
}

void FDatabasePreviewActor::Destroy()
{
	if (ActorPtr != nullptr)
	{
		ActorPtr->Destroy();
	}
}

bool FDatabasePreviewActor::DrawPreviewActors(TConstArrayView<FDatabasePreviewActor> PreviewActors, const UPoseSearchDatabase* PoseSearchDatabase, bool bDisplayRootMotionSpeed, bool bDisplayBlockTransition, bool bDisplayEventData, TConstArrayView<float> QueryVector)
{
	using namespace UE::PoseSearch;

	UWorld* CommonWorld = nullptr;
	int32 CommonCurrentPoseIndex = INDEX_NONE;
#if DO_CHECK
	int32 CommonIndexAssetIndex = INDEX_NONE;
#endif // DO_CHECK

	
	TArray<FChooserEvaluationContext, TInlineAllocator<PreallocatedRolesNum>> AnimContextsData;
	TArray<FChooserEvaluationContext*, TInlineAllocator<PreallocatedRolesNum>> AnimContexts;
	FRoleToIndex RoleToIndex;
	TArray<FArchivedPoseHistory, TInlineAllocator<PreallocatedRolesNum>> ArchivedPoseHistories;
	TArray<const IPoseHistory*, TInlineAllocator<PreallocatedRolesNum>> PoseHistories;

	const int32 NumPreviewActors = PreviewActors.Num();
	if (NumPreviewActors > PreallocatedRolesNum)
	{
		// reserve the needed amount of memory for containers
		AnimContexts.Reserve(NumPreviewActors);
		RoleToIndex.Reserve(NumPreviewActors);
		ArchivedPoseHistories.Reserve(NumPreviewActors);
		PoseHistories.Reserve(NumPreviewActors);
	}
	
	AnimContextsData.SetNum(NumPreviewActors);

	for (const FDatabasePreviewActor& PreviewActor : PreviewActors)
	{
		const FSearchIndex& SearchIndex = PoseSearchDatabase->GetSearchIndex();

		// This condition happens when the database got reindexed and the new valid SearchIndex has different cardinality for assets or poses.
		// Since we didn't refresh the PreviewActor IndexAssetIndex nor CurrentPoseIndex, now the PreviewActor is invalid
		// @todo: we should refresh the PreviewActor and restore it's preview time etc
		if (!SearchIndex.Assets.IsValidIndex(PreviewActor.GetIndexAssetIndex()) ||
			!SearchIndex.IsValidPoseIndex(PreviewActor.GetCurrentPoseIndex()))
		{
			return false;
		}

		const UDebugSkelMeshComponent* Mesh = PreviewActor.GetDebugSkelMeshComponent();
		if (!Mesh)
		{
			return false;
		}

		if (!CommonWorld)
		{
			CommonWorld = Mesh->GetWorld();
		}
		else if (CommonWorld != Mesh->GetWorld())
		{
			return false;
		}

		// making sure PreviewActors are consistent with each other
		if (CommonCurrentPoseIndex == INDEX_NONE)
		{
			CommonCurrentPoseIndex = PreviewActor.GetCurrentPoseIndex();
		}
		else if (CommonCurrentPoseIndex != PreviewActor.GetCurrentPoseIndex())
		{
			checkNoEntry();
			return false;
		}

#if DO_CHECK
		if (CommonIndexAssetIndex == INDEX_NONE)
		{
			CommonIndexAssetIndex = PreviewActor.GetIndexAssetIndex();
		}
		else if (CommonIndexAssetIndex != PreviewActor.GetIndexAssetIndex())
		{
			checkNoEntry();
			return false;
		}
#endif // DO_CHECK

		int Index = AnimContexts.Num();
		RoleToIndex.Add(PreviewActor.ActorRole) = AnimContexts.Num();
		AnimContexts.Add(&AnimContextsData[Index]);
		AnimContextsData[Index].AddObjectParam(const_cast<UDebugSkelMeshComponent*>(Mesh));

		FArchivedPoseHistory& ArchivedPoseHistory = ArchivedPoseHistories.AddDefaulted_GetRef();
		ArchivedPoseHistory.Trajectory = PreviewActor.Trajectory;

		check(PoseSearchDatabase && PoseSearchDatabase->Schema);
		if (const USkeleton* Skeleton = PoseSearchDatabase->Schema->GetSkeleton(PreviewActor.ActorRole))
		{
			// reconstructing ArchivedPoseHistory::BoneToTransformMap and ArchivedPoseHistory::Entries ONLY for the root bone.
			// @todo: add more bones if needed
			const TArray<FTransform>& RefBonePose = Skeleton->GetReferenceSkeleton().GetRefBonePose();
			const FTransform& RefRootBone = RefBonePose[RootBoneIndexType];

			ArchivedPoseHistory.BoneToTransformMap.Add(RootBoneIndexType) = RootBoneIndexType;
			FPoseHistoryEntry& PoseHistoryEntry = ArchivedPoseHistory.Entries.AddDefaulted_GetRef();
			// saving space for the root bone only
			PoseHistoryEntry.SetNum(1, true);
			PoseHistoryEntry.SetComponentSpaceTransform(RootBoneIndexType, RefRootBone);
		}

		for (FTransformTrajectorySample& TrajectorySample : ArchivedPoseHistory.Trajectory.Samples)
		{
			TrajectorySample.TimeInSeconds -= PreviewActor.QuantizedTime;
		}

		PoseHistories.Add(&ArchivedPoseHistory);
	}

	FDebugDrawParams DrawParams(AnimContexts, PoseHistories, RoleToIndex, PoseSearchDatabase, FDebugDrawParams::EDrawContext::DrawQuery);
	DrawParams.DrawFeatureVector(CommonCurrentPoseIndex);

	if (!QueryVector.IsEmpty())
	{
		DrawParams.DrawFeatureVector(QueryVector);
	}

	for (const FDatabasePreviewActor& PreviewActor : PreviewActors)
	{
		const UDebugSkelMeshComponent* Mesh = PreviewActor.GetDebugSkelMeshComponent();
		const FSearchIndex& SearchIndex = PoseSearchDatabase->GetSearchIndex();
		const FSearchIndexAsset& IndexAsset = SearchIndex.Assets[PreviewActor.GetIndexAssetIndex()];
		const int32 SamplesNum = PreviewActor.Trajectory.Samples.Num();

		if (bDisplayRootMotionSpeed)
		{
			// @todo: should we be using PreviewActor.Trajectory.DebugDrawTrajectory instead?
			// drawing PreviewActor.Trajectory
			
			if (SamplesNum > 1)
			{
				for (int32 Index = 0; Index < SamplesNum; ++Index)
				{
					const FVector& EndDown = PreviewActor.Trajectory.Samples[Index].Position;
					const FVector EndUp = EndDown + (PreviewActor.TrajectorySpeed[Index] * FVector::UpVector);

					DrawParams.DrawLine(EndDown, EndUp, FColor::Black);
					if (Index > 0)
					{
						const FColor RootMotionColor = Index % 2 == 0 ? FColor::Purple : FColor::Orange;
						const FVector& StartDown = PreviewActor.Trajectory.Samples[Index - 1].Position;
						const FVector StartUp = StartDown + (PreviewActor.TrajectorySpeed[Index - 1] * FVector::UpVector);
						DrawParams.DrawLine(StartDown, EndDown, RootMotionColor);
						DrawParams.DrawLine(StartUp, EndUp, RootMotionColor);
					}
				}
			}
		}

		if (bDisplayBlockTransition)
		{
			const int NumPoses = IndexAsset.GetNumPoses();
			if (NumPoses == SamplesNum)
			{
				for (int32 Index = 0; Index < SamplesNum; ++Index)
				{
					const int32 IndexAssetPoseIdx = Index + IndexAsset.GetFirstPoseIdx();
					if (SearchIndex.PoseMetadata[IndexAssetPoseIdx].IsBlockTransition())
					{
						DrawParams.DrawPoint(PreviewActor.Trajectory.Samples[Index].Position, FColor::Red);
					}
					else
					{
						DrawParams.DrawPoint(PreviewActor.Trajectory.Samples[Index].Position, FColor::Green);
					}
				}
			}
		}

		if (bDisplayEventData && !SearchIndex.EventData.GetData().IsEmpty())
		{
			const int NumPoses = IndexAsset.GetNumPoses();
			if (NumPoses == SamplesNum)
			{
				TSet<int32, DefaultKeyFuncs<int32>, TInlineSetAllocator<256>> AllEventEventDataPoseIndexes;
				for (const FEventData::FTagToPoseIndexes& TagToPoseIndexes : SearchIndex.EventData.GetData())
				{
					for (int32 PoseIdx : TagToPoseIndexes.Value)
					{
						AllEventEventDataPoseIndexes.Add(PoseIdx);
					}
				}
				
				for (int32 Index = 0; Index < SamplesNum; ++Index)
				{
					const int32 IndexAssetPoseIdx = Index + IndexAsset.GetFirstPoseIdx();
					if (AllEventEventDataPoseIndexes.Find(IndexAssetPoseIdx))
					{
						DrawParams.DrawPoint(PreviewActor.Trajectory.Samples[Index].Position, FColor::Blue, 8.f);
					}
				}
			}
		}

#if ENABLE_ANIM_DEBUG
		const float DebugDrawSamplerSize = GVarDatabasePreviewDebugDrawSamplerSize;
		if (DebugDrawSamplerSize > UE_KINDA_SMALL_NUMBER)
		{
			const float DebugDrawSamplerTimeOffset = GVarDatabasePreviewDebugDrawSamplerTimeOffset;
			
			const int32 NumDrawPasses = FMath::IsNearlyZero(DebugDrawSamplerTimeOffset) ? 1 : 2;

			FMemMark Mark(FMemStack::Get());
			FCompactPose Pose;
			FCSPose<FCompactPose> ComponentSpacePose;

			const FMirrorDataCache MirrorDataCache(Mesh->PreviewInstance->GetMirrorDataTable(), Mesh->PreviewInstance->GetRequiredBonesOnAnyThread());

			for (int32 DrawPass = 0; DrawPass < NumDrawPasses; ++DrawPass)
			{
				// drawing the pose extracted from the Sampler to visually compare with the pose features and the mesh drawing
				Pose.SetBoneContainer(&PreviewActor.GetAnimPreviewInstance()->GetRequiredBonesOnAnyThread());

				const float SamplerTime = DrawPass ? PreviewActor.CurrentTime + DebugDrawSamplerTimeOffset : PreviewActor.CurrentTime;
				const FColor DebugColor = DrawPass ? FColor::Blue : FColor::Red;

				PreviewActor.Sampler.ExtractPose(SamplerTime, Pose);
				MirrorDataCache.MirrorPose(Pose);
				ComponentSpacePose.InitPose(MoveTemp(Pose));

				const FTransform RootTransform = MirrorDataCache.MirrorTransform(PreviewActor.Sampler.ExtractRootTransform(SamplerTime));
				const float DebugDrawSamplerRootAxisLength = GVarDatabasePreviewDebugDrawSamplerRootAxisLength;
				if (DebugDrawSamplerRootAxisLength > 0.f)
				{
					DrawParams.DrawLine(RootTransform.GetTranslation(), RootTransform.GetTranslation() + RootTransform.GetScaledAxis(EAxis::X) * DebugDrawSamplerRootAxisLength, FColor::Red);
					DrawParams.DrawLine(RootTransform.GetTranslation(), RootTransform.GetTranslation() + RootTransform.GetScaledAxis(EAxis::Y) * DebugDrawSamplerRootAxisLength, FColor::Green);
					DrawParams.DrawLine(RootTransform.GetTranslation(), RootTransform.GetTranslation() + RootTransform.GetScaledAxis(EAxis::Z) * DebugDrawSamplerRootAxisLength, FColor::Blue);
				}

				for (int32 BoneIndex = 0; BoneIndex < ComponentSpacePose.GetPose().GetNumBones(); ++BoneIndex)
				{
					const FTransform BoneWorldTransforms = ComponentSpacePose.GetComponentSpaceTransform(FCompactPoseBoneIndex(BoneIndex)) * RootTransform;
					DrawParams.DrawPoint(BoneWorldTransforms.GetTranslation(), DebugColor, DebugDrawSamplerSize);
				}
			}
		}
#endif // ENABLE_ANIM_DEBUG
	}
	return true;
}

const UDebugSkelMeshComponent* FDatabasePreviewActor::GetDebugSkelMeshComponent() const
{
	if (ActorPtr != nullptr)
	{
		return Cast<UDebugSkelMeshComponent>(ActorPtr->GetRootComponent());
	}
	return nullptr;
}

UDebugSkelMeshComponent* FDatabasePreviewActor::GetDebugSkelMeshComponent()
{
	if (ActorPtr != nullptr)
	{
		return Cast<UDebugSkelMeshComponent>(ActorPtr->GetRootComponent());
	}
	return nullptr;
}

const UAnimPreviewInstance* FDatabasePreviewActor::GetAnimPreviewInstance() const
{
	if (const UDebugSkelMeshComponent* Mesh = GetDebugSkelMeshComponent())
	{
		return Mesh->PreviewInstance.Get();
	}
	return nullptr;
}

UAnimPreviewInstance* FDatabasePreviewActor::GetAnimPreviewInstanceInternal()
{
	if (ActorPtr != nullptr)
	{
		if (UDebugSkelMeshComponent* Mesh = Cast<UDebugSkelMeshComponent>(ActorPtr->GetRootComponent()))
		{
			return Mesh->PreviewInstance.Get();
		}
	}
	return nullptr;
}

// FDatabaseViewModel
void FDatabaseViewModel::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(PoseSearchDatabasePtr);
}

void FDatabaseViewModel::Initialize(UPoseSearchDatabase* InPoseSearchDatabase, const TSharedRef<FDatabasePreviewScene>& InPreviewScene, const TSharedRef<SDatabaseDataDetails>& InDatabaseDataDetails)
{
	PoseSearchDatabasePtr = InPoseSearchDatabase;
	PreviewScenePtr = InPreviewScene;
	DatabaseDataDetails = InDatabaseDataDetails;

	RemovePreviewActors();
}

void FDatabaseViewModel::BuildSearchIndex()
{
	using namespace UE::PoseSearch;
	FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(GetPoseSearchDatabase(), ERequestAsyncBuildFlag::NewRequest);
}

void FDatabaseViewModel::PreviewBackwardEnd()
{
	SetPlayTime(MinPreviewPlayLength, false);
}

void FDatabaseViewModel::PreviewBackwardStep()
{
	const float NewPlayTime = FMath::Clamp(PlayTime - StepDeltaTime, MinPreviewPlayLength, MaxPreviewPlayLength);
	SetPlayTime(NewPlayTime, false);
}

void FDatabaseViewModel::PreviewBackward()
{
	DeltaTimeMultiplier = -1.f;
}

void FDatabaseViewModel::PreviewPause()
{
	DeltaTimeMultiplier = 0.f;
}

void FDatabaseViewModel::PreviewForward()
{
	DeltaTimeMultiplier = 1.f;
}

void FDatabaseViewModel::PreviewForwardStep()
{
	const float NewPlayTime = FMath::Clamp(PlayTime + StepDeltaTime, MinPreviewPlayLength, MaxPreviewPlayLength);
	SetPlayTime(NewPlayTime, false);
}

void FDatabaseViewModel::PreviewForwardEnd()
{
	SetPlayTime(MaxPreviewPlayLength, false);
}

UWorld* FDatabaseViewModel::GetWorld()
{
	check(PreviewScenePtr.IsValid());
	return PreviewScenePtr.Pin()->GetWorld();
}

void FDatabaseViewModel::OnPreviewActorClassChanged()
{
	// todo: implement
}

void FDatabaseViewModel::Tick(float DeltaSeconds)
{
	if (!PreviewActors.IsEmpty())
	{
		const float DeltaPlayTime = DeltaSeconds * DeltaTimeMultiplier;

		const UPoseSearchDatabase* Database = GetPoseSearchDatabase();
		if (EAsyncBuildIndexResult::Success == FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(Database, ERequestAsyncBuildFlag::ContinueRequest))
		{
			PlayTime += DeltaPlayTime;
			PlayTime = FMath::Clamp(PlayTime, MinPreviewPlayLength, MaxPreviewPlayLength);

			for (TArray<FDatabasePreviewActor>& PreviewActorGroup : PreviewActors)
			{
				for (FDatabasePreviewActor& PreviewActor : PreviewActorGroup)
				{
					PreviewActor.UpdatePreviewActor(Database, PlayTime, bQuantizeAnimationToPoseData);
				}
			}

			bool bShouldDrawQueryVector = ShouldDrawQueryVector();
			for (TArray<FDatabasePreviewActor>& PreviewActorGroup : PreviewActors)
			{
				bShouldDrawQueryVector &= !FDatabasePreviewActor::DrawPreviewActors(PreviewActorGroup, Database, bDisplayRootMotionSpeed, bDisplayBlockTransition, bDisplayEventData, bShouldDrawQueryVector ? GetQueryVector() : TConstArrayView<float>());
			}
		}
	}
}

void FDatabaseViewModel::RemovePreviewActors()
{
	PlayTime = 0.f;
	DeltaTimeMultiplier = 1.f;
	MaxPreviewPlayLength = 0.f;
	MinPreviewPlayLength = 0.f;
	bIsEditorSelection = true;
	bDrawQueryVector = false;

	for (TArray<FDatabasePreviewActor>& PreviewActorGroup : PreviewActors)
	{
		for (FDatabasePreviewActor& PreviewActor : PreviewActorGroup)
		{
			PreviewActor.Destroy();
		}
	}

	PreviewActors.Reset();
}

void FDatabaseViewModel::AddAnimationAssetToDatabase(UObject* AnimationAsset)
{
	if (UPoseSearchDatabase* Database = GetPoseSearchDatabase())
	{
		Database->Modify();
		
		FPoseSearchDatabaseAnimationAsset NewAsset;
		NewAsset.AnimAsset = AnimationAsset;
		Database->AddAnimationAsset(NewAsset);

		Database->PostEditChange();
	}
}

bool FDatabaseViewModel::DeleteFromDatabase(int32 AnimationAssetIndex)
{
	if (UPoseSearchDatabase* Database = GetPoseSearchDatabase())
	{
		if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAssetBase = Database->GetDatabaseAnimationAsset(AnimationAssetIndex))
		{
			Database->Modify();
			
			if (DatabaseAnimationAssetBase->IsSynchronizedWithExternalDependency())
			{
				if (UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(DatabaseAnimationAssetBase->GetAnimationAsset()))
				{
					bool bModified = false;
					for (int32 NotifyIndex = AnimSequenceBase->Notifies.Num() - 1; NotifyIndex >= 0; --NotifyIndex)
					{
						const FAnimNotifyEvent& NotifyEvent = AnimSequenceBase->Notifies[NotifyIndex];
						if (NotifyEvent.NotifyStateClass && NotifyEvent.NotifyStateClass->GetClass()->IsChildOf<UAnimNotifyState_PoseSearchBranchIn>())
						{
							const UAnimNotifyState_PoseSearchBranchIn* PoseSearchBranchIn = Cast<UAnimNotifyState_PoseSearchBranchIn>(NotifyEvent.NotifyStateClass);
							check(PoseSearchBranchIn);

							if (PoseSearchBranchIn->Database == Database && PoseSearchBranchIn->GetBranchInId() == DatabaseAnimationAssetBase->BranchInId)
							{
								if (!bModified)
								{
									AnimSequenceBase->Modify();
									bModified = true;
								}

								AnimSequenceBase->Notifies.RemoveAt(NotifyIndex);
							}
						}
					}

					if (bModified)
					{
						AnimSequenceBase->RefreshCacheData();
					}
				}
				else
				{
					UE_LOGF(LogPoseSearchEditor, Error, "found DatabaseAnimationAssetBase with valid BranchInId, but invalid AnimSequenceBase in %ls", *Database->GetName());
				}
			}

			Database->RemoveAnimationAssetAt(AnimationAssetIndex);

			Database->PostEditChange();

			return true;
		}
	}

	return false;
}

void FDatabaseViewModel::SetDisableReselection(int32 AnimationAssetIndex, bool bEnabled)
{
	if (UPoseSearchDatabase* Database = GetPoseSearchDatabase())
	{
		if (const FPoseSearchDatabaseAnimationAsset* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(AnimationAssetIndex))
		{
			Database->Modify();
			
			FPoseSearchDatabaseAnimationAsset NewDatabaseAnimationAsset = *DatabaseAnimationAsset;
			NewDatabaseAnimationAsset.SetDisableReselection(bEnabled);
			Database->SetAnimationAssetAt(NewDatabaseAnimationAsset, AnimationAssetIndex);

			Database->PostEditChange();
		}
	}
}

bool FDatabaseViewModel::IsDisableReselection(int32 AnimationAssetIndex) const
{
	if (const UPoseSearchDatabase* Database = GetPoseSearchDatabase())
	{
		if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(AnimationAssetIndex))
		{
			return DatabaseAnimationAsset->IsDisableReselection();
		}
	}

	return false;
}

void FDatabaseViewModel::SetIsEnabled(int32 AnimationAssetIndex, bool bEnabled)
{
	if (UPoseSearchDatabase* Database = GetPoseSearchDatabase())
	{
		if (const FPoseSearchDatabaseAnimationAsset* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(AnimationAssetIndex))
		{
			Database->Modify();

			FPoseSearchDatabaseAnimationAsset NewDatabaseAnimationAsset = *DatabaseAnimationAsset;
			NewDatabaseAnimationAsset.SetIsEnabled(bEnabled);
			Database->SetAnimationAssetAt(NewDatabaseAnimationAsset, AnimationAssetIndex);

			Database->PostEditChange();
		}
	}
}

bool FDatabaseViewModel::IsEnabled(int32 AnimationAssetIndex) const
{
	if (const UPoseSearchDatabase* Database = GetPoseSearchDatabase())
	{
		if (const FPoseSearchDatabaseAnimationAsset* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(AnimationAssetIndex))
		{
			return DatabaseAnimationAsset->IsEnabled();
		}
	}

	return false;
}

bool FDatabaseViewModel::SetAnimationAsset(int32 AnimationAssetIndex, UObject* AnimAsset)
{
	if (AnimAsset)
	{
		if (UPoseSearchDatabase* Database = GetPoseSearchDatabase())
		{
			if (const FPoseSearchDatabaseAnimationAsset* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(AnimationAssetIndex))
			{
				Database->Modify();

				FPoseSearchDatabaseAnimationAsset NewDatabaseAnimationAsset = *DatabaseAnimationAsset;
				NewDatabaseAnimationAsset.AnimAsset = AnimAsset;
				Database->SetAnimationAssetAt(NewDatabaseAnimationAsset, AnimationAssetIndex);

				Database->PostEditChange();
				return true;
			}
		}
	}
	
	return false;
}

void FDatabaseViewModel::SetMirrorOption(int32 AnimationAssetIndex, EPoseSearchMirrorOption InMirrorOption)
{
	if (UPoseSearchDatabase* Database = GetPoseSearchDatabase())
	{
		if (const FPoseSearchDatabaseAnimationAsset* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(AnimationAssetIndex))
		{
			Database->Modify();

			FPoseSearchDatabaseAnimationAsset NewDatabaseAnimationAsset = *DatabaseAnimationAsset;
			NewDatabaseAnimationAsset.MirrorOption = InMirrorOption;
			Database->SetAnimationAssetAt(NewDatabaseAnimationAsset, AnimationAssetIndex);

			Database->PostEditChange();
		}
	}
}

EPoseSearchMirrorOption FDatabaseViewModel::GetMirrorOption(int32 AnimationAssetIndex)
{
	if (const UPoseSearchDatabase* Database = GetPoseSearchDatabase())
	{
		if (const FPoseSearchDatabaseAnimationAsset* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(AnimationAssetIndex))
		{
			return DatabaseAnimationAsset->MirrorOption;
		}
	}

	return EPoseSearchMirrorOption::MirroredOnly;
}

int32 FDatabaseViewModel::SetSelectedNode(int32 PoseIdx, bool bClearSelection, bool bDrawQuery, TConstArrayView<float> InQueryVector)
{
	int32 SelectedSourceAssetIdx = INDEX_NONE;

	if (bClearSelection)
	{
		RemovePreviewActors();
	}

	bIsEditorSelection = false;
	bDrawQueryVector = bDrawQuery;
	QueryVector = InQueryVector;

	if (const UPoseSearchDatabase* Database = GetPoseSearchDatabase())
	{
		if (EAsyncBuildIndexResult::Success == FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(Database, ERequestAsyncBuildFlag::ContinueRequest))
		{
			const FSearchIndex& SearchIndex = Database->GetSearchIndex();
			if (SearchIndex.PoseMetadata.IsValidIndex(PoseIdx))
			{
				const uint32 IndexAssetIndex = SearchIndex.PoseMetadata[PoseIdx].GetAssetIndex();
				if (SearchIndex.Assets.IsValidIndex(IndexAssetIndex))
				{
					const FSearchIndexAsset& IndexAsset = SearchIndex.Assets[IndexAssetIndex];
					const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(IndexAsset.GetSourceAssetIdx());
					check(DatabaseAnimationAsset);
					int32 PreviewActorGroupIndex = INDEX_NONE;
					for (int32 RoleIndex = 0; RoleIndex < DatabaseAnimationAsset->GetNumRoles(); ++RoleIndex)
					{
						FDatabasePreviewActor PreviewActor;
						const UE::PoseSearch::FRole Role = DatabaseAnimationAsset->GetRole(RoleIndex);
						const FTransform& RootTransformOrigin = DatabaseAnimationAsset->GetRootTransformOriginForRole(Role);
						if (PreviewActor.SpawnPreviewActor(GetWorld(), Database, IndexAssetIndex, Role, RootTransformOrigin, PoseIdx))
						{
							if (PreviewActorGroupIndex == INDEX_NONE)
							{
								PreviewActorGroupIndex = PreviewActors.AddDefaulted();
							}

							MaxPreviewPlayLength = FMath::Max(MaxPreviewPlayLength, IndexAsset.GetLastSampleTime(Database->Schema->SampleRate) - PreviewActor.GetPlayTimeOffset());
							MinPreviewPlayLength = FMath::Min(MinPreviewPlayLength, IndexAsset.GetFirstSampleTime(Database->Schema->SampleRate) - PreviewActor.GetPlayTimeOffset());
							PreviewActors[PreviewActorGroupIndex].Add(MoveTemp(PreviewActor));
							SelectedSourceAssetIdx = IndexAsset.GetSourceAssetIdx();
						}
					}
				}
			}

			DatabaseDataDetails.Pin()->Reconstruct();

			for (TArray<FDatabasePreviewActor>& PreviewActorGroup : PreviewActors)
			{
				for (FDatabasePreviewActor& PreviewActor : PreviewActorGroup)
				{
					PreviewActor.UpdatePreviewActor(Database, PlayTime, bQuantizeAnimationToPoseData);
				}
			}

			SetPlayTime(0.f, false);
		}
	}

	ProcessSelectedActor(nullptr);

	return SelectedSourceAssetIdx;
}

void FDatabaseViewModel::SetSelectedNodes(const TArrayView<TSharedPtr<FDatabaseAssetTreeNode>>& InSelectedNodes)
{
	RemovePreviewActors();

	if (const UPoseSearchDatabase* Database = GetPoseSearchDatabase())
	{
		if (EAsyncBuildIndexResult::Success == FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(Database, ERequestAsyncBuildFlag::ContinueRequest))
		{
			TMap<int32, int32> AssociatedAssetIndices;
			for (int32 i = 0; i < InSelectedNodes.Num(); ++i)
			{
				AssociatedAssetIndices.FindOrAdd(InSelectedNodes[i]->SourceAssetIdx) = i;
			}

			const int32 MaxPreviewActors = UPoseSearchSettings::Get().MaxDatabaseEditorPreviewActors;
			int32 SpawnedActorCount = 0;
			bool bHitPreviewActorCap = false;

			const FSearchIndex& SearchIndex = Database->GetSearchIndex();
			for (int32 IndexAssetIndex = 0; IndexAssetIndex < SearchIndex.Assets.Num() && !bHitPreviewActorCap; ++IndexAssetIndex)
			{
				const FSearchIndexAsset& IndexAsset = SearchIndex.Assets[IndexAssetIndex];
				if (AssociatedAssetIndices.Find(IndexAsset.GetSourceAssetIdx()))
				{
					const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAsset = Database->GetDatabaseAnimationAsset(IndexAsset.GetSourceAssetIdx());
					check(DatabaseAnimationAsset);
					int32 PreviewActorGroupIndex = INDEX_NONE;
					for (int32 RoleIndex = 0; RoleIndex < DatabaseAnimationAsset->GetNumRoles(); ++RoleIndex)
					{
						if (SpawnedActorCount >= MaxPreviewActors)
						{
							bHitPreviewActorCap = true;
							break;
						}

						FDatabasePreviewActor PreviewActor;
						const UE::PoseSearch::FRole Role = DatabaseAnimationAsset->GetRole(RoleIndex);
						const FTransform RootTransformOrigin = DatabaseAnimationAsset->GetRootTransformOriginForRole(Role);
						if (PreviewActor.SpawnPreviewActor(GetWorld(), Database, IndexAssetIndex, Role, RootTransformOrigin))
						{
							if (PreviewActorGroupIndex == INDEX_NONE)
							{
								PreviewActorGroupIndex = PreviewActors.AddDefaulted();
							}

							MaxPreviewPlayLength = FMath::Max(MaxPreviewPlayLength, IndexAsset.GetLastSampleTime(Database->Schema->SampleRate) - IndexAsset.GetFirstSampleTime(Database->Schema->SampleRate));
							PreviewActors[PreviewActorGroupIndex].Add(MoveTemp(PreviewActor));
							++SpawnedActorCount;
						}
					}
				}
			}

			if (bHitPreviewActorCap)
			{
				UE_LOG(LogPoseSearchEditor, Warning, TEXT("Pose Search Database editor preview actor cap reached (%d). Increase MaxDatabaseEditorPreviewActors in Project Settings -> Pose Search to preview more assets simultaneously."), MaxPreviewActors);
			}

			DatabaseDataDetails.Pin()->Reconstruct();
			for (TArray<FDatabasePreviewActor>& PreviewActorGroup : PreviewActors)
			{
				for (FDatabasePreviewActor& PreviewActor : PreviewActorGroup)
				{
					PreviewActor.UpdatePreviewActor(Database, PlayTime, bQuantizeAnimationToPoseData);
				}
			}
		}

		ProcessSelectedActor(nullptr);
	}
}

void FDatabaseViewModel::ProcessSelectedActor(AActor* Actor)
{
	SelectedActorIndexAssetIndex = INDEX_NONE;

	for (const TArray<FDatabasePreviewActor>& PreviewActorGroup : PreviewActors)
	{
		for (const FDatabasePreviewActor& PreviewActor : PreviewActorGroup)
		{
			if (PreviewActor.GetActor() == Actor)
			{
				SelectedActorIndexAssetIndex = PreviewActor.GetIndexAssetIndex();
				return;
			}
		}
	}
}

void FDatabaseViewModel::SetDrawQueryVector(bool bValue)
{
	if (bDrawQueryVector != bValue)
	{
		bDrawQueryVector = bValue;
		DatabaseDataDetails.Pin()->Reconstruct();
	}
}

const FSearchIndexAsset* FDatabaseViewModel::GetSelectedActorIndexAsset() const
{
	if (SelectedActorIndexAssetIndex >= 0)
	{
		const UPoseSearchDatabase* Database = GetPoseSearchDatabase();
		if (EAsyncBuildIndexResult::Success == FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(Database, ERequestAsyncBuildFlag::ContinueRequest))
		{
			const FSearchIndex& SearchIndex = Database->GetSearchIndex();
			if (SearchIndex.Assets.IsValidIndex(SelectedActorIndexAssetIndex))
			{
				return &SearchIndex.Assets[SelectedActorIndexAssetIndex];
			}
		}
	}
	return nullptr;
}

TRange<double> FDatabaseViewModel::GetPreviewPlayRange() const
{
	constexpr double ViewRangeSlack = 0.2;
	return TRange<double>(MinPreviewPlayLength - ViewRangeSlack, MaxPreviewPlayLength + ViewRangeSlack);
}

float FDatabaseViewModel::GetPlayTime() const
{
	return PlayTime;
}

void FDatabaseViewModel::SetPlayTime(float NewPlayTime, bool bInTickPlayTime)
{
	PlayTime = FMath::Clamp(NewPlayTime, MinPreviewPlayLength, MaxPreviewPlayLength);
	DeltaTimeMultiplier = bInTickPlayTime ? DeltaTimeMultiplier : 0.f;

	if (const UPoseSearchDatabase* Database = GetPoseSearchDatabase())
	{
		if (EAsyncBuildIndexResult::Success == FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(Database, ERequestAsyncBuildFlag::ContinueRequest))
		{
			for (TArray<FDatabasePreviewActor>& PreviewActorGroup : PreviewActors)
			{
				for (FDatabasePreviewActor& PreviewActor : PreviewActorGroup)
				{
					PreviewActor.UpdatePreviewActor(Database, PlayTime, bQuantizeAnimationToPoseData);
				}
			}
		}
	}
}

bool FDatabaseViewModel::GetAnimationTime(int32 SourceAssetIdx, float& CurrentPlayTime, FVector& BlendParameters) const
{
	if (const UPoseSearchDatabase* Database = GetPoseSearchDatabase())
	{
		if (EAsyncBuildIndexResult::Success == FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(Database, ERequestAsyncBuildFlag::ContinueRequest))
		{
			const FSearchIndex& SearchIndex = Database->GetSearchIndex();
			for (const TArray<FDatabasePreviewActor>& PreviewActorGroup : PreviewActors)
			{
				for (const FDatabasePreviewActor& PreviewActor : PreviewActorGroup)
				{
					if (PreviewActor.GetIndexAssetIndex() >= 0 && PreviewActor.GetIndexAssetIndex() < SearchIndex.Assets.Num())
					{
						const FSearchIndexAsset& IndexAsset = SearchIndex.Assets[PreviewActor.GetIndexAssetIndex()];
						if (IndexAsset.GetSourceAssetIdx() == SourceAssetIdx)
						{
							CurrentPlayTime = PreviewActor.GetSampler().ToNormalizedTime(PlayTime + IndexAsset.GetFirstSampleTime(Database->Schema->SampleRate) + PreviewActor.GetPlayTimeOffset());
							BlendParameters = IndexAsset.GetBlendParameters();
							return true;
						}
					}
				}
			}

			for (const FSearchIndexAsset& IndexAsset : SearchIndex.Assets)
			{
				if (IndexAsset.GetSourceAssetIdx() == SourceAssetIdx)
				{
					CurrentPlayTime = PlayTime + IndexAsset.GetFirstSampleTime(Database->Schema->SampleRate);
					BlendParameters = IndexAsset.GetBlendParameters();

					if (const UObject* AnimationAsset = Database->GetDatabaseAnimationAsset(IndexAsset.GetSourceAssetIdx())->GetAnimationAsset())
					{
						if (AnimationAsset->IsA<UBlendSpace>() && !FMath::IsNearlyEqual(MaxPreviewPlayLength, MinPreviewPlayLength))
						{
							CurrentPlayTime = (CurrentPlayTime - MaxPreviewPlayLength) / (MaxPreviewPlayLength - MinPreviewPlayLength);
						}
					}
					return true;
				}
			}
		}
	}

	CurrentPlayTime = 0.f;
	BlendParameters = FVector::ZeroVector;
	return false;
}
}
