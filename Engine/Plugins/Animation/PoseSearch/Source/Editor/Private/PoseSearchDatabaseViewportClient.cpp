// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDatabaseViewportClient.h"
#include "PoseSearchDatabaseEditor.h"
#include "PoseSearchDatabasePreviewScene.h"
#include "SPoseSearchDatabaseViewport.h"
#include "AssetEditorModeManager.h"
#include "UnrealWidget.h"
#include "PoseSearchDatabaseEdMode.h"

namespace UE::PoseSearch
{
	FDatabaseViewportClient::FDatabaseViewportClient(
		const TSharedRef<FDatabasePreviewScene>& InPreviewScene,
		const TSharedRef<SDatabaseViewport>& InViewport,
		const TSharedRef<FDatabaseEditor>& InAssetEditor)
		: FEditorViewportClient(nullptr, &InPreviewScene.Get(), StaticCastSharedRef<SEditorViewport>(InViewport))
		, PreviewScenePtr(InPreviewScene)
		, AssetEditorPtr(InAssetEditor)
	{
		Widget->SetUsesEditorModeTools(ModeTools.Get());
		StaticCastSharedPtr<FAssetEditorModeManager>(ModeTools)->SetPreviewScene(&InPreviewScene.Get());
		ModeTools->SetDefaultMode(FDatabaseEdMode::EdModeId);

		SetRealtime(true);

		SetWidgetCoordSystemSpace(COORD_Local);
		ModeTools->SetWidgetMode(UE::Widget::WM_Translate);
	}

	void FDatabaseViewportClient::TrackingStarted(
		const struct FInputEventState& InInputState,
		bool bIsDraggingWidget,
		bool bNudge)
	{
		ModeTools->StartTracking(this, Viewport);
	}

	void FDatabaseViewportClient::TrackingStopped()
	{
		ModeTools->EndTracking(this, Viewport);
		Invalidate();
	}

	void FDatabaseViewportClient::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
	{
		FEditorViewportClient::Draw(View, PDI);
	}
}
