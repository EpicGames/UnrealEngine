// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDebugger.h"
#include "IAnimationProvider.h"
#include "IGameplayProvider.h"
#include "IRewindDebugger.h"
#include "PoseSearchDebuggerView.h"
#include "PoseSearchDebuggerViewModel.h"
#include "SSimpleTimeSlider.h"
#include "Styling/SlateIconFinder.h"
#include "Trace/PoseSearchTraceProvider.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "PoseSearchDebugger"

namespace UE::PoseSearch
{

typedef SCurveTimelineView::FTimelineCurveData::CurvePoint FCurvePoint;
class SCostCurveTimelineView : public SCurveTimelineView
{
public:

	SLATE_BEGIN_ARGS(SCostCurveTimelineView) {}
		SLATE_ATTRIBUTE(FLinearColor, CurveColor)
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs );

	TRange<double> GetViewRange() const { return ViewRange.Get(); }
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override { return FReply::Unhandled(); }

	TSharedPtr<SCostCurveTimelineView::FTimelineCurveData> CurveData;
};

void SCostCurveTimelineView::Construct(const FArguments& InArgs)
{
	CurveData = MakeShared<SCurveTimelineView::FTimelineCurveData>();

	SCurveTimelineView::FArguments CurveTimelineViewArgs;

	CurveTimelineViewArgs
	.CurveColor(InArgs._CurveColor)
	.ViewRange_Lambda([]()
	{
		return IRewindDebugger::Instance()->GetCurrentViewRange();
	})
	.RenderFill(false)
	.CurveData_Lambda([this]()
	{
		return CurveData;
	});

	SCurveTimelineView::Construct(CurveTimelineViewArgs);
}

///////////////////////////////////////////////////////
// SCostTimelineView
class SCostTimelineView : public SOverlay
{
public:
	SLATE_BEGIN_ARGS(SCostTimelineView)
	: _SearchId(0)
	{}
		SLATE_ARGUMENT( int32, SearchId )

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	void UpdateInternal(uint64 ObjectId);
	int32 GetSearchId() const { return SearchId; }

protected:
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	TSharedPtr<SCostCurveTimelineView> BestCostView;
	TSharedPtr<SCostCurveTimelineView> BruteForceCostView;
	TSharedPtr<SCostCurveTimelineView> BestPosePosView;

	TSharedPtr<SToolTip> CostToolTip;

	FText ToolTipTime;
	FText ToolTipCost;
	FText ToolTipCostBruteForce;
	FText ToolTipBestPosePos;
	int32 SearchId = 0;
};

void SCostTimelineView::Construct(const FArguments& InArgs)
{
	SearchId = InArgs._SearchId;

	BestCostView = SNew(SCostCurveTimelineView).CurveColor(FLinearColor::White);
	BruteForceCostView = SNew(SCostCurveTimelineView).CurveColor(FLinearColor::Red);
	BestPosePosView = SNew(SCostCurveTimelineView).CurveColor(FLinearColor::Blue);
		
	AddSlot()
	[
		BruteForceCostView.ToSharedRef()
	];
	AddSlot()
	[
		BestCostView.ToSharedRef()
	];
	AddSlot()
	[
		BestPosePosView.ToSharedRef()
	];
}

void SCostTimelineView::UpdateInternal(uint64 ObjectId)
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();

	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	check(AnalysisSession);
	if (const FTraceProvider* PoseSearchProvider = AnalysisSession->ReadProvider<FTraceProvider>(FTraceProvider::ProviderName))
	{
		BestCostView->CurveData->Points.Reset();
		BruteForceCostView->CurveData->Points.Reset();
		BestPosePosView->CurveData->Points.Reset();

		// convert time range to from rewind debugger times to profiler times
		TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
		double StartTime = TraceTimeRange.GetLowerBoundValue();
		double EndTime = TraceTimeRange.GetUpperBoundValue();

		TraceServices::FProviderReadScopeLock ProviderReadScope(*PoseSearchProvider);
		PoseSearchProvider->EnumerateMotionMatchingStateTimelines(ObjectId, [StartTime, EndTime, this](const FTraceProvider::FMotionMatchingStateTimeline& InTimeline)
		{
			// this isn't very efficient, and it gets called every frame.  will need optimizing
			InTimeline.EnumerateEvents(StartTime, EndTime, [StartTime, EndTime, this](double InStartTime, double InEndTime, uint32 InDepth, const FTraceMotionMatchingStateMessage& InMessage)
			{
				if (InMessage.GetSearchId() == SearchId && InEndTime > StartTime && InStartTime < EndTime)
				{
					BestCostView->CurveData->Points.Add({ InMessage.RecordingTime, InMessage.SearchBestCost });
					BruteForceCostView->CurveData->Points.Add({ InMessage.RecordingTime, InMessage.SearchBruteForceCost });
					BestPosePosView->CurveData->Points.Add({ InMessage.RecordingTime, float(InMessage.SearchBestPosePos) });
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});

		float MinValue = UE_MAX_FLT;
		float MaxValue = -UE_MAX_FLT;

		bool bAnyInvalidBestCostPoints = false;
		bool bAnyInvalidBruteForceCostPoints = false;

		bool bAnyValidBestCostPoints = false;
		bool bAnyValidBruteForceCostPoints = false;
		for (const FCurvePoint& CurvePoint : BestCostView->CurveData->Points)
		{
			if (FPoseSearchCost::IsCostValid(CurvePoint.Value))
			{
				MinValue = FMath::Min(MinValue, CurvePoint.Value);
				MaxValue = FMath::Max(MaxValue, CurvePoint.Value);
				bAnyValidBestCostPoints = true;
			}
			else
			{
				bAnyInvalidBestCostPoints = true;
			}
		}
		for (const FCurvePoint& CurvePoint : BruteForceCostView->CurveData->Points)
		{
			if (FPoseSearchCost::IsCostValid(CurvePoint.Value))
			{
				MinValue = FMath::Min(MinValue, CurvePoint.Value);
				MaxValue = FMath::Max(MaxValue, CurvePoint.Value);
				bAnyValidBruteForceCostPoints = true;
			}
			else
			{
				bAnyInvalidBruteForceCostPoints = true;
			}
		}

		if ((bAnyInvalidBestCostPoints && bAnyValidBestCostPoints) || (bAnyInvalidBruteForceCostPoints && bAnyValidBruteForceCostPoints))
		{
			// highliting invalid cost points
			const float InvalidCostValue = (MaxValue - MinValue) * 2 + MinValue;
			MaxValue = InvalidCostValue;
		}

		if (bAnyInvalidBestCostPoints)
		{
			for (FCurvePoint& CurvePoint : BestCostView->CurveData->Points)
			{
				CurvePoint.Value = FMath::Min(MaxValue, CurvePoint.Value);
			}
		}

		BestCostView->SetFixedRange(MinValue, MaxValue);

		if (bAnyValidBruteForceCostPoints)
		{
			if (bAnyInvalidBruteForceCostPoints)
			{
				for (FCurvePoint& CurvePoint : BruteForceCostView->CurveData->Points)
				{
					CurvePoint.Value = FMath::Min(MaxValue, CurvePoint.Value);
				}
			}

			BruteForceCostView->SetFixedRange(MinValue, MaxValue);
			BruteForceCostView->SetVisibility(EVisibility::Visible);
		}
		else
		{
			BruteForceCostView->SetVisibility(EVisibility::Hidden);
		}
	}
}

FReply SCostTimelineView::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MyGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition()))
	{
		// Mouse position in widget space
		const FVector2D HitPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

		// Range helper struct
		const SSimpleTimeSlider::FScrubRangeToScreen RangeToScreen(BestCostView->GetViewRange(), MyGeometry.GetLocalSize());

		// Mouse position from widget space to curve input space
		const double TargetTime = RangeToScreen.LocalXToInput(HitPosition.X);

		// Get curve value at given time
		const TConstArrayView<FCurvePoint> CurvePoints = BestCostView->CurveData->Points;
		const int32 NumPoints = CurvePoints.Num();

		if (NumPoints > 0)
		{
			for (int32 i = 1; i < NumPoints; ++i)
			{
				const FCurvePoint& Point1 = CurvePoints[i - 1];
				const FCurvePoint& Point2 = CurvePoints[i];

				// Find points that contain mouse hit-point time
				if (Point1.Time >= TargetTime && TargetTime <= Point2.Time)
				{
					// Choose point with the smallest delta
					const float Delta1 = abs(TargetTime - Point1.Time);
					const float Delta2 = abs(TargetTime - Point2.Time);

					// Get closest point index
					const int32 TargetPointIndex = Delta1 < Delta2 ? i - 1 : i;

					const float Time = CurvePoints[TargetPointIndex].Time;
					const float BestCost = CurvePoints[TargetPointIndex].Value;
					const float BruteForceCost = BruteForceCostView->CurveData->Points[TargetPointIndex].Value;
					const int32 BestPosePos = FMath::RoundToInt(BestPosePosView->CurveData->Points[TargetPointIndex].Value);

					// Tooltip text formatting
					FNumberFormattingOptions FormattingOptions;
					FormattingOptions.MaximumFractionalDigits = 3;

					ToolTipBestPosePos = FText::Format(LOCTEXT("CostTimelineViewToolTip_BestPosePosFormat", "Best Index: {0}"), FText::AsNumber(BestPosePos, &FormattingOptions));
					ToolTipTime = FText::Format(LOCTEXT("CostTimelineViewToolTip_TimeFormat", "Search Time: {0}"), FText::AsNumber(Time, &FormattingOptions));
					ToolTipCost = FText::Format(LOCTEXT("CostTimelineViewToolTip_CostFormat", "Search Cost: {0}"), FText::AsNumber(BestCost, &FormattingOptions));

					if (!FPoseSearchCost::IsCostValid(BruteForceCost))
					{
						ToolTipCostBruteForce = FText::GetEmpty();
					}
					else
					{
						ToolTipCostBruteForce = FText::Format(LOCTEXT("CostTimelineViewToolTip_CostBruteForceFormat", "Search BruteForce Cost: {0}"), FText::AsNumber(BruteForceCost, &FormattingOptions));
					}

					// Update tooltip info
					if (!CostToolTip.IsValid())
					{
						SetToolTip(
							SAssignNew(CostToolTip, SToolTip)
							.BorderImage(FCoreStyle::Get().GetBrush("ToolTip.Background"))
							[
								SNew(SVerticalBox)
								+ SVerticalBox::Slot()
								[
									SNew(STextBlock)
									.Text_Lambda([this]() { return ToolTipTime; })
									.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
									.ColorAndOpacity(FLinearColor::Black)
								]
								+ SVerticalBox::Slot()
								[
									SNew(STextBlock)
									.Text_Lambda([this]() { return ToolTipBestPosePos; })
									.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
									.ColorAndOpacity(FLinearColor::Blue)
								]
								+ SVerticalBox::Slot()
								[
									SNew(STextBlock)
									.Text_Lambda([this]() { return ToolTipCost; })
									.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
									.ColorAndOpacity(FLinearColor::White)
								]
								+ SVerticalBox::Slot()
								[
									SNew(STextBlock)
									.Visibility_Lambda([this]() { return ToolTipCostBruteForce.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; })
									.Text_Lambda([this]() { return ToolTipCostBruteForce; })
									.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
									.ColorAndOpacity(FLinearColor::Red)
								]
							]);
					}

					break;
				}
			}
		}
	}

	return FReply::Unhandled();
}

///////////////////////////////////////////////////////
// FDebugger
FDebugger* FDebugger::Debugger;
void FDebugger::Initialize()
{
	check(Debugger == nullptr);
	Debugger = new FDebugger;
	IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, Debugger);
}

void FDebugger::Shutdown()
{
	if (Debugger != nullptr)
	{
		IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, Debugger);
		delete Debugger;
		Debugger = nullptr;
	}
}

bool FDebugger::IsPIESimulating()
{
	return Debugger->RewindDebugger->IsPIESimulating();
}

bool FDebugger::IsRecording()
{
	return Debugger->RewindDebugger->IsRecording();
}

double FDebugger::GetRecordingDuration()
{
	return Debugger->RewindDebugger->GetRecordingDuration();
}

UWorld* FDebugger::GetWorld()
{
	return Debugger->RewindDebugger->GetWorldToVisualize();
}

const IRewindDebugger* FDebugger::GetRewindDebugger()
{
	return Debugger->RewindDebugger;
}

void FDebugger::Update(float DeltaTime, IRewindDebugger* InRewindDebugger)
{
	// Update active rewind debugger in use
	RewindDebugger = InRewindDebugger;
}

void FDebugger::OnViewClosed(uint64 InAnimInstanceId)
{
	TArray<TSharedRef<FDebuggerViewModel>>& Models = Debugger->ViewModels;
	for (int32 i = 0; i < Models.Num(); ++i)
	{
		if (Models[i]->AnimInstanceId == InAnimInstanceId)
		{
			Models.RemoveAtSwap(i);
			return;
		}
	}
	// Should always be a valid remove
	checkNoEntry();
}

TSharedPtr<FDebuggerViewModel> FDebugger::GetViewModel(uint64 InAnimInstanceId)
{
	TArray<TSharedRef<FDebuggerViewModel>>& Models = Debugger->ViewModels;
	for (int32 i = 0; i < Models.Num(); ++i)
	{
		if (Models[i]->AnimInstanceId == InAnimInstanceId)
		{
			return Models[i];
		}
	}
	return nullptr;
}

TSharedPtr<SDebuggerView> FDebugger::GenerateInstance(uint64 InAnimInstanceId, int32 InWantedSearchId)
{
	ViewModels.Add_GetRef(MakeShared<FDebuggerViewModel>(InAnimInstanceId))->RewindDebugger.BindStatic(&FDebugger::GetRewindDebugger);

	TSharedPtr<SDebuggerView> DebuggerViewSharedPtr;
	SAssignNew(DebuggerViewSharedPtr, SDebuggerView, InAnimInstanceId, InWantedSearchId)
		.ViewModel_Static(&FDebugger::GetViewModel, InAnimInstanceId)
		.OnViewClosed_Static(&FDebugger::OnViewClosed);

	DebuggerView = DebuggerViewSharedPtr;
	return DebuggerViewSharedPtr;
}

///////////////////////////////////////////////////////
// FSearchTrack
FSearchTrack::FSearchTrack(uint64 InObjectId, int32 InSearchId, FText InTrackName)
: RewindDebugger::FRewindDebuggerTrack()
, CostTimelineView(SNew(SCostTimelineView).SearchId(InSearchId))
, ObjectId(InObjectId)
, TrackName(InTrackName)
, Icon(FSlateIconFinder::FindIconForClass(UAnimInstance::StaticClass()))
{
}

int32 FSearchTrack::GetSearchId() const
{
	return CostTimelineView->GetSearchId();
}

FText FSearchTrack::GetDisplayNameInternal() const
{
	return TrackName;
}

bool FSearchTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PoseSearchSearchTrack::UpdateInternal);
	CostTimelineView->UpdateInternal(ObjectId);

	return false;
}

TSharedPtr<SWidget> FSearchTrack::GetTimelineViewInternal()
{
	return CostTimelineView;
}

TSharedPtr<SWidget> FSearchTrack::GetDetailsViewInternal()
{
	return FDebugger::Get()->GenerateInstance(ObjectId, GetSearchId());
}

///////////////////////////////////////////////////////
// FDebuggerTrack
FDebuggerTrack::FDebuggerTrack(uint64 InObjectId)
: RewindDebugger::FRewindDebuggerTrack()
, ObjectId(InObjectId)
, Icon(FSlateIconFinder::FindIconForClass(UAnimInstance::StaticClass()))
{
}

bool FDebuggerTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PoseSearchDebuggerTrack::UpdateInternal);
	
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();

	if (TSharedPtr<IRewindDebuggerView> PinnedView = FDebugger::Get()->GetDebuggerView().Pin())
	{
		PinnedView->SetTimeMarker(RewindDebugger->CurrentTraceTime());
	}

	bool bChanged = false;

	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	check(AnalysisSession);
	if (const FTraceProvider* PoseSearchProvider = AnalysisSession->ReadProvider<FTraceProvider>(FTraceProvider::ProviderName))
	{
		// convert time range to from rewind debugger times to profiler times
		TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
		double StartTime = TraceTimeRange.GetLowerBoundValue();
		double EndTime = TraceTimeRange.GetUpperBoundValue();

		TArray<int32, TInlineAllocator<64>> OldSearchIds;
		for (TSharedPtr<FSearchTrack>& SearchTrack : SearchTracks)
		{
			OldSearchIds.Add(SearchTrack->GetSearchId());
		}

		TMap<int32, FText, TInlineSetAllocator<64>> SearchIdNames;
		const IGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<IGameplayProvider>("GameplayProvider");

		TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);
		TraceServices::FProviderReadScopeLock PoseSearchProviderReadScope(*PoseSearchProvider);
		PoseSearchProvider->EnumerateMotionMatchingStateTimelines(ObjectId, [StartTime, EndTime, &SearchIdNames, GameplayProvider](const FTraceProvider::FMotionMatchingStateTimeline& InTimeline)
			{
				// this isn't very efficient, and it gets called every frame.  will need optimizing
				InTimeline.EnumerateEvents(StartTime, EndTime, [StartTime, EndTime, &SearchIdNames, GameplayProvider](double InStartTime, double InEndTime, uint32 InDepth, const FTraceMotionMatchingStateMessage& InMessage)
					{
						if (!SearchIdNames.Find(InMessage.GetSearchId()) && InEndTime > StartTime && InStartTime < EndTime)
						{
							SearchIdNames.Add(InMessage.GetSearchId()) = GenerateSearchName(InMessage, GameplayProvider);
						}
						return TraceServices::EEventEnumerate::Continue;
					});
			});

		TArray<int32, TInlineAllocator<64>> SearchIds;
		for (TPair<int32, FText> SearchIdNamePair : SearchIdNames)
		{
			SearchIds.Add(SearchIdNamePair.Key);
		}
		SearchIds.StableSort();

		if (SearchIds != OldSearchIds)
		{
			TMap<int32, TSharedPtr<FSearchTrack>, TInlineSetAllocator<64>> OldSearchIdsMap;
			for (TSharedPtr<FSearchTrack>& SearchTrack : SearchTracks)
			{
				OldSearchIdsMap.Add(SearchTrack->GetSearchId()) = SearchTrack;
			}

			SearchTracks.SetNum(SearchIds.Num());
			for (int32 SearchIdIndex = 0; SearchIdIndex < SearchIds.Num(); ++SearchIdIndex)
			{
				if (TSharedPtr<FSearchTrack>* SearchTrack = OldSearchIdsMap.Find(SearchIds[SearchIdIndex]))
				{
					SearchTracks[SearchIdIndex] = *SearchTrack;
				}
				else
				{
					SearchTracks[SearchIdIndex] = MakeShared<FSearchTrack>(ObjectId, SearchIds[SearchIdIndex], SearchIdNames[SearchIds[SearchIdIndex]]);
				}
			}

			bChanged = true;
		}
		
		for (TSharedPtr<FSearchTrack>& SearchTrack : SearchTracks)
		{
			if (SearchTrack.IsValid())
			{
				bChanged |= SearchTrack->Update();
			}
		}
	}

	return bChanged;
}

TSharedPtr<SWidget> FDebuggerTrack::GetDetailsViewInternal()
{
	return FDebugger::Get()->GenerateInstance(ObjectId);
}

TConstArrayView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>> FDebuggerTrack::GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
{
	return MakeConstArrayView<TSharedPtr<FRewindDebuggerTrack>>(reinterpret_cast<const TSharedPtr<FRewindDebuggerTrack>*>(SearchTracks.GetData()), SearchTracks.Num());
}

} // namespace UE::PoseSearch

#undef LOCTEXT_NAMESPACE
