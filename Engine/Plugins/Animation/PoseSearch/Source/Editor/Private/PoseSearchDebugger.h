// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IRewindDebuggerExtension.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "RewindDebuggerTrack.h"
#include "SCurveTimelineView.h"

namespace TraceServices { class IAnalysisSession; }

namespace UE::PoseSearch
{

class SCostTimelineView;
class SDebuggerView;
class FDebuggerViewModel;

/**
 * PoseSearch debugger, containing the data to be acquired and relayed to the view
 */
class FDebugger : public TSharedFromThis<FDebugger>, public IRewindDebuggerExtension
{
public:
	virtual void Update(float DeltaTime, IRewindDebugger* InRewindDebugger) override;
	virtual ~FDebugger() = default;

	static FDebugger* Get() { return Debugger; }
	static void Initialize();
	static void Shutdown();
	virtual FString GetName() { return TEXT("PoseSearchDebugger"); }

	// Shared data from the Rewind Debugger singleton
	static bool IsPIESimulating();
	static bool IsRecording();
	static double GetRecordingDuration();
	static UWorld* GetWorld();
	static const IRewindDebugger* GetRewindDebugger();

	/** Generates the slate debugger view widget */
	TSharedPtr<SDebuggerView> GenerateInstance(uint64 InAnimInstanceId, int32 InWantedSearchId = InvalidSearchId);
	TWeakPtr<SDebuggerView> GetDebuggerView() { return DebuggerView; }

private:
	/** Removes the reference from the model array when closed, destroying the model */
	static void OnViewClosed(uint64 InAnimInstanceId);

	/** Acquire view model from the array */
	static TSharedPtr<FDebuggerViewModel> GetViewModel(uint64 InAnimInstanceId);

	/** Last stored Rewind Debugger */
	const IRewindDebugger* RewindDebugger = nullptr;

	/** List of all active debugger instances */
	TArray<TSharedRef<FDebuggerViewModel>> ViewModels;
	
	TWeakPtr<SDebuggerView> DebuggerView;

	/** Internal instance */
	static FDebugger* Debugger;
};

class FSearchTrack : public RewindDebugger::FRewindDebuggerTrack
{
public:
	FSearchTrack(uint64 InObjectId, int32 InSearchId, FText InTrackName);
	int32 GetSearchId() const;

private:
	virtual FSlateIcon GetIconInternal() override { return Icon;  }
	virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
	virtual FName GetNameInternal() const override { return "PoseSearchTrack"; }
	virtual FText GetDisplayNameInternal() const override;
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }
	virtual bool UpdateInternal() override;

	TSharedPtr<SCostTimelineView> CostTimelineView;

	uint64 ObjectId;
	FText TrackName;
	FSlateIcon Icon;
};

/**
 * Creates the slate widgets associated with the PoseSearch debugger
 * when prompted by the Rewind Debugger
 */
class FDebuggerTrack : public RewindDebugger::FRewindDebuggerTrack
{
public:
	explicit FDebuggerTrack(uint64 InObjectId);

private:
	virtual FSlateIcon GetIconInternal() override { return Icon;  }
	virtual TSharedPtr<SWidget> GetTimelineViewInternal() override { return nullptr; }
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
	virtual FName GetNameInternal() const override { return "PoseSearchDebugger"; }
	virtual FText GetDisplayNameInternal() const override { return NSLOCTEXT("PoseSearchDebugger", "PoseSearchDebuggerTabTitle", "Pose Search"); }
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }
	virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;
	virtual bool UpdateInternal() override;

	uint64 ObjectId;
	FSlateIcon Icon;

	TArray<TSharedPtr<FSearchTrack>> SearchTracks;
};


} // namespace UE::PoseSearch
