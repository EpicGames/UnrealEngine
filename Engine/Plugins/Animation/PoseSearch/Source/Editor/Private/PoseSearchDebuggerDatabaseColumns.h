// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/DebugSkelMeshComponent.h"
#include "AnimPreviewInstance.h"
#include "Editor.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "IAnimationEditor.h"
#include "IMultiAnimAssetEditor.h"
#include "IPersonaToolkit.h"
#include "PoseSearchDatabaseEditor.h"
#include "PoseSearchDatabaseViewModel.h"
#include "PoseSearchDebuggerDatabaseRowData.h"
#include "PoseSearchDebuggerViewModel.h"
#include "Preferences/PersonaOptions.h"
#include "Styling/AppStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "PoseSearchDebugger"

namespace UE::PoseSearch::DebuggerDatabaseColumns
{
constexpr float ColumnWidthPadding = 10.f;
constexpr float ColumnSortWidthPadding = 20.f;
	
/** Column struct to represent each column in the debugger database */
struct IColumn : TSharedFromThis<IColumn>
{
	explicit IColumn(int32 InSortIndex, bool InEnabled = true)
		: SortIndex(InSortIndex)
		, bEnabled(InEnabled)
	{
		ColumnId = FName(FString::Printf(TEXT("Column %d"), SortIndex));
	}

	virtual ~IColumn() = default;

	FName ColumnId;

	/** Sorted left to right based on this index */
	int32 SortIndex = 0;
	/** Current width, starts at 1 to be evenly spaced between all columns */
	float Width = 1.0f;
	/** Disabled selectively with view options */
	bool bEnabled = false;

	virtual FText GetLabel() const = 0;
	virtual FText GetLabelTooltip() const { return GetLabel(); }
	
	using FRowDataRef = TSharedRef<FDebuggerDatabaseRowData>;
	using FSortPredicate = TFunction<bool(const FRowDataRef&, const FRowDataRef&)>;

	virtual FSortPredicate GetSortPredicate() const = 0;

	virtual TSharedRef<SWidget> GenerateWidget(const FRowDataRef& RowData) const = 0;

	virtual float ComputeColumnWidth(TConstArrayView<FRowDataRef> Rows) const = 0;
};

/** Column struct to represent each column in the debugger database */
struct ITextColumn : IColumn
{
	explicit ITextColumn(int32 InSortIndex, bool InEnabled = true)
		: IColumn(InSortIndex, InEnabled)
	{
	}

	virtual ~ITextColumn() = default;
		
	virtual TSharedRef<SWidget> GenerateWidget(const FRowDataRef& RowData) const override
	{
		static FSlateFontInfo RowFont = FAppStyle::Get().GetFontStyle("DetailsView.CategoryTextStyle");
			
		return SNew(STextBlock)
            .Font(RowFont)
			.Text_Lambda([this, RowData]() -> FText { return GetRowText(RowData); })
			.ToolTipText_Lambda([this, RowData]() -> FText { return GetRowToolTipText(RowData); })
            .Justification(ETextJustify::Center)
			.ColorAndOpacity_Lambda([this, RowData] { return GetColorAndOpacity(RowData); });
	}

protected:
	virtual FText GetRowText(const FRowDataRef& Row) const = 0;
	virtual FText GetRowToolTipText(const FRowDataRef& Row) const
	{
		return FText::GetEmpty();
	}
	virtual FSlateColor GetColorAndOpacity(const FRowDataRef& Row) const
	{
		return FSlateColor(FLinearColor::White);
	}
	
	virtual float ComputeColumnWidth(TConstArrayView<TSharedRef<FDebuggerDatabaseRowData>> Rows) const override
	{
		const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const FSlateFontInfo & CellTextStyle = FAppStyle::GetFontStyle("SmallText");

		float MinWidth = FontMeasure->Measure(GetLabel(), CellTextStyle).X + ColumnWidthPadding;
		
		for (const TSharedRef<FDebuggerDatabaseRowData>& Row : Rows)
		{
			MinWidth = FMath::Max((FontMeasure->Measure(GetRowText(Row), CellTextStyle).X + ColumnWidthPadding), MinWidth);
		}
		
		return MinWidth;
	}
};

struct FPoseIdx : ITextColumn
{
	using ITextColumn::ITextColumn;

	virtual FText GetLabel() const override
	{
		static const FText Label = LOCTEXT("ColumnLabelPoseIndex", "Pose Id");
		return Label;
	}

	virtual FText GetLabelTooltip() const override
	{
		static const FText LabelTooltip = LOCTEXT("ColumnLabelTooltipPoseIndex", "Index of the Pose in the Database");
		return LabelTooltip;
	}

	virtual FSortPredicate GetSortPredicate() const override
	{
		return [](const FRowDataRef& Row0, const FRowDataRef& Row1) -> bool
			{
				return Row0->PoseIdx < Row1->PoseIdx;
			};
	}

	virtual FText GetRowText(const FRowDataRef& Row) const override
	{
		if (Row->IsValid())
		{
			return FText::AsNumber(Row->PoseIdx, &FNumberFormattingOptions::DefaultNoGrouping());
		}
		return FText::FromString(TEXT("-"));
	}
};

struct FAssetIdx : ITextColumn
{
	using ITextColumn::ITextColumn;

	virtual FText GetLabel() const override
	{
		static const FText Label = LOCTEXT("ColumnLabelAssetIndex", "Asset Id");
		return Label;
	}

	virtual FText GetLabelTooltip() const override
	{
		static const FText LabelTooltip = LOCTEXT("ColumnLabelTooltipAssetIndex", "Index of the Asset in the Database");
		return LabelTooltip;
	}

	virtual FSortPredicate GetSortPredicate() const override
	{
		return [](const FRowDataRef& Row0, const FRowDataRef& Row1) -> bool
			{
				return Row0->DbAssetIdx < Row1->DbAssetIdx;
			};
	}

	virtual FText GetRowText(const FRowDataRef& Row) const override
	{
		if (Row->IsValid())
		{
			return FText::AsNumber(Row->DbAssetIdx, &FNumberFormattingOptions::DefaultNoGrouping());
		}
		return FText::FromString(TEXT("-"));
	}
};


struct FDatabaseName : IColumn
{
	TSharedPtr<FDebuggerViewModel> DebuggerViewModel;

	FDatabaseName(int32 InSortIndex, TSharedPtr<FDebuggerViewModel> InDebuggerViewModel)
	: IColumn(InSortIndex)
	, DebuggerViewModel(InDebuggerViewModel)
	{
	}

	virtual FText GetLabel() const override
	{
		static const FText Label = LOCTEXT("ColumnLabelDatabaseName", "Database");
		return Label;
	}

	virtual FText GetLabelTooltip() const override
	{
		static const FText LabelTooltip = LOCTEXT("ColumnLabelTooltipDatabaseName", "Database Name");
		return LabelTooltip;
	}

	virtual FSortPredicate GetSortPredicate() const override
	{
		return [](const FRowDataRef& Row0, const FRowDataRef& Row1) -> bool
			{
				return GetNameSafe(Row0->SharedData->SourceDatabase.Get()) < GetNameSafe(Row1->SharedData->SourceDatabase.Get());
			};
	}

	virtual TSharedRef<SWidget> GenerateWidget(const FRowDataRef& RowData) const override
	{
		if (RowData->IsValid())
		{
			return SNew(SHyperlink)
				.Text_Lambda([RowData]() -> FText
					{
						return FText::FromString(GetNameSafe(RowData->SharedData->SourceDatabase.Get()));
					})
				.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallText"))
				.ToolTipText_Lambda([RowData]() -> FText
					{
						return FText::Format(
							LOCTEXT("DatabaseHyperlinkTooltipFormat", "Open database '{0}'"),
							FText::FromString(GetPathNameSafe(RowData->SharedData->SourceDatabase.Get())));
					})
				.OnNavigate_Lambda([this, RowData]()
					{
						// Open editor
						if (UPoseSearchDatabase* Database = RowData->SharedData->SourceDatabase.Get())
						{
							if (UAssetEditorSubsystem* AssetEditorSS = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
							{
								AssetEditorSS->OpenEditorForAsset(Database);

								if (IAssetEditorInstance* Editor = AssetEditorSS->FindEditorForAsset(Database, true))
								{
									if (Editor->GetEditorName() == FName("PoseSearchDatabaseEditor"))
									{
										FDatabaseEditor* DatabaseEditor = static_cast<FDatabaseEditor*>(Editor);

										// Open asset paused and at specific time as seen on the pose search debugger.
										DatabaseEditor->SetSelectedPoseIdx(RowData->PoseIdx, DebuggerViewModel->GetDrawQuery(), RowData->SharedData->QueryVector);
									}
								}
							}
						}
					});
		}

		return SNew(STextBlock)
            .Font(FAppStyle::Get().GetFontStyle("DetailsView.CategoryTextStyle"))
			.Text(FText::FromString(TEXT("-")))
            .Justification(ETextJustify::Center)
			.ColorAndOpacity(FSlateColor(FLinearColor::White));
	}

	virtual float ComputeColumnWidth(TConstArrayView<FRowDataRef> Rows) const override
	{
		const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const FSlateFontInfo & CellTextStyle = FAppStyle::GetFontStyle("SmallText");

		float MinWidth = FontMeasure->Measure(GetLabel(), CellTextStyle).X + ColumnWidthPadding;
		
		for (const TSharedRef<FDebuggerDatabaseRowData>& Row : Rows)
		{
			MinWidth = FMath::Max(FontMeasure->Measure(GetNameSafe(Row->SharedData->SourceDatabase.Get()), CellTextStyle).X + ColumnWidthPadding, MinWidth);
		}
		
		return MinWidth;
	}
};

struct FAssetName : IColumn
{
	using IColumn::IColumn;

	virtual FText GetLabel() const override
	{
		static const FText Label = LOCTEXT("ColumnLabelAssetName", "Asset");
		return Label;
	}
	
	virtual FText GetLabelTooltip() const override
	{
		static const FText LabelTooltip = LOCTEXT("ColumnLabelTooltipAssetName", "Animation Asset Name");
		return LabelTooltip;
	}

	virtual FSortPredicate GetSortPredicate() const override
	{
		return [](const FRowDataRef& Row0, const FRowDataRef& Row1) -> bool
			{
				return Row0->AssetName < Row1->AssetName;
			};
	}

	virtual TSharedRef<SWidget> GenerateWidget(const FRowDataRef& RowData) const override
	{
		if (RowData->IsValid())
		{
			return SNew(SHyperlink)
				.Text_Lambda([RowData]() -> FText { return FText::FromString(RowData->AssetName); })
				.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallText"))
				.ToolTipText_Lambda([RowData]() -> FText
					{
						return FText::Format(
							LOCTEXT("AssetHyperlinkTooltipFormat", "Open asset '{0}'"),
							FText::FromString(RowData->AssetPath));
					})
				.OnNavigate_Lambda([RowData]()
					{
						UObject* Asset = nullptr;

						// Load asset
						if (UPackage* Package = LoadPackage(NULL, *RowData->AssetPath, LOAD_NoRedirects))
						{
							Package->FullyLoad();

							const FString AssetName = FPaths::GetBaseFilename(RowData->AssetPath);
							Asset = FindObject<UObject>(Package, *AssetName);
						}
						else
						{
							// Fallback for unsaved assets
							Asset = FindObject<UObject>(nullptr, *RowData->AssetPath);
						}

						// Open editor
						if (Asset != nullptr)
						{
							if (UAssetEditorSubsystem* AssetEditorSS = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
							{
								AssetEditorSS->OpenEditorForAsset(Asset);

								if (IAssetEditorInstance* Editor = AssetEditorSS->FindEditorForAsset(Asset, true))
								{
									if (Editor->GetEditorName() == "AnimationEditor")
									{
										const IAnimationEditor* AnimationEditor = static_cast<IAnimationEditor*>(Editor);
										const UDebugSkelMeshComponent* PreviewComponent = AnimationEditor->GetPersonaToolkit()->GetPreviewMeshComponent();

										// Open asset paused and at specific time as seen on the pose search debugger.
										PreviewComponent->PreviewInstance->SetPosition(RowData->AssetTime);
										PreviewComponent->PreviewInstance->SetPlaying(false);
										PreviewComponent->PreviewInstance->SetBlendSpacePosition(RowData->BlendParameters);
									}
									else if (Editor->GetEditorName() == "PoseSearchInteractionAssetEditor")
									{
										IMultiAnimAssetEditor* MultiAnimAssetEditor = static_cast<IMultiAnimAssetEditor*>(Editor);

										// Open asset paused and at specific time as seen on the pose search debugger.
										MultiAnimAssetEditor->SetPreviewProperties(RowData->AssetTime, RowData->BlendParameters, false);
									}
								}
							}
						}
					});
		}

		return SNew(STextBlock)
            .Font(FAppStyle::Get().GetFontStyle("DetailsView.CategoryTextStyle"))
			.Text(LOCTEXT("ColumnAssetNameInvalidPose", "Invalid Pose"))
            .Justification(ETextJustify::Center)
			.ColorAndOpacity(FSlateColor(FLinearColor::White));
	}

	virtual float ComputeColumnWidth(TConstArrayView<FRowDataRef> Rows) const override
	{
		const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const FSlateFontInfo & CellTextStyle = FAppStyle::GetFontStyle("SmallText");

		float MinWidth = FontMeasure->Measure(GetLabel(), CellTextStyle).X + ColumnWidthPadding;
		
		for (const TSharedRef<FDebuggerDatabaseRowData>& Row : Rows)
		{
			MinWidth = FMath::Max(FontMeasure->Measure(Row->AssetName, CellTextStyle).X, MinWidth);
		}
		
		return MinWidth;
	}
};

struct FFrame : ITextColumn
{
	using ITextColumn::ITextColumn;

	virtual FText GetLabel() const override
	{
		static const FText Label = LOCTEXT("ColumnLabelFrame", "Frame");
		return Label;
	}
	
	virtual FText GetLabelTooltip() const override
	{
		static const FText LabelTooltip = LOCTEXT("ColumnLabelTooltipFrame", "Frame number from the start of the Animation Asset");
		return LabelTooltip;
	}

	virtual FSortPredicate GetSortPredicate() const override
	{
		return [](const FRowDataRef& Row0, const FRowDataRef& Row1) -> bool
			{
				return Row0->AnimFrame < Row1->AnimFrame;
			};
	}

	virtual FText GetRowText(const FRowDataRef& Row) const override
	{
		if (Row->IsValid())
		{
			return FText::AsNumber(Row->AnimFrame, &FNumberFormattingOptions::DefaultNoGrouping());
		}
		return FText::FromString(TEXT("-"));
	}
};

struct FTime : ITextColumn
{
	using ITextColumn::ITextColumn;

	virtual FText GetLabel() const override
	{
		static const FText Label = LOCTEXT("ColumnLabelTime", "Time");
		return Label;
	}

	virtual FText GetLabelTooltip() const override
	{
		static const FText LabelTooltip = LOCTEXT("ColumnLabelTooltipTime", "Time in seconds from the start of the Animation Asset");
		return LabelTooltip;
	}

	virtual FSortPredicate GetSortPredicate() const override
	{
		return [](const FRowDataRef& Row0, const FRowDataRef& Row1) -> bool
			{
				return Row0->AssetTime < Row1->AssetTime;
			};
	}

	virtual FText GetRowText(const FRowDataRef& Row) const override
	{
		if (Row->IsValid())
		{
			return FText::AsNumber(Row->AssetTime, &FNumberFormattingOptions().SetUseGrouping(false).SetMaximumFractionalDigits(2));
		}
		return FText::FromString(TEXT("-"));
	}
};

struct FPercentage : ITextColumn
{
	using ITextColumn::ITextColumn;

	virtual FText GetLabel() const override
	{
		static const FText Label = LOCTEXT("ColumnLabelPercentage", "Percent");
		return Label;
	}

	virtual FText GetLabelTooltip() const override
	{
		static const FText LabelTooltip = LOCTEXT("ColumnLabelTooltipPercentage", "Time in percentage from the start of the Animation Asset");
		return LabelTooltip;
	}

	virtual FSortPredicate GetSortPredicate() const override
	{
		return [](const FRowDataRef& Row0, const FRowDataRef& Row1) -> bool
			{
				return Row0->AnimPercentage < Row1->AnimPercentage;
			};
	}

	virtual FText GetRowText(const FRowDataRef& Row) const override
	{
		if (Row->IsValid())
		{
			return FText::AsPercent(Row->AnimPercentage, &FNumberFormattingOptions().SetMaximumFractionalDigits(2));
		}
		return FText::FromString(TEXT("-"));
	}
};

struct FCost : ITextColumn
{
	using ITextColumn::ITextColumn;

	virtual FText GetLabel() const override
	{
		static const FText Label = LOCTEXT("ColumnLabelCost", "Cost");
		return Label;
	}
	
	virtual FText GetLabelTooltip() const override
	{
		static const FText LabelTooltip = LOCTEXT("ColumnLabelTooltipCost", "Total Cost of the associated Pose");
		return LabelTooltip;
	}

	virtual FSortPredicate GetSortPredicate() const override
	{
		return [](const FRowDataRef& Row0, const FRowDataRef& Row1) -> bool
			{
				return Row0->PoseCost < Row1->PoseCost;
			};
	}
		
	virtual FText GetRowText(const FRowDataRef& Row) const override
	{
		if (Row->IsValid() && Row->PoseCost.IsValid())
		{
			return FText::AsNumber(Row->PoseCost);
		}
		return FText::FromString(TEXT("-"));
    }

	virtual FSlateColor GetColorAndOpacity(const FRowDataRef& Row) const override
	{
		return Row->CostColor;
	}
};

struct FPCACost : ITextColumn
{
	using ITextColumn::ITextColumn;

	virtual FText GetLabel() const override
	{
		static const FText Label = LOCTEXT("ColumnLabelPCACost", "PCA Cost");
		return Label;
	}
	
	virtual FText GetLabelTooltip() const override
	{
		static const FText LabelTooltip = LOCTEXT("ColumnLabelTooltipPCACost", "Total PCA Cost of the associated Pose");
		return LabelTooltip;
	}

	virtual FSortPredicate GetSortPredicate() const override
	{
		return [](const FRowDataRef& Row0, const FRowDataRef& Row1) -> bool
			{
				return Row0->PosePCACost < Row1->PosePCACost;
			};
	}
		
	virtual FText GetRowText(const FRowDataRef& Row) const override
	{
		if (Row->IsValid())
		{
			return FText::AsNumber(Row->PosePCACost);
		}
		return FText::FromString(TEXT("-"));
    }

	virtual FSlateColor GetColorAndOpacity(const FRowDataRef& Row) const override
	{
		return Row->PCACostColor;
	}
};

struct FChannelBreakdownCostColumn : ITextColumn
{
	FChannelBreakdownCostColumn(int32 SortIndex, int32 InBreakdownCostIndex, const FText& InLabel)
		: ITextColumn(SortIndex)
		, Label(InLabel)
		, LabelTooltip(FText::Format(LOCTEXT("ColumnLabelTooltipChannelBreakdownCost", "Breakdown Cost for the Channel '{0}'"), InLabel))
		, BreakdownCostIndex(InBreakdownCostIndex)
	{
	}

	virtual FText GetLabel() const override
	{
		return Label;
	}

	virtual FText GetLabelTooltip() const override
	{
		return LabelTooltip;
	}

	virtual FSortPredicate GetSortPredicate() const override
	{
		return [this](const FRowDataRef& Row0, const FRowDataRef& Row1) -> bool
			{
				if (!Row0->IsValid())
				{
					return true;
				}
				if (!Row1->IsValid())
				{
					return false;
				}
				return Row0->CostBreakdowns[BreakdownCostIndex] < Row1->CostBreakdowns[BreakdownCostIndex];
			};
	}

	virtual FText GetRowText(const FRowDataRef& Row) const override
	{
		if (Row->IsValid())
		{
			const float LabelCost = Row->CostBreakdowns[BreakdownCostIndex];
			if (LabelCost != UE_MAX_FLT)
			{
				return FText::AsNumber(LabelCost);
			}
		}
		return FText::FromString(TEXT("-"));
	}

	virtual FSlateColor GetColorAndOpacity(const FRowDataRef& Row) const override
	{
		if (Row->CostBreakdownsColors.IsValidIndex(BreakdownCostIndex))
		{
			return Row->CostBreakdownsColors[BreakdownCostIndex];
		}
		return FLinearColor::White;
	}

	FText Label;
	FText LabelTooltip;
	int32 BreakdownCostIndex = INDEX_NONE;
};

struct FCostModifier : ITextColumn
{
	using ITextColumn::ITextColumn;

	virtual FText GetLabel() const override
	{
		static const FText Label = LOCTEXT("ColumnLabelCostModifier", "Bias");
		return Label;
	}

	virtual FText GetLabelTooltip() const override
	{
		static const FText LabelTooltip = LOCTEXT("ColumnLabelTooltipCostModifier", "Total Cost for all the Bias contributions");
		return LabelTooltip;
	}

	virtual FSortPredicate GetSortPredicate() const override
	{
		return [](const FRowDataRef& Row0, const FRowDataRef& Row1) -> bool
			{
				return Row0->PoseCost.GetCostAddend() < Row1->PoseCost.GetCostAddend();
			};
	}

	virtual FText GetRowText(const FRowDataRef& Row) const override
	{
		if (Row->IsValid())
		{
			return FText::AsNumber(Row->PoseCost.GetCostAddend());
		}
		return FText::FromString(TEXT("-"));
	}
};

struct FContinuingPoseCost : ITextColumn
{
	using ITextColumn::ITextColumn;

	virtual FText GetLabel() const override
	{
		static const FText Label = LOCTEXT("ColumnLabelContinuingPoseCost", "ContinuingPoseCost");
		return Label;
	}

	virtual FText GetLabelTooltip() const override
	{
		static const FText LabelTooltip = LOCTEXT("ColumnLabelTooltipContinuingPoseCost", "Continuing Pose Cost Bias contribution");
		return LabelTooltip;
	}

	virtual FSortPredicate GetSortPredicate() const override
	{
		return [](const FRowDataRef& Row0, const FRowDataRef& Row1) -> bool
			{
				return Row0->PoseCost.GetContinuingPoseCostAddend() < Row1->PoseCost.GetContinuingPoseCostAddend();
			};
	}

	virtual FText GetRowText(const FRowDataRef& Row) const override
	{
		if (Row->IsValid())
		{
			return FText::AsNumber(Row->PoseCost.GetContinuingPoseCostAddend());
		}
		return FText::FromString(TEXT("-"));
	}
};

struct FContinuingInteractionCost : ITextColumn
{
	using ITextColumn::ITextColumn;

	virtual FText GetLabel() const override
	{
		static const FText Label = LOCTEXT("ColumnLabelContinuingInteractionCost", "ContinuingInteractionCost");
		return Label;
	}

	virtual FText GetLabelTooltip() const override
	{
		static const FText LabelTooltip = LOCTEXT("ColumnLabelTooltipContinuingInteractionCost", "Continuing Interaction Cost Bias contribution (interaction with the same anim contexts with the same assigned roles)");
		return LabelTooltip;
	}

	virtual FSortPredicate GetSortPredicate() const override
	{
		return [](const FRowDataRef& Row0, const FRowDataRef& Row1) -> bool
			{
				return Row0->PoseCost.GetContinuingInteractionCostAddend() < Row1->PoseCost.GetContinuingInteractionCostAddend();
			};
	}

	virtual FText GetRowText(const FRowDataRef& Row) const override
	{
		if (Row->IsValid())
		{
			return FText::AsNumber(Row->PoseCost.GetContinuingInteractionCostAddend());
		}
		return FText::FromString(TEXT("-"));
	}
};

struct FContinuingContextInteractionCost : ITextColumn
{
	using ITextColumn::ITextColumn;

	virtual FText GetLabel() const override
	{
		static const FText Label = LOCTEXT("ColumnLabelContinuingContextInteractionCost", "ContinuingContextInteractionCost");
		return Label;
	}

	virtual FText GetLabelTooltip() const override
	{
		static const FText LabelTooltip = LOCTEXT("ColumnLabelTooltipContinuingContextInteractionCost", "Continuing Context Interaction Cost Bias contribution (interaction with the same anim contexts, assigned roles may differ)");
		return LabelTooltip;
	}

	virtual FSortPredicate GetSortPredicate() const override
	{
		return [](const FRowDataRef& Row0, const FRowDataRef& Row1) -> bool
			{
				return Row0->PoseCost.GetContinuingContextInteractionCostAddend() < Row1->PoseCost.GetContinuingContextInteractionCostAddend();
			};
	}

	virtual FText GetRowText(const FRowDataRef& Row) const override
	{
		if (Row->IsValid())
		{
			return FText::AsNumber(Row->PoseCost.GetContinuingContextInteractionCostAddend());
		}
		return FText::FromString(TEXT("-"));
	}
};

struct FNotifyCost : ITextColumn
{
	using ITextColumn::ITextColumn;

	virtual FText GetLabel() const override
	{
		static const FText Label = LOCTEXT("ColumnLabelNotifyCost", "NotifyCost");
		return Label;
	}

	virtual FText GetLabelTooltip() const override
	{
		static const FText LabelTooltip = LOCTEXT("ColumnLabelTooltipNotifyCost", "Notify Cost Bias contribution");
		return LabelTooltip;
	}

	virtual FSortPredicate GetSortPredicate() const override
	{
		return [](const FRowDataRef& Row0, const FRowDataRef& Row1) -> bool
			{
				return Row0->PoseCost.GetNotifyCostAddend() < Row1->PoseCost.GetNotifyCostAddend();
			};
	}

	virtual FText GetRowText(const FRowDataRef& Row) const override
	{
		if (Row->IsValid())
		{
			return FText::AsNumber(Row->PoseCost.GetNotifyCostAddend());
		}
		return FText::FromString(TEXT("-"));
	}
};


struct FMirrored : ITextColumn
{
	using ITextColumn::ITextColumn;

	virtual FText GetLabel() const override
	{
		static const FText Label = LOCTEXT("ColumnLabelMirrored", "Mirror");
		return Label;
	}

	virtual FText GetLabelTooltip() const override
	{
		static const FText LabelTooltip = LOCTEXT("ColumnLabelTooltipMirrored", "Mirror state of the associated Pose");
		return LabelTooltip;
	}

	virtual FSortPredicate GetSortPredicate() const override
	{
		return [](const FRowDataRef& Row0, const FRowDataRef& Row1) -> bool
			{
				return Row0->bMirrored < Row1->bMirrored;
			};
	}

	virtual FText GetRowText(const FRowDataRef& Row) const override
	{
		static const FText RowTextMirrored = FText::Format(LOCTEXT("ColumnMirrored_Mirrored", "{0}"), { true } );
		static const FText RowTextUnmirrored = FText::Format(LOCTEXT("ColumnMirrored_Mirrored", "{0}"), { false } );
		static const FText RowTextUnknown = LOCTEXT("ColumnMirrored_Unknown", "-");

		if (!Row->IsValid())
		{
			return RowTextUnknown;
		}

		if (Row->bMirrored)
		{
			return RowTextMirrored;
		}
			
		return RowTextUnmirrored;
	}
};

struct FLooping : ITextColumn
{
	using ITextColumn::ITextColumn;

	virtual FText GetLabel() const override
	{
		static const FText Label = LOCTEXT("ColumnLabelLooping", "Loop");
		return Label;
	}

	virtual FText GetLabelTooltip() const override
	{
		static const FText LabelTooltip = LOCTEXT("ColumnLabelTooltipLooping", "Loop state of the associated Pose");
		return LabelTooltip;
	}

	virtual FSortPredicate GetSortPredicate() const override
	{
		return [](const FRowDataRef& Row0, const FRowDataRef& Row1) -> bool
			{
				return Row0->bLooping < Row1->bLooping;
			};
	}

	virtual FText GetRowText(const FRowDataRef& Row) const override
	{
		static const FText RowTextLooping = FText::Format(LOCTEXT("ColumnLooping_Looping", "{0}"), { true } );
		static const FText RowTextNotLooping = FText::Format(LOCTEXT("ColumnLooping_NotLooping", "{0}"), { false } );
		static const FText RowTextUnknown = LOCTEXT("ColumnLooping_Unknown", "-");

		if (!Row->IsValid())
		{
			return RowTextUnknown;
		}

		if (Row->bLooping)
		{
			return RowTextLooping;
		}
			
		return RowTextNotLooping;
	}
};

struct FBlendParameters : ITextColumn
{
	using ITextColumn::ITextColumn;

	virtual FText GetLabel() const override
	{
		static const FText Label = LOCTEXT("ColumnLabelBlendParams", "Blend Params");
		return Label;
	}

	virtual FText GetLabelTooltip() const override
	{
		static const FText LabelTooltip = LOCTEXT("ColumnLabelBlendTooltipParams", "Blend Params used to sample the associated BlendSpace asset");
		return LabelTooltip;
	}

	virtual FSortPredicate GetSortPredicate() const override
	{
		return [](const FRowDataRef& Row0, const FRowDataRef& Row1) -> bool
		{
			return (
				Row0->BlendParameters[0] < Row1->BlendParameters[0] ||
				Row0->BlendParameters[1] < Row1->BlendParameters[1]);
		};
	}

	virtual FText GetRowText(const FRowDataRef& Row) const override
	{
		if (Row->IsValid())
		{
			return FText::Format(LOCTEXT("Blend Parameters", "{0}, {1}"), FText::AsNumber(Row->BlendParameters[0]), FText::AsNumber(Row->BlendParameters[1]));
		}
		return FText::FromString(TEXT("-"));
	}
};

struct FPoseCandidateFlags : ITextColumn
{
	using ITextColumn::ITextColumn;

	virtual FText GetLabel() const override
	{
		static const FText Label = LOCTEXT("ColumnLabelPoseCandidateFlags", "Flags");
		return Label;
	}

	virtual FText GetLabelTooltip() const override
	{
		static const FText LabelTooltip = LOCTEXT("ColumnLabelTooltipPoseCandidateFlags", "Flags indicating why a Pose has been discarded");
		return LabelTooltip;
	}

	virtual FSortPredicate GetSortPredicate() const override
	{
		return [](const FRowDataRef& Row0, const FRowDataRef& Row1) -> bool
			{
				return Row0->PoseCandidateFlags < Row1->PoseCandidateFlags;
			};
	}

	virtual FText GetRowText(const FRowDataRef& Row) const override
	{
		if (Row->IsValid() && EnumHasAnyFlags(Row->PoseCandidateFlags, EPoseCandidateFlags::AnyDiscardedMask))
		{
			FString Sring;

			if (EnumHasAnyFlags(Row->PoseCandidateFlags, EPoseCandidateFlags::DiscardedBy_PoseJumpThresholdTime))
			{
				Sring.Append("J ");
			}

			if (EnumHasAnyFlags(Row->PoseCandidateFlags, EPoseCandidateFlags::DiscardedBy_PoseReselectHistory))
			{
				Sring.Append("H ");
			}
			
			if (EnumHasAnyFlags(Row->PoseCandidateFlags, EPoseCandidateFlags::DiscardedBy_BlockTransition))
			{
				Sring.Append("B ");
			}
			
			if (EnumHasAnyFlags(Row->PoseCandidateFlags, EPoseCandidateFlags::DiscardedBy_PoseFilter))
			{
				Sring.Append("F ");
			}

			if (EnumHasAnyFlags(Row->PoseCandidateFlags, EPoseCandidateFlags::DiscardedBy_AssetIdxFilter))
			{
				Sring.Append("A ");
			}
			
			if (EnumHasAnyFlags(Row->PoseCandidateFlags, EPoseCandidateFlags::DiscardedBy_Search))
			{
				Sring.Append("S ");
			}

			if (EnumHasAnyFlags(Row->PoseCandidateFlags, EPoseCandidateFlags::DiscardedBy_AssetReselection))
			{
				Sring.Append("R ");
			}

			return FText::FromString(Sring);
		}

		return FText::GetEmpty();
	}

	virtual FText GetRowToolTipText(const FRowDataRef& Row) const override
	{
		if (EnumHasAnyFlags(Row->PoseCandidateFlags, EPoseCandidateFlags::AnyDiscardedMask))
		{
			FTextBuilder TextBuilder;
			TextBuilder.AppendLine(LOCTEXT("DiscardedBy_Reason_Tooltip", "Pose discarded because of:"));

			if (EnumHasAnyFlags(Row->PoseCandidateFlags, EPoseCandidateFlags::DiscardedBy_PoseJumpThresholdTime))
			{
				TextBuilder.AppendLine(LOCTEXT("DiscardedBy_PoseJumpThresholdTime_Tooltip", "(J) Pose Jump Threshold Time"));
			}

			if (EnumHasAnyFlags(Row->PoseCandidateFlags, EPoseCandidateFlags::DiscardedBy_PoseReselectHistory))
			{
				TextBuilder.AppendLine(LOCTEXT("DiscardedBy_PoseReselectHistory_Tooltip", "(H) Pose Reselect History"));
			}

			if (EnumHasAnyFlags(Row->PoseCandidateFlags, EPoseCandidateFlags::DiscardedBy_BlockTransition))
			{
				TextBuilder.AppendLine(LOCTEXT("DiscardedBy_BlockTransition_Tooltip", "(B) Block Transition"));
			}

			if (EnumHasAnyFlags(Row->PoseCandidateFlags, EPoseCandidateFlags::DiscardedBy_PoseFilter))
			{
				TextBuilder.AppendLine(LOCTEXT("DiscardedBy_PoseFilter_Tooltip", "(F) Filter"));
			}

			if (EnumHasAnyFlags(Row->PoseCandidateFlags, EPoseCandidateFlags::DiscardedBy_AssetIdxFilter))
			{
				TextBuilder.AppendLine(LOCTEXT("DiscardedBy_AssetIdxFilter_Tooltip", "(A) Asset Idx Filter"));
			}
			
			if (EnumHasAnyFlags(Row->PoseCandidateFlags, EPoseCandidateFlags::DiscardedBy_Search))
			{
				TextBuilder.AppendLine(LOCTEXT("DiscardedBy_Search_Tooltip", "(S) Search"));
			}

			if (EnumHasAnyFlags(Row->PoseCandidateFlags, EPoseCandidateFlags::DiscardedBy_AssetReselection))
			{
				TextBuilder.AppendLine(LOCTEXT("DiscardedBy_AssetReselection_Tooltip", "(R) Disable Reselection"));
			}

			return TextBuilder.ToText();
		}

		return FText::GetEmpty();
	}
};

} // namespace UE::PoseSearch::DebuggerDatabaseColumns

#undef LOCTEXT_NAMESPACE
