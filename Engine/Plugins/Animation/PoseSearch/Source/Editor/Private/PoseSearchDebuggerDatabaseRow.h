// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ListView.h"
#include "PoseSearchDebuggerDatabaseColumns.h"
#include "PoseSearchDebuggerDatabaseView.h"
#include "Widgets/Views/STableRow.h"

namespace UE::PoseSearch
{

/**
 * Widget representing a single row of the database view
 */
class SDebuggerDatabaseRow : public SMultiColumnTableRow<TSharedRef<FDebuggerDatabaseRowData>>
{
	SLATE_BEGIN_ARGS(SDebuggerDatabaseRow) {}
		SLATE_ATTRIBUTE(const SDebuggerDatabaseView::FColumnMap*, ColumnMap)
	SLATE_END_ARGS()

	void Construct(
		const FArguments& InArgs,
		const TSharedRef<STableViewBase>& InOwnerTable,
		TSharedRef<FDebuggerDatabaseRowData> InRow,
		const FTableRowStyle& InRowStyle,
		const FSlateBrush* InRowBrush,
		FMargin InPaddingMargin
	)
	{
		ColumnMap = InArgs._ColumnMap;
		check(ColumnMap.IsBound());
		
		Row = InRow;
		
		RowBrush = InRowBrush;
		check(RowBrush);

		SMultiColumnTableRow<TSharedRef<FDebuggerDatabaseRowData>>::Construct(
			FSuperRowType::FArguments()
			.Padding(InPaddingMargin)
			.Style(&InRowStyle),
			InOwnerTable
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		// Get column
		const TSharedRef<DebuggerDatabaseColumns::IColumn>& Column = (*ColumnMap.Get())[InColumnName];
		
		const TSharedRef<SWidget> Widget = Column->GenerateWidget(Row.ToSharedRef());
		
		return
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.BorderImage(RowBrush)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Padding(0.0f, 3.0f)
				[
					Widget
				]
			];
	}

	/** Row data associated with this widget */
	TSharedPtr<FDebuggerDatabaseRowData> Row;

	/** Used for cell styles (active vs database row) */
	const FSlateBrush* RowBrush = nullptr;

	/** Used to grab the column struct given a column name */
	TAttribute<const SDebuggerDatabaseView::FColumnMap*> ColumnMap;
};

} // namespace UE::PoseSearch

