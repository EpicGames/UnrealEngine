// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchIndex.h"
#include "Templates/SharedPointer.h"

namespace UE::PoseSearch
{

class FDebuggerDatabaseSharedData : public TSharedFromThis<FDebuggerDatabaseSharedData>
{
public:
	TWeakObjectPtr<UPoseSearchDatabase> SourceDatabase;
	
	UE_DEPRECATED(5.7, "Use GetNameSafe(SourceDatabase.Get()) instead")
	FString DatabaseName = "";
	UE_DEPRECATED(5.7, "Use GetPathNameSafe(SourceDatabase.Get()) instead")
	FString DatabasePath = "";
	
	TAlignedArray<float> QueryVector;
	TAlignedArray<float> PCAQueryVector;
};

class FDebuggerDatabaseRowData : public TSharedFromThis<FDebuggerDatabaseRowData>
{
public:
	explicit FDebuggerDatabaseRowData(TSharedRef<FDebuggerDatabaseSharedData> InSharedData) : SharedData(InSharedData) {}
	bool IsValid() const { return PoseIdx != INDEX_NONE; }

	int32 PoseIdx = INDEX_NONE;
	EPoseCandidateFlags PoseCandidateFlags = EPoseCandidateFlags::None;
	FString AssetName = "";
	FString AssetPath = "";
	int32 DbAssetIdx = 0;
	int32 AnimFrame = 0;
	float AnimPercentage = 0.0f;
	float AssetTime = 0.0f;
	bool bMirrored = false;
	bool bLooping = false;
	FVector BlendParameters = FVector::Zero();
	FPoseSearchCost PoseCost;
	FLinearColor CostColor = FLinearColor::White;
	float PosePCACost = 0.f;
	FLinearColor PCACostColor = FLinearColor::White;
	TArray<float> CostBreakdowns;
	TArray<FLinearColor> CostBreakdownsColors;
	TArray<float> CostVector;

	TSharedRef<FDebuggerDatabaseSharedData> SharedData;
};

} // namespace UE::PoseSearch
