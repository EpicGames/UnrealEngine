// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDebuggerDatabaseView.h"
#include "Algo/AllOf.h"
#include "Animation/AnimationSettings.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequence.h"
#include "Common/ProviderLock.h"
#include "IGameplayProvider.h"
#include "Internationalization/Regex.h"
#include "IRewindDebugger.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchDerivedData.h"
#include "PoseSearch/PoseSearchFeatureChannel.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearchDebuggerDatabaseRow.h"
#include "PoseSearchDebuggerSettings.h"
#include "PoseSearchDebuggerView.h"
#include "PoseSearchDebuggerViewModel.h"
#include "PoseSearchEditor.h"
#include "StructUtils/InstancedStruct.h"
#include "Trace/PoseSearchTraceProvider.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "PoseSearchDebugger"

namespace UE::PoseSearch
{

UPoseSearchDatabase* ResolveDatabaseFromId(uint64 DatabaseId)
{
	UPoseSearchDatabase* Database = nullptr;
	if (const TraceServices::IAnalysisSession* AnalysisSession = IRewindDebugger::Instance()->GetAnalysisSession())
	{
		if (const IGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<IGameplayProvider>("GameplayProvider"))
		{
			FString PathName;
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
				if (const FObjectInfo* DatabaseInfo = GameplayProvider->FindObjectInfo(DatabaseId))
				{
					PathName = DatabaseInfo->PathName;
				}
			}

			if (!PathName.IsEmpty())
			{
				Database = TSoftObjectPtr<UPoseSearchDatabase>(FSoftObjectPath(PathName)).LoadSynchronous();
			}
		}
	}
	return Database;
}

static FORCEINLINE float ArraySum(TConstArrayView<float> View, int32 StartIndex, int32 Offset)
{
	float Sum = 0.f;
	const int32 EndIndex = StartIndex + Offset;
	for (int32 i = StartIndex; i < EndIndex; ++i)
	{
		Sum += View[i];
	}
	return Sum;
}

class SCostBreakDownData
{
public:
	SCostBreakDownData(const TArray<FTraceMotionMatchingStateDatabaseEntry>& DatabaseEntries, bool bIsVerbose)
	{
		// processing all the DatabaseEntries to collect the LabelToChannels
		TLabelBuilder LabelBuilder;
		FAsyncPoseSearchDatabasesManagementCache DatabasesManagementCache;
		for (const FTraceMotionMatchingStateDatabaseEntry& DbEntry : DatabaseEntries)
		{
			const UPoseSearchDatabase* Database = ResolveDatabaseFromId(DbEntry.DatabaseId);
			if (EAsyncBuildIndexResult::Success == DatabasesManagementCache(Database))
			{
				for (const TObjectPtr<UPoseSearchFeatureChannel>& ChannelPtr : Database->Schema->GetChannels())
				{
					AnalyzeChannelRecursively(LabelBuilder, ChannelPtr.Get(), bIsVerbose);
				}
			}
		}
	}

	void ProcessData(TArray<TSharedRef<FDebuggerDatabaseRowData>>& InOutUnfilteredDatabaseRows) const
	{
		for (TSharedRef<FDebuggerDatabaseRowData>& UnfilteredDatabaseRowRef : InOutUnfilteredDatabaseRows)
		{
			FDebuggerDatabaseRowData& UnfilteredDatabaseRow = UnfilteredDatabaseRowRef.Get();
			UnfilteredDatabaseRow.CostBreakdowns.AddDefaulted(LabelToChannels.Num());

			for (int32 LabelToChannelIndex = 0; LabelToChannelIndex < LabelToChannels.Num(); ++LabelToChannelIndex)
			{
				const FLabelToChannels& LabelToChannel = LabelToChannels[LabelToChannelIndex];

				// there should only be at most one channel per schema with the unique label,
				// but we'll keep this generic allowing multiple channels from the same schema having the same label.
				// the cost will be the sum of all the channels cost
				float CostBreakdown = 0.f;
				for (const UPoseSearchFeatureChannel* Channel : LabelToChannel.Channels)
				{
					// checking if the row is associated to the Channel
					if (UnfilteredDatabaseRow.SharedData->SourceDatabase->Schema == Channel->GetSchema())
					{
						CostBreakdown += ArraySum(UnfilteredDatabaseRow.CostVector, Channel->GetChannelDataOffset(), Channel->GetChannelCardinality());
					}
				}
				UnfilteredDatabaseRow.CostBreakdowns[LabelToChannelIndex] = CostBreakdown;
			}
		}
	}

	bool AreLabelsEqualTo(const TConstArrayView<FText> OtherLabels) const
	{
		if (LabelToChannels.Num() != OtherLabels.Num())
		{
			return false;
		}

		for (int32 i = 0; i < LabelToChannels.Num(); ++i)
		{
			if (!LabelToChannels[i].Label.EqualTo(OtherLabels[i]))
			{
				return false;
			}
		}

		return true;
	}

	const TArray<FText> GetLabels() const
	{
		TArray<FText> Labels;
		for (const FLabelToChannels& LabelToChannel : LabelToChannels)
		{
			Labels.Add(LabelToChannel.Label);
		}
		return Labels;
	}

private:
	void AnalyzeChannelRecursively(TLabelBuilder& LabelBuilder, const UPoseSearchFeatureChannel* Channel, bool bIsVerbose)
	{
		LabelBuilder.Reset();
		const FText Label = FText::FromString(Channel->GetLabel(LabelBuilder, ELabelFormat::Full_Vertical).ToString());

		bool bLabelFound = false;
		for (int32 i = 0; i < LabelToChannels.Num(); ++i)
		{
			if (LabelToChannels[i].Label.EqualTo(Label))
			{
				LabelToChannels[i].Channels.AddUnique(Channel);
				bLabelFound = true;
			}
		}
		if (!bLabelFound)
		{
			LabelToChannels.AddDefaulted();
			LabelToChannels.Last().Label = Label;
			LabelToChannels.Last().Channels.Add(Channel);
		}

		if (bIsVerbose)
		{
			for (const TObjectPtr<UPoseSearchFeatureChannel>& SubChannelPtr : Channel->GetSubChannels())
			{
				if (const UPoseSearchFeatureChannel* SubChannel = SubChannelPtr.Get())
				{
					AnalyzeChannelRecursively(LabelBuilder, SubChannel, bIsVerbose);
				}
			}
		}
	}

	struct FLabelToChannels
	{
		FText Label;
		TArray<const UPoseSearchFeatureChannel*> Channels; // NoTe: channels can be from different schemas
	};
	TArray<FLabelToChannels> LabelToChannels;
};

static void AddUnfilteredDatabaseRow(const UPoseSearchDatabase* Database, TConstArrayView<float> DynamicWeightsSqrt,
	TArray<TSharedRef<FDebuggerDatabaseRowData>>& UnfilteredDatabaseRows, TSharedRef<FDebuggerDatabaseSharedData> SharedData,
	int32 DbPoseIdx, EPoseCandidateFlags PoseCandidateFlags, TConstArrayView<uint32> PoseToPCAValuesVectorIndexes, const FPoseSearchCost& Cost = FPoseSearchCost())
{
	const FSearchIndex& SearchIndex = Database->GetSearchIndex();
	if (const FSearchIndexAsset* SearchIndexAsset = SearchIndex.GetAssetForPoseSafe(DbPoseIdx))
	{
		TSharedRef<FDebuggerDatabaseRowData>& Row = UnfilteredDatabaseRows.Add_GetRef(MakeShared<FDebuggerDatabaseRowData>(SharedData));

		const float Time = Database->GetNormalizedAssetTime(DbPoseIdx);

		Row->PoseIdx = DbPoseIdx;
		Row->PoseCandidateFlags = PoseCandidateFlags;
		Row->DbAssetIdx = SearchIndexAsset->GetSourceAssetIdx();
		Row->AssetTime = Time;
		Row->bMirrored = SearchIndexAsset->IsMirrored();

		Row->CostVector.SetNum(Database->Schema->SchemaCardinality);
		TArray<float> BufferUsedForReconstruction;
		const TConstArrayView<float> PoseValues = SearchIndex.GetPoseValuesSafe(DbPoseIdx, BufferUsedForReconstruction);

		// in case we modify the schema while PIE is paused and displaying the Pose Search Editor, we could end up with a stale State with a SharedData->QueryVector saved with the previous schema
		// so the cardinality of SharedData->QueryVector and PoseValues don't match. In that case we just use PoseValues as query to have all costs set to zero
		const bool bIsQueryVectorValid = SharedData->QueryVector.Num() == PoseValues.Num();
		const TConstArrayView<float> QueryVector = bIsQueryVectorValid ? TConstArrayView<float>(SharedData->QueryVector) : TConstArrayView<float>(PoseValues);

		CompareFeatureVectors(PoseValues, QueryVector, DynamicWeightsSqrt, Row->CostVector);

		if (Cost.IsValid())
		{
			Row->PoseCost = Cost;
		}
		else
		{
			// @todo: gather ContinuingPoseCostAddend, ContinuingInteractionCostAddend, and ContinuingContextInteractionCostAddend properly to reconstruct the real FPoseSearchCost
			const float ContinuingPoseCostAddend = 0.f;
			const float ContinuingInteractionCostAddend = 0.f;
			const float ContinuingContextInteractionCostAddend = 0.f;
			Row->PoseCost = FPoseSearchCost(ArraySum(Row->CostVector, 0, Database->Schema->SchemaCardinality), SearchIndex.PoseMetadata[DbPoseIdx].GetCostAddend(), ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend);
		}

		if (!SharedData->PCAQueryVector.IsEmpty())
		{
			const int32 PCAValuesVectorIdx = PoseToPCAValuesVectorIndexes.IsEmpty() ? DbPoseIdx : PoseToPCAValuesVectorIndexes[DbPoseIdx];
			if (PCAValuesVectorIdx >= 0)
			{
				TConstArrayView<float> PCAPoseValues = SearchIndex.GetPCAPoseValues(PCAValuesVectorIdx);
				if (SharedData->PCAQueryVector.Num() == PCAPoseValues.Num())
				{
					Row->PosePCACost = CompareFeatureVectors(SharedData->PCAQueryVector, PCAPoseValues);
				}
			}
		}

		if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAsset = Database->GetDatabaseAnimationAsset(SearchIndexAsset->GetSourceAssetIdx()))
		{
			const FVector BlendParameters = SearchIndexAsset->GetBlendParameters();
			const float PlayLength = DatabaseAsset->GetPlayLength(BlendParameters);
			const UObject* AnimationAsset = DatabaseAsset->GetAnimationAsset();

			Row->AssetName = DatabaseAsset->GetName();
			Row->AssetPath = AnimationAsset ? AnimationAsset->GetPathName() : "";
			Row->bLooping = DatabaseAsset->IsLooping();
			Row->BlendParameters = BlendParameters;
			
			const float RealAssetTime = Database->GetRealAssetTime(DbPoseIdx);
			const FFrameRate& DefaultFrameRate = UAnimationSettings::Get()->GetDefaultFrameRate();
			Row->AnimFrame = DefaultFrameRate.AsFrameTime(RealAssetTime).RoundToFrame().Value;
			Row->AnimPercentage = FMath::IsNearlyZero(PlayLength) ? 0.f : RealAssetTime / PlayLength;
		}
	}
}

void SDebuggerDatabaseView::Update(const FTraceMotionMatchingStateMessage& State)
{
	// row cost color palette
	static const FLinearColor DiscardedRowColor(0.314f, 0.314f, 0.314f); // darker gray
	static const FLinearColor BestScoreRowColor = FLinearColor::Green;
	static const FLinearColor WorstScoreRowColor = FLinearColor::Red;

	using namespace DebuggerDatabaseColumns;

	bool bIsVerbose = false;

	TSharedPtr<FDebuggerViewModel> ViewModel;
	if (TSharedPtr<SDebuggerView> DebuggerView = ParentDebuggerViewPtr.Pin())
	{
		ViewModel = DebuggerView->GetViewModel();
		bIsVerbose = ViewModel->IsVerbose();
	}

	UnfilteredDatabaseRows.Reset();

	bool bAddPCACost = false;
	TArray<uint32> PoseToPCAValuesVectorIndexes;
	TAlignedArray<float> DynamicWeightsSqrtBuffer;

	InvalidDatabaseRows.Reset();
	InvalidDatabaseRows.Add(MakeShared<FDebuggerDatabaseRowData>(MakeShared<FDebuggerDatabaseSharedData>()));

	FAsyncPoseSearchDatabasesManagementCache DatabasesManagementCache;
	for (const FTraceMotionMatchingStateDatabaseEntry& DbEntry : State.DatabaseEntries)
	{
		UPoseSearchDatabase* Database = ResolveDatabaseFromId(DbEntry.DatabaseId);
		if (EAsyncBuildIndexResult::Success == DatabasesManagementCache(Database))
		{
			const FSearchIndex& SearchIndex = Database->GetSearchIndex();
			SearchIndex.GetPoseToPCAValuesVectorIndexes(PoseToPCAValuesVectorIndexes);

			TSharedRef<FDebuggerDatabaseSharedData> SharedData = MakeShared<FDebuggerDatabaseSharedData>();
			SharedData->SourceDatabase = Database;
			SharedData->QueryVector = DbEntry.QueryVector;

			// checking for DbEntry.QueryVector.Num() == Database->Schema->SchemaCardinality to make sure the stored DbEntry data is still aligned with the Database->Schema
			// (this could happen when pausing PIE, change the schema cardinality, scrubbing back to previously recorded data)
			if (Database->PoseSearchMode == EPoseSearchMode::PCAKDTree && DbEntry.QueryVector.Num() == Database->Schema->SchemaCardinality)
			{
				SharedData->PCAQueryVector.SetNumZeroed(Database->GetNumberOfPrincipalComponents());
				SearchIndex.PCAProject(DbEntry.QueryVector, SharedData->PCAQueryVector);
				bAddPCACost = bIsVerbose;
			}

			DynamicWeightsSqrtBuffer.SetNumUninitialized(SearchIndex.WeightsSqrt.Num());
			TConstArrayView<float> DynamicWeightsSqrt = Database->CalculateDynamicWeightsSqrt(DynamicWeightsSqrtBuffer);

			for (const FTraceMotionMatchingStatePoseEntry& PoseEntry : DbEntry.PoseEntries)
			{
				AddUnfilteredDatabaseRow(Database, DynamicWeightsSqrt, UnfilteredDatabaseRows, SharedData, PoseEntry.DbPoseIdx, PoseEntry.PoseCandidateFlags, PoseToPCAValuesVectorIndexes, PoseEntry.Cost);
			}

			if (UPoseSearchDebuggerConfig::Get().bShowAllPoses)
			{
				bool bAddMissingPoseEntries = false;

				// excluding the already added PoseEntry
				TSet<int32> PoseEntriesIdx;
				for (const FTraceMotionMatchingStatePoseEntry& PoseEntry : DbEntry.PoseEntries)
				{
					PoseEntriesIdx.Add(PoseEntry.DbPoseIdx);

					const bool bIsContinuingPose = EnumHasAnyFlags(PoseEntry.PoseCandidateFlags, EPoseCandidateFlags::Valid_ContinuingPose);
					bAddMissingPoseEntries |= !bIsContinuingPose;
				}

				// adding the missing ones
				if (bAddMissingPoseEntries)
				{
					for (int32 DbPoseIdx = 0; DbPoseIdx < SearchIndex.GetNumPoses(); ++DbPoseIdx)
					{
						if (!PoseEntriesIdx.Find(DbPoseIdx))
						{
							AddUnfilteredDatabaseRow(Database, DynamicWeightsSqrt, UnfilteredDatabaseRows, SharedData, DbPoseIdx, EPoseCandidateFlags::DiscardedBy_Search, PoseToPCAValuesVectorIndexes);
						}
					}
				}
			}
		}
	}

	SCostBreakDownData CostBreakDownData(State.DatabaseEntries, bIsVerbose);
	if (!UnfilteredDatabaseRows.IsEmpty())
	{
		CostBreakDownData.ProcessData(UnfilteredDatabaseRows);

		// calculating min/max for Cost, PCACost, and breakdowns
		float MinCost = UE_MAX_FLT;
		float MaxCost = -UE_MAX_FLT;

		float MinPCACost = UE_MAX_FLT;
		float MaxPCACost = -UE_MAX_FLT;

		TArray<float> MinCostBreakdowns;
		TArray<float> MaxCostBreakdowns;

		const int32 CostBreakdownsCardinality = UnfilteredDatabaseRows[0]->CostBreakdowns.Num();
		MinCostBreakdowns.Init(UE_MAX_FLT, CostBreakdownsCardinality);
		MaxCostBreakdowns.Init(-UE_MAX_FLT, CostBreakdownsCardinality);
		
		for (TSharedRef<FDebuggerDatabaseRowData>& UnfilteredRow : UnfilteredDatabaseRows)
		{
			if (EnumHasAnyFlags(UnfilteredRow->PoseCandidateFlags, EPoseCandidateFlags::AnyValidMask))
			{
				const float Cost = UnfilteredRow->PoseCost;
				MinCost = FMath::Min(MinCost, Cost);
				MaxCost = FMath::Max(MaxCost, Cost);

				const float PCACost = UnfilteredRow->PosePCACost;
				MinPCACost = FMath::Min(MinPCACost, PCACost);
				MaxPCACost = FMath::Max(MaxPCACost, PCACost);

				for (int32 Index = 0; Index < CostBreakdownsCardinality; ++Index)
				{
					const float Value = UnfilteredRow->CostBreakdowns[Index];
					if (Value != UE_MAX_FLT)
					{
						MinCostBreakdowns[Index] = FMath::Min(MinCostBreakdowns[Index], Value);
						MaxCostBreakdowns[Index] = FMath::Max(MaxCostBreakdowns[Index], Value);
					}
				}
			}
		}
		
		// calculating min max deltas (always >= UE_KINDA_SMALL_NUMBER to avoid division by zero)
		const float DeltaCost = FMath::Max(MaxCost - MinCost, UE_KINDA_SMALL_NUMBER);
		const float DeltaPCACost = FMath::Max(MaxPCACost - MinPCACost, UE_KINDA_SMALL_NUMBER);
		TArray<float> DeltaCostBreakdowns;
		DeltaCostBreakdowns.SetNumUninitialized(CostBreakdownsCardinality);
		for (int32 Index = 0; Index < CostBreakdownsCardinality; ++Index)
		{
			DeltaCostBreakdowns[Index] = FMath::Max(MaxCostBreakdowns[Index] - MinCostBreakdowns[Index], UE_KINDA_SMALL_NUMBER);
		}

		// calculating colors for Cost, PCACost, and breakdowns (using min/max as range)
		for (TSharedRef<FDebuggerDatabaseRowData>& UnfilteredRow : UnfilteredDatabaseRows)
		{
			if (EnumHasAnyFlags(UnfilteredRow->PoseCandidateFlags, EPoseCandidateFlags::AnyValidMask))
			{
				const float CostBlend = (UnfilteredRow->PoseCost - MinCost) / DeltaCost;
				UnfilteredRow->CostColor = BestScoreRowColor + (WorstScoreRowColor - BestScoreRowColor) * CostBlend;

				const float PCACostBlend = (UnfilteredRow->PosePCACost - MinPCACost) / DeltaPCACost;
				UnfilteredRow->PCACostColor = BestScoreRowColor + (WorstScoreRowColor - BestScoreRowColor) * PCACostBlend;

				UnfilteredRow->CostBreakdownsColors.SetNumUninitialized(CostBreakdownsCardinality);
				for (int32 Index = 0; Index < CostBreakdownsCardinality; ++Index)
				{
					const float CostBreakdownBlend = (UnfilteredRow->CostBreakdowns[Index] - MinCostBreakdowns[Index]) / DeltaCostBreakdowns[Index];
					UnfilteredRow->CostBreakdownsColors[Index] = BestScoreRowColor + (WorstScoreRowColor - BestScoreRowColor) * CostBreakdownBlend;
				}
			}
			else
			{
				UnfilteredRow->CostColor = DiscardedRowColor;
				UnfilteredRow->PCACostColor = DiscardedRowColor;
				UnfilteredRow->CostBreakdownsColors.Init(DiscardedRowColor, CostBreakdownsCardinality);
			}
		}
	}

	if (bHasPCACostColumn != bAddPCACost || bWasVerbose != bIsVerbose || !CostBreakDownData.AreLabelsEqualTo(OldLabels))
	{
		bHasPCACostColumn = bAddPCACost;
		bWasVerbose = bIsVerbose;

		OldLabels = CostBreakDownData.GetLabels();

		// recreating and binding the columns
		Columns.Reset();

		// Construct all column types
		int32 ColumnIdx = 0;
		AddColumn(MakeShared<FDatabaseName>(ColumnIdx++, ViewModel));
		AddColumn(MakeShared<FAssetName>(ColumnIdx++));

		TSharedRef<FCost> CostColumn = MakeShared<FCost>(ColumnIdx++);
		AddColumn(CostColumn);

		if (bAddPCACost)
		{
			AddColumn(MakeShared<FPCACost>(ColumnIdx++));
		}

		if (bIsVerbose)
		{
			AddColumn(MakeShared<FContinuingPoseCost>(ColumnIdx++));
			AddColumn(MakeShared<FNotifyCost>(ColumnIdx++));
			AddColumn(MakeShared<FContinuingInteractionCost>(ColumnIdx++));
			AddColumn(MakeShared<FContinuingContextInteractionCost>(ColumnIdx++));
		}
		else
		{
			AddColumn(MakeShared<FCostModifier>(ColumnIdx++));
		}
		
		int32 LabelIdx = 0;
		for (const FText& Label : CostBreakDownData.GetLabels())
		{
			AddColumn(MakeShared<FChannelBreakdownCostColumn>(ColumnIdx++, LabelIdx++, Label));
		}

		AddColumn(MakeShared<FFrame>(ColumnIdx++));
		AddColumn(MakeShared<FTime>(ColumnIdx++));
		AddColumn(MakeShared<FPercentage>(ColumnIdx++));
		AddColumn(MakeShared<FMirrored>(ColumnIdx++));
		AddColumn(MakeShared<FLooping>(ColumnIdx++));
		AddColumn(MakeShared<FPoseIdx>(ColumnIdx++));
		AddColumn(MakeShared<FAssetIdx>(ColumnIdx++));
		AddColumn(MakeShared<FBlendParameters>(ColumnIdx++));
		AddColumn(MakeShared<FPoseCandidateFlags>(ColumnIdx++));

		SortColumn = CostColumn->ColumnId;

		// Active and Continuing Pose view scroll bars only for indenting the columns to align w/ database
		ActiveView.ScrollBar->SetVisibility(EVisibility::Hidden);
		ContinuingPoseView.ScrollBar->SetVisibility(EVisibility::Hidden);

		// Refresh Columns
		ActiveView.HeaderRow->ClearColumns();
		ContinuingPoseView.HeaderRow->ClearColumns();
		FilteredDatabaseView.HeaderRow->ClearColumns();

		// Sort columns by index
		Columns.ValueSort([](const TSharedRef<IColumn> Column0, const TSharedRef<IColumn> Column1)
		{
			return Column0->SortIndex < Column1->SortIndex;
		});

		// Compute columns widths
		for (TPair<FName, TSharedRef<IColumn>>& ColumnPair : Columns)
		{
			ColumnPair.Value->Width = ColumnPair.Value->ComputeColumnWidth(UnfilteredDatabaseRows);

			// Add spacing for sorting arrow
			if (ColumnPair.Value->ColumnId == SortColumn && SortMode != EColumnSortMode::None)
			{
				ColumnPair.Value->Width += DebuggerDatabaseColumns::ColumnSortWidthPadding;
			}
		}
		
		// Add columns from map to header row
		for (TPair<FName, TSharedRef<IColumn>>& ColumnPair : Columns)
		{
			IColumn& Column = ColumnPair.Value.Get();
			if (ColumnPair.Value->bEnabled)
			{
				SHeaderRow::FColumn::FArguments ColumnArgs = SHeaderRow::FColumn::FArguments()
					.ColumnId(Column.ColumnId)
					.DefaultTooltip(Column.GetLabelTooltip())
					.SortMode(this, &SDebuggerDatabaseView::GetColumnSortMode, Column.ColumnId)
					.OnSort(this, &SDebuggerDatabaseView::OnColumnSortModeChanged)
					.ManualWidth(this, &SDebuggerDatabaseView::GetColumnWidth, Column.ColumnId)
					.OnWidthChanged(this, &SDebuggerDatabaseView::OnColumnWidthChanged, Column.ColumnId)
					.VAlignHeader(VAlign_Center)
					.VAlignCell(VAlign_Center)
					.HAlignHeader(HAlign_Center)
					.HAlignCell(HAlign_Fill)
					.HeaderContent()
					[
						SNew(SBox)
						.VAlign( VAlign_Fill )
						.HAlign( HAlign_Fill )
						[
							SNew(STextBlock)
							.TextStyle(FAppStyle::Get(), "SmallText")
							.Text(Column.GetLabel())
							.Margin(FMargin(5.f, 5.f, 5.f, 5.f))
							.Justification(ETextJustify::Center)
						]
					];

				FilteredDatabaseView.HeaderRow->AddColumn(ColumnArgs);
				ContinuingPoseView.HeaderRow->AddColumn(ColumnArgs);
				ActiveView.HeaderRow->AddColumn(ColumnArgs);
			}
		}
	}

	SortDatabaseRows();
	PopulateViewRows();
}

void SDebuggerDatabaseView::AddColumn(TSharedRef<DebuggerDatabaseColumns::IColumn>&& Column)
{
	Columns.Add(Column->ColumnId, Column);
}

EColumnSortMode::Type SDebuggerDatabaseView::GetColumnSortMode(const FName ColumnId) const
{
	if (ColumnId == SortColumn)
	{
		return SortMode;
	}
	
	return EColumnSortMode::None;
}

float SDebuggerDatabaseView::GetColumnWidth(const FName ColumnId) const
{
	return Columns[ColumnId]->Width;
}

void SDebuggerDatabaseView::OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName & ColumnId, const EColumnSortMode::Type InSortMode)
{
	const bool bHasSortColumnChanged = SortColumn != ColumnId;
	
	// Restore previous sort column to its default width
	if (bHasSortColumnChanged)
	{
		Columns[SortColumn]->Width -= DebuggerDatabaseColumns::ColumnSortWidthPadding;
	}
	
	SortColumn = ColumnId;
	SortMode = InSortMode;
	SortDatabaseRows();
	PopulateViewRows();

	// Take into account sorting arrow for the column width
	if (bHasSortColumnChanged && SortMode != EColumnSortMode::None)
	{
		Columns[SortColumn]->Width += DebuggerDatabaseColumns::ColumnSortWidthPadding;
	}
}

void SDebuggerDatabaseView::OnColumnWidthChanged(const float NewWidth, FName ColumnId) const
{
	Columns[ColumnId]->Width = NewWidth;
}

void SDebuggerDatabaseView::OnFilterTextChanged(const FText& SearchText)
{
	FilterText = SearchText;
	PopulateViewRows();
}

void SDebuggerDatabaseView::OnShowAllPosesCheckboxChanged(ECheckBoxState State)
{
	UPoseSearchDebuggerConfig::Get().bShowAllPoses = State == ECheckBoxState::Checked;

	if (TSharedPtr<SDebuggerView> DebuggerView = ParentDebuggerViewPtr.Pin())
	{
		if (TSharedPtr<FDebuggerViewModel> ViewModel = DebuggerView->GetViewModel())
		{
			if (const FTraceMotionMatchingStateMessage* MotionMatchingState = ViewModel.Get()->GetMotionMatchingState())
			{
				Update(*MotionMatchingState);
			}
		}
	}
}

void SDebuggerDatabaseView::OnShowOnlyBestAssetPoseCheckboxChanged(ECheckBoxState State)
{
	UPoseSearchDebuggerConfig::Get().bShowOnlyBestAssetPose = State == ECheckBoxState::Checked;
	PopulateViewRows();
}

void SDebuggerDatabaseView::OnHideInvalidPosesCheckboxChanged(ECheckBoxState State)
{
	UPoseSearchDebuggerConfig::Get().bHideInvalidPoses = State == ECheckBoxState::Checked;
	PopulateViewRows();
}

void SDebuggerDatabaseView::OnUseRegexCheckboxChanged(ECheckBoxState State)
{
	UPoseSearchDebuggerConfig::Get().bUseRegex = State == ECheckBoxState::Checked;
	PopulateViewRows();
}

void SDebuggerDatabaseView::OnViewSelectionChanged(TSharedPtr<FDebuggerDatabaseRowData> Row, SDebuggerDatabaseView::FTable& Table)
{
	Table.PreviouslySelectedItems = Table.ListView->GetSelectedItems();

	if (Row.IsValid())
	{
		OnPoseSelectionChanged.ExecuteIfBound(Row->SharedData->SourceDatabase.Get(), Row->PoseIdx, Row->AssetTime);
	}
}

void SDebuggerDatabaseView::OnActiveViewSelectionChanged(TSharedPtr<FDebuggerDatabaseRowData> Row, ESelectInfo::Type SelectInfo)
{
	OnViewSelectionChanged(Row, ActiveView);
}

void SDebuggerDatabaseView::OnContinuingPoseViewSelectionChanged(TSharedPtr<FDebuggerDatabaseRowData> Row, ESelectInfo::Type SelectInfo)
{
	OnViewSelectionChanged(Row, ContinuingPoseView);
}

void SDebuggerDatabaseView::OnFilteredDatabaseViewSelectionChanged(TSharedPtr<FDebuggerDatabaseRowData> Row, ESelectInfo::Type SelectInfo)
{
	OnViewSelectionChanged(Row, FilteredDatabaseView);
}

void SDebuggerDatabaseView::OnViewMouseButtonClick(const TSharedPtr<FDebuggerDatabaseRowData> Row, SDebuggerDatabaseView::FTable& Table)
{
	const TArray<TSharedRef<FDebuggerDatabaseRowData>> CurrentlySelectedItems = Table.ListView->GetSelectedItems();

	if (CurrentlySelectedItems.Num() == 1 && Table.PreviouslySelectedItems.Num() == 1 &&
		CurrentlySelectedItems[0] == Row && Table.PreviouslySelectedItems[0] == Row)
	{
		Table.ListView->ClearSelection();
	}
}

void SDebuggerDatabaseView::OnActiveViewMouseButtonClick(const TSharedRef<FDebuggerDatabaseRowData> Row)
{
	OnViewMouseButtonClick(Row, ActiveView);
}

void SDebuggerDatabaseView::OnContinuingPoseViewMouseButtonClick(const TSharedRef<FDebuggerDatabaseRowData> Row)
{
	OnViewMouseButtonClick(Row, ContinuingPoseView);
}

void SDebuggerDatabaseView::OnFilteredDatabaseViewMouseButtonClick(const TSharedRef<FDebuggerDatabaseRowData> Row)
{
	OnViewMouseButtonClick(Row, FilteredDatabaseView);
}

void SDebuggerDatabaseView::SortDatabaseRows()
{
	if (!UnfilteredDatabaseRows.IsEmpty() && Columns.Contains(SortColumn))
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			UnfilteredDatabaseRows.Sort(Columns[SortColumn]->GetSortPredicate());
		}
		else if (SortMode == EColumnSortMode::Descending)
		{
			auto DescendingPredicate = [this](const auto& Lhs, const auto& Rhs) -> bool
			{
				return !Columns[SortColumn]->GetSortPredicate()(Lhs, Rhs);
			};

			UnfilteredDatabaseRows.Sort(DescendingPredicate);
		}
	}
}

void SDebuggerDatabaseView::PopulateViewRows()
{
	ActiveView.Rows.Reset();
	ContinuingPoseView.Rows.Reset();
	FilteredDatabaseView.Rows.Reset();

	const int32 UnfilteredDatabaseRowsNum = UnfilteredDatabaseRows.Num();
	if (UnfilteredDatabaseRowsNum > 0)
	{
		FString FilterString = FilterText.ToString();
		TArray<FString> Tokens;
		FilterString.ParseIntoArrayWS(Tokens);
		const bool bHasNameFilter = !Tokens.IsEmpty();
		FRegexPattern Pattern(FilterString);
	
		TSet<int32> BestAssetPoseIndex;
		if (UPoseSearchDebuggerConfig::Get().bShowOnlyBestAssetPose)
		{
			TArray<int32> SortIndex;
			SortIndex.SetNumUninitialized(UnfilteredDatabaseRowsNum);
			for (int32 i = 0; i < UnfilteredDatabaseRowsNum; ++i)
			{
				SortIndex[i] = i;
			}

			// sorting SortIndex elements by ascending order in cost, after grouping them by asset index
			Algo::Sort(SortIndex, [this](int32 IndexA, int32 IndexB)
			{
				const FDebuggerDatabaseRowData& RowDataA = UnfilteredDatabaseRows[IndexA].Get();
				const FDebuggerDatabaseRowData& RowDataB = UnfilteredDatabaseRows[IndexB].Get();

				if (RowDataA.SharedData->SourceDatabase != RowDataB.SharedData->SourceDatabase)
				{
					const int32 DatabaseNameCompare = GetNameSafe(RowDataA.SharedData->SourceDatabase.Get()).Compare(GetNameSafe(RowDataB.SharedData->SourceDatabase.Get()));
					if (DatabaseNameCompare < 0)
					{
						return true;
					}

					const int32 DatabasePathCompare = GetPathNameSafe(RowDataA.SharedData->SourceDatabase.Get()).Compare(GetPathNameSafe(RowDataB.SharedData->SourceDatabase.Get()));
					return DatabasePathCompare < 0;
				}

				if (RowDataA.DbAssetIdx == RowDataB.DbAssetIdx)
				{
					const bool bIsRowDataAValid = EnumHasAnyFlags(RowDataA.PoseCandidateFlags, EPoseCandidateFlags::AnyValidMask);
					const bool bIsRowDataBValid = EnumHasAnyFlags(RowDataB.PoseCandidateFlags, EPoseCandidateFlags::AnyValidMask);

					if (bIsRowDataAValid == bIsRowDataBValid)
					{
						return RowDataA.PoseCost < RowDataB.PoseCost;
					}
					return bIsRowDataAValid;
				}
				return RowDataA.DbAssetIdx < RowDataB.DbAssetIdx;
			});

			// populating BestAssetPoseIndex only with the best pose index (lowest total cost) for each asset
			int32 PreviousDbAssetIdx = INDEX_NONE;
			TWeakObjectPtr<const UPoseSearchDatabase> PreviousSourceDatabase;
			for (int32 i = 0; i < UnfilteredDatabaseRowsNum; ++i)
			{
				const FDebuggerDatabaseRowData& RowData = UnfilteredDatabaseRows[SortIndex[i]].Get();
				if (PreviousDbAssetIdx != RowData.DbAssetIdx || PreviousSourceDatabase != RowData.SharedData->SourceDatabase)
				{
					// adding only the best pose index for the RowData.DbAssetIdx asset to BestAssetPoseIndex 
					// (the first we find by iterating over UnfilteredDatabaseRows[SortIndex[i]])
					BestAssetPoseIndex.Add(RowData.PoseIdx);
					PreviousDbAssetIdx = RowData.DbAssetIdx;
					PreviousSourceDatabase = RowData.SharedData->SourceDatabase;
				}
			}
		}

		for (int32 i = 0; i < UnfilteredDatabaseRowsNum; ++i)
		{
			const TSharedRef<FDebuggerDatabaseRowData>& UnfilteredRow = UnfilteredDatabaseRows[i];

			bool bTryAddToFilteredDatabaseViewRows = true;
			if (EnumHasAnyFlags(UnfilteredRow->PoseCandidateFlags, EPoseCandidateFlags::Valid_ContinuingPose))
			{
				ContinuingPoseView.Rows.Add(UnfilteredRow);
				bTryAddToFilteredDatabaseViewRows = false;
			}

			if (EnumHasAnyFlags(UnfilteredRow->PoseCandidateFlags, EPoseCandidateFlags::Valid_CurrentPose))
			{
				ActiveView.Rows.Add(UnfilteredRow);
				bTryAddToFilteredDatabaseViewRows = false;
			}

			if (UPoseSearchDebuggerConfig::Get().bShowOnlyBestAssetPose)
			{
				if (!BestAssetPoseIndex.Find(UnfilteredRow->PoseIdx))
				{
					bTryAddToFilteredDatabaseViewRows = false;
				}
			}

			if (bTryAddToFilteredDatabaseViewRows)
			{
				bool bPassesNameFilter = true;
				if (UPoseSearchDebuggerConfig::Get().bHideInvalidPoses && !EnumHasAnyFlags(UnfilteredRow->PoseCandidateFlags, EPoseCandidateFlags::AnyValidMask))
				{
					bPassesNameFilter = false;
				}
				else if (UPoseSearchDebuggerConfig::Get().bUseRegex)
				{
					FRegexMatcher Matcher(Pattern, UnfilteredRow->AssetName);
					bPassesNameFilter = Matcher.FindNext();
				}
				else if (bHasNameFilter)
				{
					bPassesNameFilter = Algo::AllOf(Tokens, [&](FString Token)
						{
							return UnfilteredRow->AssetName.Contains(Token);
						});
				}
				
				if (bPassesNameFilter)
				{
					FilteredDatabaseView.Rows.Add(UnfilteredRow);
				}
			}
		}
	}

	ActiveView.ListView->RequestListRefresh();
	ContinuingPoseView.ListView->RequestListRefresh();
	FilteredDatabaseView.ListView->RequestListRefresh();

	if (ActiveView.Rows.IsEmpty())
	{
		ActiveView.Rows.Add(InvalidDatabaseRows[0]);
	}

	if (ContinuingPoseView.Rows.IsEmpty())
	{
		ContinuingPoseView.Rows.Add(InvalidDatabaseRows[0]);
	}
}

TSharedRef<ITableRow> SDebuggerDatabaseView::HandleGenerateDatabaseRow(TSharedRef<FDebuggerDatabaseRowData> Item, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(SDebuggerDatabaseRow, OwnerTable, Item, FilteredDatabaseView.RowStyle, &FilteredDatabaseView.RowBrush, FMargin(0.0f, 2.0f, 6.0f, 2.0f))
		.ColumnMap(this, &SDebuggerDatabaseView::GetColumnMap);	
}

TSharedRef<ITableRow> SDebuggerDatabaseView::HandleGenerateActiveRow(TSharedRef<FDebuggerDatabaseRowData> Item, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(SDebuggerDatabaseRow, OwnerTable, Item, ActiveView.RowStyle, &ActiveView.RowBrush, FMargin(0.0f, 2.0f, 6.0f, 4.0f))
		.ColumnMap(this, &SDebuggerDatabaseView::GetColumnMap);
}

TSharedRef<ITableRow> SDebuggerDatabaseView::HandleGenerateContinuingPoseRow(TSharedRef<FDebuggerDatabaseRowData> Item, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(SDebuggerDatabaseRow, OwnerTable, Item, ContinuingPoseView.RowStyle, &ContinuingPoseView.RowBrush, FMargin(0.0f, 2.0f, 6.0f, 4.0f))
		.ColumnMap(this, &SDebuggerDatabaseView::GetColumnMap);
}

void SDebuggerDatabaseView::Construct(const FArguments& InArgs)
{
	ParentDebuggerViewPtr = InArgs._Parent;
	OnPoseSelectionChanged = InArgs._OnPoseSelectionChanged;
	check(OnPoseSelectionChanged.IsBound());

	// Active Row
	ActiveView.HeaderRow = SNew(SHeaderRow);

	// Used for spacing
	ActiveView.ScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Vertical)
		.HideWhenNotInUse(false)
		.AlwaysShowScrollbar(true)
		.AlwaysShowScrollbarTrack(true);

	ActiveView.ListView = SNew(SListView<TSharedRef<FDebuggerDatabaseRowData>>)
		.ListItemsSource(&ActiveView.Rows)
		.HeaderRow(ActiveView.HeaderRow.ToSharedRef())
		.OnGenerateRow(this, &SDebuggerDatabaseView::HandleGenerateActiveRow)
		.ExternalScrollbar(ActiveView.ScrollBar)
		.OnSelectionChanged(this, &SDebuggerDatabaseView::OnActiveViewSelectionChanged)
		.OnMouseButtonClick(this, &SDebuggerDatabaseView::OnActiveViewMouseButtonClick);

	ActiveView.RowStyle = FAppStyle::GetWidgetStyle<FTableRowStyle>("TableView.Row");
	ActiveView.RowBrush = *FAppStyle::GetBrush("DetailsView.CategoryTop");

	// ContinuingPose Row
	ContinuingPoseView.HeaderRow = SNew(SHeaderRow).Visibility(EVisibility::Collapsed);

	// Used for spacing
	ContinuingPoseView.ScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Vertical)
		.HideWhenNotInUse(false)
		.AlwaysShowScrollbar(true)
		.AlwaysShowScrollbarTrack(true);

	ContinuingPoseView.ListView = SNew(SListView<TSharedRef<FDebuggerDatabaseRowData>>)
		.ListItemsSource(&ContinuingPoseView.Rows)
		.HeaderRow(ContinuingPoseView.HeaderRow.ToSharedRef())
		.OnGenerateRow(this, &SDebuggerDatabaseView::HandleGenerateContinuingPoseRow)
		.ExternalScrollbar(ContinuingPoseView.ScrollBar)
		.OnSelectionChanged(this, &SDebuggerDatabaseView::OnContinuingPoseViewSelectionChanged)
		.OnMouseButtonClick(this, &SDebuggerDatabaseView::OnContinuingPoseViewMouseButtonClick);

	ContinuingPoseView.RowStyle = FAppStyle::GetWidgetStyle<FTableRowStyle>("TableView.Row");
	ContinuingPoseView.RowBrush = *FAppStyle::GetBrush("DetailsView.CategoryTop");

	// Filtered Database
	FilteredDatabaseView.ScrollBar =
		SNew(SScrollBar)
		.Orientation(Orient_Vertical)
		.HideWhenNotInUse(false)
		.AlwaysShowScrollbar(true)
		.AlwaysShowScrollbarTrack(true);
	FilteredDatabaseView.HeaderRow = SNew(SHeaderRow).Visibility(EVisibility::Collapsed);

	FilteredDatabaseView.ListView = SNew(SListView<TSharedRef<FDebuggerDatabaseRowData>>)
		.ListItemsSource(&FilteredDatabaseView.Rows)
		.HeaderRow(FilteredDatabaseView.HeaderRow.ToSharedRef())
		.OnGenerateRow(this, &SDebuggerDatabaseView::HandleGenerateDatabaseRow)
		.ExternalScrollbar(FilteredDatabaseView.ScrollBar)
		.OnSelectionChanged(this, &SDebuggerDatabaseView::OnFilteredDatabaseViewSelectionChanged)
		.OnMouseButtonClick(this, &SDebuggerDatabaseView::OnFilteredDatabaseViewMouseButtonClick);

	FilteredDatabaseView.RowStyle = FAppStyle::GetWidgetStyle<FTableRowStyle>("TableView.Row");
	// Set selected color to white to retain visibility when multi-selecting
	FilteredDatabaseView.RowStyle.SetSelectedTextColor(FLinearColor(FVector3f(0.8f)));
	FilteredDatabaseView.RowBrush = *FAppStyle::GetBrush("ToolPanel.GroupBorder");

	ChildSlot
	[
		SNew(SScrollBox)
		.Orientation(Orient_Horizontal)
		.ScrollBarAlwaysVisible(true)
		+ SScrollBox::Slot()
		.FillSize(1.f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			// Side and top margins, ignore bottom handled by the color border below
			.Padding(0.0f, 5.0f, 0.0f, 0.0f)
			.AutoHeight()
			[
				// Active Row text tab
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Fill)
					.Padding(0.0f)
					.AutoWidth()
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
						.Padding(FMargin(30.0f, 3.0f, 30.0f, 0.0f))
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Fill)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ActivePoseLabel", "Active Pose"))
						]
					]
				]

				// Active row list view with scroll bar
				+ SVerticalBox::Slot()
			
				.AutoHeight()
				[
					SNew(SHorizontalBox)
				
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.Padding(0.0f)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("NoBorder"))
						.Padding(0.0f)
						[
							ActiveView.ListView.ToSharedRef()
						]
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						ActiveView.ScrollBar.ToSharedRef()
					]
				]	
			]

			+ SVerticalBox::Slot()
			// Side and top margins, ignore bottom handled by the color border below
			.Padding(0.0f, 5.0f, 0.0f, 0.0f)
			.AutoHeight()
			[
				// ContinuingPose Row text tab
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Fill)
					.Padding(0.0f)
					.AutoWidth()
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
						.Padding(FMargin(30.0f, 3.0f, 30.0f, 0.0f))
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Fill)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ContinuingPoseLabel", "Continuing Pose"))
						]
					]
				]

				// ContinuingPose row list view with scroll bar
				+ SVerticalBox::Slot()
			
				.AutoHeight()
				[
					SNew(SHorizontalBox)
				
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.Padding(0.0f)
					[
						
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("NoBorder"))
						.Padding(0.0f)
						[
							ContinuingPoseView.ListView.ToSharedRef()
						]
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						ContinuingPoseView.ScrollBar.ToSharedRef()
					]
				]	
			]
		
			+ SVerticalBox::Slot()
			.Padding(0.0f, 0.0f, 0.0f, 5.0f)
			[
				// Database view text tab
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Fill)
					.Padding(0.0f)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
						.Padding(FMargin(30.0f, 3.0f, 30.0f, 0.0f))
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Fill)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("PoseCandidatesLabel", "Pose Candidates"))
						]
					]
					.AutoWidth()
				
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					[
						SNew(SBorder)
						.BorderImage(&FilteredDatabaseView.RowStyle.EvenRowBackgroundBrush)
					]
				]
				.AutoHeight()

				// Gray line below the tab 
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
					.Padding(FMargin(0.0f, 3.0f, 0.0f, 3.0f))
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
				]

				+ SVerticalBox::Slot()
				.Padding(0.0f, 0.0f, 0.0f, 5.0f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(10, 5, 10, 5)
					[
						SAssignNew(FilterBox, SSearchBox)
						.OnTextChanged(this, &SDebuggerDatabaseView::OnFilterTextChanged)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(10, 5, 10, 5)
					[
						SNew(SCheckBox)
						.IsChecked_Lambda([this]
						{
							return UPoseSearchDebuggerConfig::Get().bShowAllPoses ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; 
						})
						.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
						{
							   SDebuggerDatabaseView::OnShowAllPosesCheckboxChanged(State);
						})
						[
							SNew(STextBlock)
							.Text(LOCTEXT("PoseSearchDebuggerShowAllPosesFlag", "Show All Poses"))
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(10, 5, 10, 5)
					[
						SNew(SCheckBox)
						.IsChecked_Lambda([this]
						{
							return UPoseSearchDebuggerConfig::Get().bShowOnlyBestAssetPose ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; 
						})
						.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
						{
							SDebuggerDatabaseView::OnShowOnlyBestAssetPoseCheckboxChanged(State);
						})
						[
							SNew(STextBlock)
							.Text(LOCTEXT("PoseSearchDebuggerShowOnlyBestAssetPoseFlag", "Only Best Asset Pose"))
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(10, 5, 10, 5)
					[
						SNew(SCheckBox)
						.IsChecked_Lambda([this]
						{
							return UPoseSearchDebuggerConfig::Get().bHideInvalidPoses ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; 
						})
						.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
						{
							   SDebuggerDatabaseView::OnHideInvalidPosesCheckboxChanged(State);
						})
						[
							SNew(STextBlock)
							.Text(LOCTEXT("PoseSearchDebuggerHideInvalidPosesFlag", "Hide Invalid Poses"))
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(10, 5, 10, 5)
					[
						SNew(SCheckBox)
						.IsChecked_Lambda([this]
						{
							return UPoseSearchDebuggerConfig::Get().bUseRegex ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; 
						})
						.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
							{
								SDebuggerDatabaseView::OnUseRegexCheckboxChanged(State);
							})
						[
							SNew(STextBlock)
							.Text(LOCTEXT("PoseSearchDebuggerUseRegexFlag", "Use Regex"))
						]
					]
				]
		
				+ SVerticalBox::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(0.0f)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("NoBorder"))
						.Padding(0.0f)
						[
							FilteredDatabaseView.ListView.ToSharedRef()
						]
					]
				
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						FilteredDatabaseView.ScrollBar.ToSharedRef()
					]
				]
			]
		]
	];

	SortMode = EColumnSortMode::Ascending;
	OldLabels.Reset();
	Columns.Reset();
	bHasPCACostColumn = false;
	bWasVerbose = false;
}

} // namespace UE::PoseSearch

#undef LOCTEXT_NAMESPACE
