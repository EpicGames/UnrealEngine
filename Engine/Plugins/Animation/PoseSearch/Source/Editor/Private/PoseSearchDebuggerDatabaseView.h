// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearchDebuggerDatabaseRowData.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/SHeaderRow.h"

template <typename ItemType> class SListView;

namespace UE::PoseSearch
{

UPoseSearchDatabase* ResolveDatabaseFromId(uint64 DatabaseId);

namespace DebuggerDatabaseColumns { struct IColumn; }
struct FTraceMotionMatchingStateMessage;

/** Sets model selection data on row selection */
DECLARE_DELEGATE_ThreeParams(FOnPoseSelectionChanged, const UPoseSearchDatabase*, int32, float)

/**
 * Database panel view widget of the PoseSearch debugger
 */
class SDebuggerDatabaseView : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDebuggerDatabaseView) {}
		SLATE_ARGUMENT(TWeakPtr<class SDebuggerView>, Parent)
		SLATE_EVENT(FOnPoseSelectionChanged, OnPoseSelectionChanged)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);
	void Update(const FTraceMotionMatchingStateMessage& State);

	const TSharedPtr<SListView<TSharedRef<FDebuggerDatabaseRowData>>>& GetActiveRow() const { return ActiveView.ListView; }
	const TSharedPtr<SListView<TSharedRef<FDebuggerDatabaseRowData>>>& GetContinuingPoseRow() const { return ContinuingPoseView.ListView; }
	const TSharedPtr<SListView<TSharedRef<FDebuggerDatabaseRowData>>>& GetDatabaseRows() const { return FilteredDatabaseView.ListView; }
	const TSharedRef<FDebuggerDatabaseRowData>& GetPoseIdxDatabaseRow(int32 PoseIdx) const;

	/** Used by database rows to acquire column-specific information */
	using FColumnMap = TMap<FName, TSharedRef<DebuggerDatabaseColumns::IColumn>>;

private:

	struct FTable
	{
		/** Header row*/
		TSharedPtr<SHeaderRow> HeaderRow;

		/** Widget for displaying the list of row objects */
		TSharedPtr<SListView<TSharedRef<FDebuggerDatabaseRowData>>> ListView;

		/** List of row objects */
		TArray<TSharedRef<FDebuggerDatabaseRowData>> Rows;

		/** Background style for the list view */
		FTableRowStyle RowStyle;

		/** Row color */
		FSlateBrush RowBrush;

		/** Scroll bar for the data table */
		TSharedPtr<SScrollBar> ScrollBar;

		TArray<TSharedRef<FDebuggerDatabaseRowData>> PreviouslySelectedItems;
	};

	/** Adds a column to the existing list */
	void AddColumn(TSharedRef<DebuggerDatabaseColumns::IColumn>&& Column);

	/** Retrieves current column map, used as an attribute by rows */
	const FColumnMap* GetColumnMap() const { return &Columns; }

	/** Sorts the database by the current sort predicate, updating the view order */
	void SortDatabaseRows();

	void PopulateViewRows();

	/** Acquires sort predicate for the given column */
	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;

	/** Gets active column width, used to align active and database view */
	float GetColumnWidth(const FName ColumnId) const;
	
	/** Updates the active sort predicate, setting the sorting order of all other columns to none
	 * (to be dependent on active column */
	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName & ColumnId, const EColumnSortMode::Type InSortMode);

	/** Aligns the active and database views */
	void OnColumnWidthChanged(const float NewWidth, FName ColumnId) const;

	/** Called when the text in the filter box is modified to update the filtering */
	void OnFilterTextChanged(const FText& SearchText);

	/** Called when the ShowAllPoses Checkbox is Toggled */
	void OnShowAllPosesCheckboxChanged(ECheckBoxState State);
	
	/** Called when the ShowOnlyBestAssetPose Checkbox is Toggled */
	void OnShowOnlyBestAssetPoseCheckboxChanged(ECheckBoxState State);	

	/** Called when the HideInvalidPoses Checkbox is Toggled */
	void OnHideInvalidPosesCheckboxChanged(ECheckBoxState State);

	/** Called when the UseRegex Checkbox is Toggled */
	void OnUseRegexCheckboxChanged(ECheckBoxState State);
	
	void OnViewSelectionChanged(TSharedPtr<FDebuggerDatabaseRowData> Row, FTable& Table);
	void OnActiveViewSelectionChanged(TSharedPtr<FDebuggerDatabaseRowData> Row, ESelectInfo::Type SelectInfo);
	void OnContinuingPoseViewSelectionChanged(TSharedPtr<FDebuggerDatabaseRowData> Row, ESelectInfo::Type SelectInfo);
	void OnFilteredDatabaseViewSelectionChanged(TSharedPtr<FDebuggerDatabaseRowData> Row, ESelectInfo::Type SelectInfo);

	void OnViewMouseButtonClick(const TSharedPtr<FDebuggerDatabaseRowData> Row, FTable& Table);
	void OnActiveViewMouseButtonClick(const TSharedRef<FDebuggerDatabaseRowData> Row);
	void OnContinuingPoseViewMouseButtonClick(const TSharedRef<FDebuggerDatabaseRowData> Row);
	void OnFilteredDatabaseViewMouseButtonClick(const TSharedRef<FDebuggerDatabaseRowData> Row);

	/** Generates a database row widget for the given data */
	TSharedRef<ITableRow> HandleGenerateDatabaseRow(TSharedRef<FDebuggerDatabaseRowData> Item, const TSharedRef<STableViewBase>& OwnerTable) const;
	
	/** Generates the active row widget for the given data */
	TSharedRef<ITableRow> HandleGenerateActiveRow(TSharedRef<FDebuggerDatabaseRowData> Item, const TSharedRef<STableViewBase>& OwnerTable) const;

	/** Generates the continuing pose row widget for the given data */
	TSharedRef<ITableRow> HandleGenerateContinuingPoseRow(TSharedRef<FDebuggerDatabaseRowData> Item, const TSharedRef<STableViewBase>& OwnerTable) const;

	TWeakPtr<SDebuggerView> ParentDebuggerViewPtr;

	FOnPoseSelectionChanged OnPoseSelectionChanged;

	/** Current column to sort by */
	FName SortColumn = "";

	/** Current sorting mode */
	EColumnSortMode::Type SortMode = EColumnSortMode::Ascending;
	
	/** Column data container, used to emplace defined column structures of various types */
    FColumnMap Columns;

	TArray<FText> OldLabels;
	bool bHasPCACostColumn = false;
	bool bWasVerbose = false;

	/** Active row at the top of the view */
	FTable ActiveView;

	/** Continuing pose row below Active row */
	FTable ContinuingPoseView;

	/** All database poses */
	TArray<TSharedRef<FDebuggerDatabaseRowData>> UnfilteredDatabaseRows;
	TArray<TSharedRef<FDebuggerDatabaseRowData>> InvalidDatabaseRows;

	/** Database listing for filtered poses */
	FTable FilteredDatabaseView;

	/** Search box widget */
	TSharedPtr<SSearchBox> FilterBox;

	/** Text used to filter DatabaseView */
	FText FilterText;
};

} // namespace UE::PoseSearch
