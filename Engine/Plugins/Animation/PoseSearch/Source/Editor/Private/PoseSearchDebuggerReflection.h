// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchLibrary.h"
#include "PoseSearchDebuggerReflection.generated.h"

/**
 * Reflection UObject being observed in the details view panel of the debugger
 */
UCLASS(MinimalAPI)
class UPoseSearchDebuggerReflection : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, Category = "Motion Matching State")
	EPoseSearchInterruptMode InterruptMode = EPoseSearchInterruptMode::DoNotInterrupt;

	/** Time since last PoseSearch */
	UPROPERTY(VisibleAnywhere, Category="Motion Matching State", meta = (ForceUnits = "s"))
	float ElapsedPoseSearchTime = 0.0f;

	UPROPERTY(VisibleAnywhere, Category = "Motion Matching State", meta = (ForceUnits = "s"))
	float AssetPlayerTime = 0.0f;

	UPROPERTY(VisibleAnywhere, Category = "Motion Matching State", meta = (ForceUnits = "s"))
	float LastDeltaTime = 0.0f;

	UPROPERTY(VisibleAnywhere, Category = "Motion Matching State", meta = (ForceUnits = "cm/s"))
	float SimLinearVelocity;

	UPROPERTY(VisibleAnywhere, Category = "Motion Matching State", meta = (ForceUnits = "deg/s"))
	float SimAngularVelocity;

	UPROPERTY(VisibleAnywhere, Category = "Motion Matching State", meta = (ForceUnits = "cm/s"))
	float AnimLinearVelocity;

	UPROPERTY(VisibleAnywhere, Category = "Motion Matching State", meta = (ForceUnits = "deg/s"))
	float AnimAngularVelocity;

	UPROPERTY(VisibleAnywhere, Category = "Motion Matching State", meta = (ForceUnits = "s"))
	float Playrate;

	UPROPERTY(VisibleAnywhere, Category = "Motion Matching State", meta = (ForceUnits = "cm/s"))
	float AnimLinearVelocityNoTimescale;

	UPROPERTY(VisibleAnywhere, Category = "Motion Matching State", meta = (ForceUnits = "deg/s"))
	float AnimAngularVelocityNoTimescale;
};
