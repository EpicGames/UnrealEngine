// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDebuggerSettings.h"
#include "Misc/CoreDelegates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchDebuggerSettings)

UPoseSearchDebuggerConfig::UPoseSearchDebuggerConfig()
{
	// Ensure we save settings on exit.
	FCoreDelegates::OnPreExit.AddLambda([]()
	{
		if (UPoseSearchDebuggerConfig* Config = GetMutableDefault<UPoseSearchDebuggerConfig>())
		{
			Config->SaveConfig();
		}
	});
}

UPoseSearchDebuggerConfig& UPoseSearchDebuggerConfig::Get()
{
	UPoseSearchDebuggerConfig* MutableCDO = GetMutableDefault<UPoseSearchDebuggerConfig>();
	check(MutableCDO != nullptr)
	
	return *MutableCDO;
}
