// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"

#include "PoseSearchDebuggerSettings.generated.h"

/** Settings that holds editor configurations. Not accessible in Project Settings. */
UCLASS(Config = EditorPerProjectUserSettings, MinimalAPI)
class UPoseSearchDebuggerConfig : public UObject
{
	GENERATED_BODY()

public:

	UPoseSearchDebuggerConfig();
	
	static UPoseSearchDebuggerConfig& Get();
	
	// General options

	/** Used to draw the query used to get pose for this frame from Motion Matching algorithm */
	UPROPERTY(config)
	bool bDrawQuery = false;

	/** Used to show the trajectory used for this frame to run Motion Matching against */
	UPROPERTY(config)
	bool bDrawTrajectory = false;

	/** Used to show the traced pose search history value */
	UPROPERTY(config)
	bool bDrawHistory = false;

	/** Bool used to break down channels and display a complete picture of the weights that determined the final pose from Motion Matching algorithm */
	UPROPERTY(config)
	bool bIsVerbose = false;

	// Pose Candidates options

	/* Bool used to show all poses from display */
	UPROPERTY(config)
	bool bShowAllPoses = false;

	/* Bool used to show only the best pose of every asset */
	UPROPERTY(config)
	bool bShowOnlyBestAssetPose = false;

	/* Bool used to hide invalid poses from display */
	UPROPERTY(config)
	bool bHideInvalidPoses = false;

	/* Bool used to use FilterText as regex filter*/
	UPROPERTY(config)
	bool bUseRegex = false;
};