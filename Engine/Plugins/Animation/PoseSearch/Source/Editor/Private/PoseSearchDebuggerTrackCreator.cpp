// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDebuggerTrackCreator.h"

#include "IGameplayProvider.h"
#include "IAnimationProvider.h"
#include "PoseSearchDebugger.h"
#include "Trace/PoseSearchTraceProvider.h"

#define LOCTEXT_NAMESPACE "PoseSearchDebugger"

// FDebuggerTrackCreator
///////////////////////////////////////////////////

namespace UE::PoseSearch
{

FName FDebuggerTrackCreator::GetTargetTypeNameInternal() const
{
	return UObject::StaticClass()->GetFName();
}

void FDebuggerTrackCreator::GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const
{
	Types.Add({ GetNameInternal(), LOCTEXT("Pose Search", "Pose Search") });
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FDebuggerTrackCreator::CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	return MakeShared<UE::PoseSearch::FDebuggerTrack>(InObjectId.GetMainId());
}

bool FDebuggerTrackCreator::HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PoseSearchDebugger::HasDebugInfoInternal);
	// Get provider and validate
	const TraceServices::IAnalysisSession* Session = IRewindDebugger::Instance()->GetAnalysisSession();
	const FTraceProvider* PoseSearchProvider = Session->ReadProvider<FTraceProvider>(FTraceProvider::ProviderName);
	if (!PoseSearchProvider)
	{
		return false;
	}

	bool bHasData = false;

	TraceServices::FProviderReadScopeLock ProviderReadScope(*PoseSearchProvider);
	PoseSearchProvider->EnumerateMotionMatchingStateTimelines(InObjectId.GetMainId(), [&bHasData](const FTraceProvider::FMotionMatchingStateTimeline& InTimeline)
	{
		bHasData = true;
	});

	return bHasData;
}

} // namespace UE::PoseSearch

#undef LOCTEXT_NAMESPACE
