// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDebuggerView.h"
#include "Animation/TrajectoryTypes.h"
#include "Common/ProviderLock.h"
#include "Editor.h"
#include "IGameplayProvider.h"
#include "Modules/ModuleManager.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchDerivedData.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearchDatabaseEditor.h"
#include "PoseSearchDebugger.h"
#include "PoseSearchDebuggerDatabaseRowData.h"
#include "PoseSearchDebuggerDatabaseView.h"
#include "PoseSearchDebuggerReflection.h"
#include "PoseSearchDebuggerViewModel.h"
#include "PoseSearchEditor.h"
#include "PropertyEditorModule.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Trace/PoseSearchTraceProvider.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "PoseSearchDebugger"

namespace UE::PoseSearch
{

typedef TStringBuilder<256> FStringBuilder256;
void GenerateSearchName(const FTraceMotionMatchingStateMessage& MotionMatchingState, const IGameplayProvider* GameplayProvider, FStringBuilder256& StringBuilder)
{
	if (GameplayProvider)
	{
		bool bAddDatabasesSeparator = false;
		for (const FTraceMotionMatchingStateDatabaseEntry& DbEntry : MotionMatchingState.DatabaseEntries)
		{
			if (bAddDatabasesSeparator)
			{
				StringBuilder.Append(" - ");
			}

			const FObjectInfo& DatabaseObjectInfo = GameplayProvider->GetObjectInfo(DbEntry.DatabaseId);
			StringBuilder.Append(DatabaseObjectInfo.Name);
			bAddDatabasesSeparator = true;
		}

		if (MotionMatchingState.Roles.Num() > 1)
		{
			if (MotionMatchingState.Roles.Num() != MotionMatchingState.SkeletalMeshComponentIds.Num())
			{
				StringBuilder.Append("Error!");
			}
			else
			{
				StringBuilder.Append(" [");

				bool bAddRolesSeparator = false;

				for (int32 RoleIndex = 0; RoleIndex < MotionMatchingState.Roles.Num(); ++RoleIndex)
				{
					if (bAddRolesSeparator)
					{
						StringBuilder.Append(" - ");
					}

					StringBuilder.Append(MotionMatchingState.Roles[RoleIndex].ToString());

					StringBuilder.Append(": ");

					const FObjectInfo& SkeletalMeshComponentObjectInfo = GameplayProvider->GetObjectInfo(MotionMatchingState.SkeletalMeshComponentIds[RoleIndex]);
					const FObjectInfo& SkeletalMeshComponentOuterObjectInfo = GameplayProvider->GetObjectInfo(SkeletalMeshComponentObjectInfo.GetOuterId());

					StringBuilder.Append(SkeletalMeshComponentOuterObjectInfo.Name);

					bAddRolesSeparator = true;
				}

				StringBuilder.Append("]");
			}
		}
	}
	else
	{
		StringBuilder.Append("Error!");
	}
}

FText GenerateSearchName(const FTraceMotionMatchingStateMessage& MotionMatchingState, const IGameplayProvider* GameplayProvider)
{
	FStringBuilder256 StringBuilder;
	GenerateSearchName(MotionMatchingState, GameplayProvider, StringBuilder);
	return FText::FromString(StringBuilder.ToString());
}

FText GenerateSearchNameWithBestScore(const FTraceMotionMatchingStateMessage& MotionMatchingState, const IGameplayProvider* GameplayProvider)
{
	FStringBuilder256 StringBuilder;
	GenerateSearchName(MotionMatchingState, GameplayProvider, StringBuilder);

	float BestCost = UE_MAX_FLT;
	for (const FTraceMotionMatchingStateDatabaseEntry& DbEntry : MotionMatchingState.DatabaseEntries)
	{
		for (const FTraceMotionMatchingStatePoseEntry& PoseEntry : DbEntry.PoseEntries)
		{
			if (EnumHasAnyFlags(PoseEntry.PoseCandidateFlags, EPoseCandidateFlags::AnyValidMask))
			{
				BestCost = FMath::Min(PoseEntry.Cost, BestCost);
			}
		}
	}

	if (FMath::IsNearlyEqual(BestCost, UE_MAX_FLT))
	{
		StringBuilder.Appendf(TEXT(" [Invalid]"), BestCost);
	}
	else
	{
		StringBuilder.Appendf(TEXT(" [%.3f]"), BestCost);
	}
	return FText::FromString(StringBuilder.ToString());
}

class SDebuggerMessageBox : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDebuggerMessageBox) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FString& Message)
	{
		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(Message))
				.Font(FAppStyle::Get().GetFontStyle("DetailsView.CategoryFontStyle"))
			]
		];
	}
};

void SDebuggerDetailsView::Construct(const FArguments& InArgs)
{
	ParentDebuggerViewPtr = InArgs._Parent;

	// Add property editor (detail view) UObject to world root so that it persists when PIE is stopped
	Reflection = NewObject<UPoseSearchDebuggerReflection>();
	Reflection->AddToRoot();
	check(IsValid(Reflection));

	// @TODO: Convert this to a custom builder instead of of a standard details view
	// Load property module and create details view with our reflection UObject
	FPropertyEditorModule& PropPlugin = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
    DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;

	Details = PropPlugin.CreateDetailView(DetailsViewArgs);
	Details->SetObject(Reflection);
	
	ChildSlot
	[
		Details.ToSharedRef()
	];
}

void SDebuggerDetailsView::Update(const FTraceMotionMatchingStateMessage& State) const
{
	UpdateReflection(State);
}

SDebuggerDetailsView::~SDebuggerDetailsView()
{
	// Our previously instantiated object attached to root may be cleaned up at this point
	if (UObjectInitialized())
	{
		Reflection->RemoveFromRoot();
	}
}

void SDebuggerDetailsView::UpdateReflection(const FTraceMotionMatchingStateMessage& State) const
{
	check(Reflection);

	Reflection->InterruptMode = State.InterruptMode;
	Reflection->ElapsedPoseSearchTime = State.ElapsedPoseSearchTime;
	Reflection->AssetPlayerTime = State.AssetPlayerTime;
	Reflection->LastDeltaTime = State.DeltaTime;
	Reflection->SimLinearVelocity = State.SimLinearVelocity;
	Reflection->SimAngularVelocity = State.SimAngularVelocity;
	Reflection->AnimLinearVelocity = State.AnimLinearVelocity;
	Reflection->AnimAngularVelocity = State.AnimAngularVelocity;
	Reflection->Playrate = State.Playrate;
	Reflection->AnimLinearVelocityNoTimescale = State.AnimLinearVelocityNoTimescale;
	Reflection->AnimAngularVelocityNoTimescale = State.AnimAngularVelocityNoTimescale;
}

void SDebuggerView::Construct(const FArguments& InArgs, uint64 InAnimInstanceId, int32 InWantedSearchId)
{
	ViewModel = InArgs._ViewModel;
	OnViewClosed = InArgs._OnViewClosed;
	
	// Validate the existence of the passed getters
	check(ViewModel.IsBound())
	check(OnViewClosed.IsBound());
	
	AnimInstanceId = InAnimInstanceId;
	WantedSearchId = InWantedSearchId;
	SelectedSearchId = InWantedSearchId;

	ChildSlot
	[
		SAssignNew(DebuggerView, SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		[
			SAssignNew(Switcher, SWidgetSwitcher)
			.WidgetIndex(this, &SDebuggerView::SelectView)

			// [0] Selection view before node selection is made
			+ SWidgetSwitcher::Slot()
			.Padding(40.0f)
			.HAlign(HAlign_Fill)
            .VAlign(VAlign_Fill)
			[
				SAssignNew(SelectionView, SVerticalBox)
			]

			// [1] Node selected; node debugger view
			+ SWidgetSwitcher::Slot()
			[
				GenerateNodeDebuggerView()
			]

			// [2] Occluding message box when stopped (no recording)
			+ SWidgetSwitcher::Slot()
			[
				SNew(SDebuggerMessageBox, "Record gameplay to begin debugging")
			]

			// [3] Occluding message box when recording
			+ SWidgetSwitcher::Slot()
			[
				SNew(SDebuggerMessageBox, "Recording...")
			]
			
			// [4] Occluding message box when there is no data for the selected MM node
			+ SWidgetSwitcher::Slot()
			[
				GenerateNoDataMessageView()
			]
		]
	];
}

void SDebuggerView::SetTimeMarker(double InTimeMarker)
{
	if (FDebugger::IsPIESimulating())
	{
		return;
	}

	TimeMarker = InTimeMarker;
}

void SDebuggerView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (FDebugger::IsPIESimulating())
	{
		return;
	}

	const UWorld* DebuggerWorld = FDebugger::GetWorld();
	check(DebuggerWorld);

	const bool bSameTime = FMath::Abs(TimeMarker - PreviousTimeMarker) < DOUBLE_SMALL_NUMBER;
	PreviousTimeMarker = TimeMarker;

	TSharedPtr<FDebuggerViewModel> Model = ViewModel.Get();
	check(Model.IsValid());

	// We haven't reached the update point yet
	if (CurrentConsecutiveFrames < ConsecutiveFramesUpdateThreshold)
	{
		// If we're on the same time marker, it is consecutive
		if (bSameTime)
		{
			++CurrentConsecutiveFrames;
		}
	}
	else
	{
		// New frame after having updated, reset consecutive frames count and start counting again
		if (!bSameTime)
		{
			CurrentConsecutiveFrames = 0;
			bUpdated = false;
		}
		// Haven't updated since passing through frame gate, update once
		else if (!bUpdated)
		{
			Model->OnUpdate();
			if (UpdateNodeSelection())
			{
				Model->OnUpdateSearchSelection(SelectedSearchId);
				UpdateViews();
			}
			bUpdated = true;
		}
	}

	// Draw features
	if (const FTraceMotionMatchingStateMessage* State = Model->GetMotionMatchingState())
	{
		FRoleToIndex RoleToIndex;
		TArray<FChooserEvaluationContext, TInlineAllocator<PreallocatedRolesNum>> AnimContextsData;
		TArray<FChooserEvaluationContext*, TInlineAllocator<PreallocatedRolesNum>> AnimContexts;
		TArray<const IPoseHistory*, TInlineAllocator<PreallocatedRolesNum>> PoseHistories;
		UWorld* World = nullptr;

		const int32 NumRoles = State->Roles.Num();

		RoleToIndex.Reserve(NumRoles);
		AnimContextsData.SetNum(NumRoles);
		AnimContexts.SetNum(NumRoles);
		PoseHistories.SetNum(NumRoles);

		for (int32 RoleIndex = 0; RoleIndex < NumRoles; ++RoleIndex)
		{
			const uint64 ActorSkeletalMeshComponentId = State->SkeletalMeshComponentIds[RoleIndex];
			if (const TWeakObjectPtr<AActor>* ActorPtr = Model->GetDebugDrawActors().Find(ActorSkeletalMeshComponentId))
			{
				if (ActorPtr->IsValid())
				{
					const FRole& Role = State->Roles[RoleIndex];

					for (UActorComponent* ActorComponent : (*ActorPtr)->GetInstanceComponents())
					{
						if (UPoseSearchMeshComponent* PoseSearchMeshComponent = Cast<UPoseSearchMeshComponent>(ActorComponent))
						{
							World = PoseSearchMeshComponent->GetWorld();
							RoleToIndex.Add(Role) = RoleIndex;
							AnimContextsData[RoleIndex].AddObjectParam(PoseSearchMeshComponent);
							AnimContexts[RoleIndex] = &AnimContextsData[RoleIndex];
							PoseHistories[RoleIndex] = &State->PoseHistories[RoleIndex];
							break;
						}
					}
				}
			}
		}

		// checking if all roles have been resolved properly
		if (RoleToIndex.Num() != NumRoles)
		{
			return;
		}

		// Draw world space trajectory
#if ENABLE_ANIM_DEBUG
		const bool bDrawTrajectory = Model->GetDrawTrajectory();
		const bool bDrawHistory = Model->GetDrawHistory();

		if (bDrawTrajectory || bDrawHistory)
		{
			for (int32 RoleIndex = 0; RoleIndex < NumRoles; ++RoleIndex)
			{
				const FTransformTrajectory& Trajectory = State->PoseHistories[RoleIndex].Trajectory;
				
				if (bDrawTrajectory)
				{
					UTransformTrajectoryBlueprintLibrary::DebugDrawTrajectory(Trajectory, World);
				}

				if (bDrawHistory)
				{
					State->PoseHistories[RoleIndex].DebugDraw(World, FColor::Red);
				}
			}
		}
#endif
		
		FAsyncPoseSearchDatabasesManagementCache DatabasesManagementCache;

		// Draw selected poses
		int32 NumOfRemainingRowsToDraw = 250;
		TSet<const UPoseSearchDatabase*, DefaultKeyFuncs<const UPoseSearchDatabase*>, TInlineSetAllocator<8>> SelectedRowsDatabases;
		TMap<const UPoseSearchDatabase*, FDebugDrawParams, TInlineSetAllocator<8>> CachedDebugDrawParams;
		
		auto DrawFeatureVectors = [&DatabasesManagementCache, &SelectedRowsDatabases, &CachedDebugDrawParams, &AnimContexts, &PoseHistories, &RoleToIndex, &NumOfRemainingRowsToDraw](TArray<TSharedRef<FDebuggerDatabaseRowData>> Rows)
			{
				for (const TSharedRef<FDebuggerDatabaseRowData>& Row : Rows)
				{
					if (NumOfRemainingRowsToDraw == 0)
					{
						break;
					}

					const UPoseSearchDatabase* RowDatabase = Row->SharedData->SourceDatabase.Get();
					if (EAsyncBuildIndexResult::Success == DatabasesManagementCache(RowDatabase))
					{
						SelectedRowsDatabases.Add(RowDatabase);
						FDebugDrawParams* DrawParams = CachedDebugDrawParams.Find(RowDatabase);
						if (!DrawParams)
						{
							DrawParams = &CachedDebugDrawParams.Add(RowDatabase);
							DrawParams->Init(AnimContexts, PoseHistories, RoleToIndex, RowDatabase, FDebugDrawParams::EDrawContext::DrawCandidate);
						}

						DrawParams->DrawFeatureVector(Row->PoseIdx);
						--NumOfRemainingRowsToDraw;
					}
				}
			};

		// @todo: avoid returning arrays by copy here
		DrawFeatureVectors(DatabaseView->GetActiveRow()->GetSelectedItems());
		DrawFeatureVectors(DatabaseView->GetContinuingPoseRow()->GetSelectedItems());
		DrawFeatureVectors(DatabaseView->GetDatabaseRows()->GetSelectedItems());

		// Draw queries: either the one for the selected row databases or ALL of them
		if (Model->GetDrawQuery())
		{
			// resetting CachedDebugDrawParams since we're now drawing the queries, that have a different drawing context
			CachedDebugDrawParams.Reset();

			for (const FTraceMotionMatchingStateDatabaseEntry& DbEntry : State->DatabaseEntries)
			{
				const UPoseSearchDatabase* Database = ResolveDatabaseFromId(DbEntry.DatabaseId);
				if (Database &&
					(SelectedRowsDatabases.IsEmpty() || SelectedRowsDatabases.Contains(Database)) &&
					EAsyncBuildIndexResult::Success == DatabasesManagementCache(Database) &&
					DbEntry.QueryVector.Num() == Database->Schema->SchemaCardinality)
				{
					FDebugDrawParams* DrawParams = CachedDebugDrawParams.Find(Database);
					if (!DrawParams)
					{
						DrawParams = &CachedDebugDrawParams.Add(Database);
						DrawParams->Init(AnimContexts, PoseHistories, RoleToIndex, Database, FDebugDrawParams::EDrawContext::DrawQuery);
					}

					DrawParams->DrawFeatureVector(DbEntry.QueryVector);
				}
			}
		}
	}

	// synchronizing the model DrawQuery state with all the open PoseSearchDatabaseEditor(s)
	const bool bDrawQuery = Model->GetDrawQuery();
	if (UAssetEditorSubsystem* AssetEditorSS = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
	{
		TArray<UObject*> EditedAssets = AssetEditorSS->GetAllEditedAssets();
		for (UObject* EditedAsset : EditedAssets)
		{
			if (Cast<UPoseSearchDatabase>(EditedAsset))
			{
				if (IAssetEditorInstance* Editor = AssetEditorSS->FindEditorForAsset(EditedAsset, false))
				{
					if (Editor->GetEditorName() == FName("PoseSearchDatabaseEditor"))
					{
						FDatabaseEditor* DatabaseEditor = static_cast<FDatabaseEditor*>(Editor);
						DatabaseEditor->SetDrawQueryVector(bDrawQuery);
					}
				}
			}
		}
	}
}

bool SDebuggerView::UpdateNodeSelection()
{
	TSharedPtr<FDebuggerViewModel> Model = ViewModel.Get();
	check(Model.IsValid());

	const TArray<FTraceMotionMatchingStateMessage>& MotionMatchingStates = Model->GetMotionMatchingStates();

	// Update selection view if no node selected
	if (SelectedSearchId != InvalidSearchId)
	{
		if (!MotionMatchingStates.IsEmpty())
		{
			// making sure SelectedSearchId is still valid. if not, let's pick the first available MotionMatchingStates
			for (const FTraceMotionMatchingStateMessage& MotionMatchingState : MotionMatchingStates)
			{
				if (MotionMatchingState.GetSearchId() == SelectedSearchId)
				{
					return true;
				}
			}

			if (WantedSearchId == InvalidSearchId)
			{
				// SelectedSearchId is not valid, and since there's no WantedSearchId, by specifically double clicking the 
				// search track instead of selecting "Pose Search" track, we reassign SelectedSearchId to the first valid SearchId
				SelectedSearchId = MotionMatchingStates[0].GetSearchId();
			}
		}

		return true;
	}
	
	// Only one active state, bypass selection view
	if (MotionMatchingStates.Num() == 1)
	{
		SelectedSearchId = MotionMatchingStates[0].GetSearchId();
		return true;
	}

	// Create selection view with buttons for each node, displaying the database name
	SelectionView->ClearChildren();

	if (!MotionMatchingStates.IsEmpty())
	{
		IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
		const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
		check(AnalysisSession);

		// Cache display text under a tight lock scope
		struct FCachedSearchEntry
		{
			FText DisplayText;
			int32 SearchId;
		};

		TArray<FCachedSearchEntry> CachedEntries;
		{
			const IGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<IGameplayProvider>("GameplayProvider");
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
			for (const FTraceMotionMatchingStateMessage& MotionMatchingState : MotionMatchingStates)
			{
				CachedEntries.Add({ GenerateSearchNameWithBestScore(MotionMatchingState, GameplayProvider), MotionMatchingState.GetSearchId() });
			}
		}

		// Build widgets outside the lock
		for (const FCachedSearchEntry& Entry : CachedEntries)
		{
			Model->OnUpdateSearchSelection(Entry.SearchId);
			SelectionView->AddSlot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.Padding(10.0f)
				[
					SNew(SButton)
						.Text(Entry.DisplayText)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.ContentPadding(10.0f)
						.OnClicked(this, &SDebuggerView::OnUpdateSearchSelection, Entry.SearchId)
				];
		}
	}
	return false;
}

void SDebuggerView::UpdateViews() const
{
	if (const FTraceMotionMatchingStateMessage* State = ViewModel.Get()->GetMotionMatchingState())
	{
		DatabaseView->Update(*State);
		DetailsView->Update(*State);
	}
}

TArray<TSharedRef<FDebuggerDatabaseRowData>> SDebuggerView::GetSelectedDatabaseRows() const
{
	return DatabaseView->GetDatabaseRows()->GetSelectedItems();
}

int32 SDebuggerView::SelectView() const
{
	// Currently recording
	if (FDebugger::IsPIESimulating() && FDebugger::IsRecording())
	{
		return RecordingMsg;
	}

	// Data has not been recorded yet
	if (FDebugger::GetRecordingDuration() < DOUBLE_SMALL_NUMBER)
	{
		return StoppedMsg;
	}

	const TSharedPtr<FDebuggerViewModel> Model = ViewModel.Get();
	check(Model.IsValid());

	const bool bNoActiveNodes = Model->GetNodesNum() == 0;
	const bool bNodeSelectedWithoutData = SelectedSearchId != InvalidSearchId && Model->GetMotionMatchingState() == nullptr;

	// No active nodes, or node selected has no data
	if (bNoActiveNodes || bNodeSelectedWithoutData)
    {
    	return NoDataMsg;
    }

	// Node not selected yet, showcase selection view
	if (SelectedSearchId == InvalidSearchId)
	{
		return Selection;
	}

	// Standard debugger view
	return Debugger;
}

void SDebuggerView::OnPoseSelectionChanged(const UPoseSearchDatabase* Database, int32 DbPoseIdx, float Time)
{
	const TSharedPtr<FDebuggerViewModel> Model = ViewModel.Get();
	check(Model.IsValid());

	if (const FTraceMotionMatchingStateMessage* State = Model->GetMotionMatchingState())
	{
		DetailsView->Update(*State);
	}
}

FReply SDebuggerView::OnUpdateSearchSelection(int32 InSelectedSearchId)
{
	// InvalidSearchId will backtrack to selection view
	SelectedSearchId = InSelectedSearchId;
	bUpdated = false;
	return FReply::Handled();
}

TSharedRef<SWidget> SDebuggerView::GenerateNoDataMessageView()
{
	TSharedRef<SWidget> ReturnButtonView = GenerateReturnButtonView();
	ReturnButtonView->SetVisibility(TAttribute<EVisibility>::CreateLambda([this]() -> EVisibility
	{
		// Hide the return button for the no data message if we have no nodes at all
		return ViewModel.Get()->GetNodesNum() > 0 ? EVisibility::Visible : EVisibility::Hidden;
	}));
	
	return 
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		[
			SNew(SDebuggerMessageBox, "No recorded data available for the selected frame")
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(20.0f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			ReturnButtonView
		];
}

TSharedRef<SHorizontalBox> SDebuggerView::GenerateReturnButtonView()
{
	return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.Padding(10, 5, 0, 0)
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.Visibility_Lambda([this] { return ViewModel.Get()->GetNodesNum() > 1 ? EVisibility::Visible : EVisibility::Hidden; })
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ContentPadding( FMargin(1, 0) )
			.OnClicked(this, &SDebuggerView::OnUpdateSearchSelection, InvalidSearchId)
			// Contents of button, icon then text
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.CircleArrowLeft"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ReturnToSearchSelectionLabel", "Return to Search Selection"))
					.Justification(ETextJustify::Center)
				]
			]
		]
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Left)
		.Padding(64, 5, 0, 0)
		.AutoWidth()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 5, 0, 0)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]
				{
					return ViewModel.Get()->GetDrawQuery() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; 
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
				{
					ViewModel.Get()->SetDrawQuery(State == ECheckBoxState::Checked);
				})
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PoseSearchDebuggerDrawQuery", "Draw Query"))
				]
			]
		]
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Left)
		.Padding(64, 5, 0, 0)
		.AutoWidth()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 5, 0, 0)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]
				{
					return ViewModel.Get()->GetDrawTrajectory() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; 
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
				{
					ViewModel.Get()->SetDrawTrajectory(State == ECheckBoxState::Checked);
				})
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PoseSearchDebuggerDrawTrajectory", "Draw Trajectory"))
				]
			]
		]
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Left)
		.Padding(64, 5, 0, 0)
		.AutoWidth()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 5, 0, 0)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]
				{
					return ViewModel.Get()->GetDrawHistory() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; 
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
				{
					ViewModel.Get()->SetDrawHistory(State == ECheckBoxState::Checked);
				})
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PoseSearchDebuggerDrawHistory", "Draw History"))
					.ToolTipText(LOCTEXT("PoseSearchDebuggerDrawHistory_ToolTip", "Draws recorded bones paths in red. Additionally draws the previous frame directions of the root bone in White, the trajectory in Black, and the mesh at the time of the search in Turquoise"))
				]
			]
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Left)
		.Padding(64, 5, 0, 0)
		.AutoWidth()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 5, 0, 0)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]
				{
					return ViewModel.Get()->IsVerbose() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
				{
					ViewModel.Get()->SetVerbose(State == ECheckBoxState::Checked);
					UpdateViews();
				})
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PoseSearchDebuggerShowVerbose", "Channels Breakdown"))
				]
			]
		];
}

TSharedRef<SWidget> SDebuggerView::GenerateNodeDebuggerView()
{
	return 
		SNew(SSplitter)
		.Orientation(Orient_Vertical)
		.ResizeMode(ESplitterResizeMode::Fill)
	
		// Database view
		+ SSplitter::Slot()
		.Value(0.8f)
		[
			SNew(SVerticalBox)
			
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				GenerateReturnButtonView()
			]
			
			+ SVerticalBox::Slot()
			[
				SAssignNew(DatabaseView, SDebuggerDatabaseView)
				.Parent(SharedThis(this))
				.OnPoseSelectionChanged(this, &SDebuggerView::OnPoseSelectionChanged)
			]
		]

		// Details panel view
		+ SSplitter::Slot()
		.Value(0.2f)
		[
			SAssignNew(DetailsView, SDebuggerDetailsView)
			.Parent(SharedThis(this))
		];
}

FName SDebuggerView::GetName() const
{
	static const FName DebuggerName("PoseSearchDebugger");
	return DebuggerName;
}

uint64 SDebuggerView::GetObjectId() const
{
	return AnimInstanceId;
}

SDebuggerView::~SDebuggerView()
{
	OnViewClosed.Execute(AnimInstanceId);
}

} // namespace UE::PoseSearch

#undef LOCTEXT_NAMESPACE
