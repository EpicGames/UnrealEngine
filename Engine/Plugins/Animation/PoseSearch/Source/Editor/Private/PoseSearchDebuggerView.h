// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRewindDebuggerView.h"
#include "PoseSearch/PoseSearchDefines.h"

class IDetailsView;
class IGameplayProvider;
class SHorizontalBox;
class SSplitter;
class SVerticalBox;
class SWidgetSwitcher;
class UPoseSearchDatabase;
class UPoseSearchDebuggerReflection;

namespace UE::PoseSearch
{

class FDebuggerDatabaseRowData;
class FDebuggerViewModel;
struct FTraceMotionMatchingStateMessage;
class SDebuggerDatabaseView;

/**
 * Details panel view widget of the PoseSearch debugger
 */
class SDebuggerDetailsView : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDebuggerDetailsView) {}
		SLATE_ARGUMENT(TWeakPtr<class SDebuggerView>, Parent)
	SLATE_END_ARGS()

	SDebuggerDetailsView() = default;
	virtual ~SDebuggerDetailsView() override;

	void Construct(const FArguments& InArgs);
	void Update(const FTraceMotionMatchingStateMessage& State) const;

	/** Get a const version of our reflection object */
	const TObjectPtr<UPoseSearchDebuggerReflection> GetReflection() const { return Reflection; }

private:
	/** Update our details view object with new state information */
	void UpdateReflection(const FTraceMotionMatchingStateMessage& State) const;

	TWeakPtr<SDebuggerView> ParentDebuggerViewPtr;

	/** Details widget constructed for the MM node */
	TSharedPtr<IDetailsView> Details;

	/** Last updated reflection data relative to MM state. This is not gonna be garbage collected because of AddToRoot during Construct */
	TObjectPtr<UPoseSearchDebuggerReflection> Reflection;
};

/** Callback to relay closing of the view to destroy the debugger instance */
DECLARE_DELEGATE_OneParam(FOnViewClosed, uint64 AnimInstanceId);

/**
 * Entire view of the PoseSearch debugger, containing all sub-widgets
 */
class SDebuggerView : public IRewindDebuggerView
{
public:
	SLATE_BEGIN_ARGS(SDebuggerView){}
		SLATE_ATTRIBUTE(TSharedPtr<FDebuggerViewModel>, ViewModel)
		SLATE_EVENT(FOnViewClosed, OnViewClosed)
	SLATE_END_ARGS()

	SDebuggerView() = default;
    virtual ~SDebuggerView() override;

	void Construct(const FArguments& InArgs, uint64 InAnimInstanceId, int32 InWantedSearchId);
	virtual void SetTimeMarker(double InTimeMarker) override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FName GetName() const override;
	virtual uint64 GetObjectId() const override;

	TSharedPtr<FDebuggerViewModel> GetViewModel() const { return ViewModel.Get(); }
	TArray<TSharedRef<FDebuggerDatabaseRowData>> GetSelectedDatabaseRows() const;

private:
	/** Check if a node selection was made, true if a node is selected */
	bool UpdateNodeSelection();

	/** Update the database and details views */
	void UpdateViews() const;

	/** Returns an int32 appropriate to the index of our widget selector */
	int32 SelectView() const;

	/** Callback when a button in the selection view is clicked */
	FReply OnUpdateSearchSelection(int32 InSelectedSearchId);

	void OnPoseSelectionChanged(const UPoseSearchDatabase* Database, int32 PoseIdx, float Time);

	/** Generates the message view relaying that there is no data */
	TSharedRef<SWidget> GenerateNoDataMessageView();

	/** Generates the return button to go back to the selection mode */
	TSharedRef<SHorizontalBox> GenerateReturnButtonView();

	/** Generates the entire node debugger widget, including database and details view */
	TSharedRef<SWidget> GenerateNodeDebuggerView();

	/** Pointer to the debugger instance / model for this view */
	TAttribute<TSharedPtr<FDebuggerViewModel>> ViewModel;
	
	/** Destroy the debugger instanced when closed */
	FOnViewClosed OnViewClosed;

	/** Active User selected SearchId*/
	int32 WantedSearchId = InvalidSearchId;
	
	/** Active SearchId being debugged */
	int32 SelectedSearchId = InvalidSearchId;

	/** Database view of the motion matching node */
	TSharedPtr<SDebuggerDatabaseView> DatabaseView;
	
	/** Details panel for introspecting the motion matching node */
	TSharedPtr<SDebuggerDetailsView> DetailsView;
	
	/** Node debugger view hosts the above two views */
	TSharedPtr<SSplitter> NodeDebuggerView;

	/** Selection view before node is selected */
	TSharedPtr<SVerticalBox> SelectionView;
	
	/** Gray box occluding the debugger view when simulating */
	TSharedPtr<SVerticalBox> SimulatingView;

	/** Used to switch between views in the switcher, int32 maps to index in the SWidgetSwitcher */
	enum ESwitcherViewType : int32
	{
		Selection = 0,
		Debugger = 1,
		StoppedMsg = 2,
		RecordingMsg = 3,
		NoDataMsg = 4
	} SwitcherViewType = StoppedMsg;
	
	/** Contains all the above, switches between them depending on context */
	TSharedPtr<SWidgetSwitcher> Switcher;

	/** Contains the switcher, the entire debugger view */
	TSharedPtr<SVerticalBox> DebuggerView;

	// @todo: rename it to AnimContextId
	/** AnimInstance this view was created for */
	uint64 AnimInstanceId = 0;

	/** Current position of the time marker */
	double TimeMarker = -1.0;

	/** Previous position of the time marker */
	double PreviousTimeMarker = -1.0;

	/** Tracks if the current time has been updated yet (delayed) */
	bool bUpdated = false;

	/** Tracks number of consecutive frames, once it reaches threshold it will update the view */
	int32 CurrentConsecutiveFrames = 0;

	/** Once the frame count has reached this value, an update will trigger for the view */
	static constexpr int32 ConsecutiveFramesUpdateThreshold = 10;
};

FText GenerateSearchName(const FTraceMotionMatchingStateMessage& MotionMatchingState, const IGameplayProvider* GameplayProvider);

} // namespace UE::PoseSearch
