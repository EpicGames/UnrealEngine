// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchDebuggerViewModel.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimSequence.h"
#include "Animation/MirrorDataTable.h"
#include "Common/ProviderLock.h"
#include "Engine/SkeletalMesh.h"
#include "IAnimationProvider.h"
#include "IGameplayProvider.h"
#include "StructUtils/InstancedStruct.h"
#include "IRewindDebugger.h"
#include "PoseSearchDebuggerSettings.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchDerivedData.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "Trace/PoseSearchTraceProvider.h"
#include "TraceServices/Model/Frames.h"

namespace UE::PoseSearch
{

FDebuggerViewModel::FDebuggerViewModel(uint64 InAnimInstanceId)
	: AnimInstanceId(InAnimInstanceId)
{
}

FDebuggerViewModel::~FDebuggerViewModel()
{
	for (TPair<uint64, TWeakObjectPtr<AActor>>& DebugDrawActorPair : DebugDrawActors)
	{
		if (DebugDrawActorPair.Value.IsValid())
		{
			DebugDrawActorPair.Value->Destroy();
		}
	}
}

const FTraceMotionMatchingStateMessage* FDebuggerViewModel::GetMotionMatchingState() const
{
	if (MotionMatchingStates.IsValidIndex(ActiveMotionMatchingStateIdx))
	{
		return &MotionMatchingStates[ActiveMotionMatchingStateIdx];
	}
	return nullptr;
}

int32 FDebuggerViewModel::GetNodesNum() const
{
	return MotionMatchingStates.Num();
}

void FDebuggerViewModel::OnUpdate()
{
	MotionMatchingStates.Reset();

	// Get provider and validate
	const TraceServices::IAnalysisSession* Session = RewindDebugger.Get()->GetAnalysisSession();

	const FTraceProvider* PoseSearchProvider = Session->ReadProvider<FTraceProvider>(FTraceProvider::ProviderName);
	const IAnimationProvider* AnimationProvider = Session->ReadProvider<IAnimationProvider>("AnimationProvider");
	const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider");
	if (!(PoseSearchProvider && AnimationProvider && GameplayProvider))
	{
		return;
	}
	const double TraceTime = RewindDebugger.Get()->CurrentTraceTime();

	bool bFrameFound;
	TraceServices::FFrame Frame;
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session);
		const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*Session);
		bFrameFound = ReadFrameProvider(*Session).GetFrameFromTime(TraceFrameType_Game, TraceTime, Frame);
	}

	// Phase 1: Read motion matching states under lock
	if (bFrameFound)
	{
		TraceServices::FProviderReadScopeLock PoseSearchProviderReadScope(*PoseSearchProvider);
		PoseSearchProvider->EnumerateMotionMatchingStateTimelines(AnimInstanceId, [&Frame, this](const FTraceProvider::FMotionMatchingStateTimeline& InTimeline)
		{
			const FTraceMotionMatchingStateMessage* Message = nullptr;

			InTimeline.EnumerateEvents(Frame.StartTime, Frame.EndTime, [&Message](double InStartTime, double InEndTime, const FTraceMotionMatchingStateMessage& InMessage)
			{
				Message = &InMessage;
				return TraceServices::EEventEnumerate::Stop;
			});
			if (Message)
			{
				check(Message->Roles.Num() == Message->SkeletalMeshComponentIds.Num());
				MotionMatchingStates.Add(*Message);
			}
		});
	}

	// Phase 2: Actor spawning/cleanup outside any lock
	TSet<uint64> UsedDebugDrawActorIds;
	for (const FTraceMotionMatchingStateMessage& MotionMatchingState : MotionMatchingStates)
	{
		for (uint64 SkeletalMeshComponentId : MotionMatchingState.SkeletalMeshComponentIds)
		{
			UsedDebugDrawActorIds.Add(SkeletalMeshComponentId);

			bool bNeedToAdd = true;
			if (TWeakObjectPtr<AActor>* DebugDrawActor = DebugDrawActors.Find(SkeletalMeshComponentId))
			{
				if (DebugDrawActor->IsValid())
				{
					bNeedToAdd = false;
				}
			}

			if (bNeedToAdd)
			{
				UWorld* World = RewindDebugger.Get()->GetWorldToVisualize();
				FActorSpawnParameters ActorSpawnParameters;
				ActorSpawnParameters.bHideFromSceneOutliner = false;
				ActorSpawnParameters.ObjectFlags |= RF_Transient;
				AActor* DebugDrawActor = World->SpawnActor<AActor>(ActorSpawnParameters);
				DebugDrawActor->SetActorLabel(TEXT("PoseSearch"));
				UPoseSearchMeshComponent* DebugDrawMeshComponent = NewObject<UPoseSearchMeshComponent>(DebugDrawActor);
				DebugDrawActor->AddInstanceComponent(DebugDrawMeshComponent);
				DebugDrawMeshComponent->RegisterComponentWithWorld(World);

				DebugDrawActors.Add(SkeletalMeshComponentId) = DebugDrawActor;
			}
		}
	}

	// cleaning up the no longer used actors (with a SkeletalMeshComponentId not in UsedDebugDrawActorIds)
	for (auto DebugDrawActorsIt = DebugDrawActors.CreateIterator(); DebugDrawActorsIt; ++DebugDrawActorsIt)
	{
		if (!UsedDebugDrawActorIds.Find(DebugDrawActorsIt.Key()))
		{
			if (DebugDrawActorsIt.Value().IsValid())
			{
				DebugDrawActorsIt.Value()->Destroy();
			}
			DebugDrawActorsIt.RemoveCurrent();
		}
	}

	/** No active motion matching state as no messages were read */
	if (MotionMatchingStates.IsEmpty())
	{
		return;
	}

	// Phase 3: Read pose data under lock, cache mesh path names, then apply outside lock
	struct FCachedPoseData
	{
		UPoseSearchMeshComponent* MeshComponent;
		FString MeshPathName;
		FTransform ComponentWorldTransform;
		TArray<FTransform> ComponentSpaceTransforms;
	};
	TArray<FCachedPoseData> CachedPoseEntries;

	{
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
		TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);

		for (TPair<uint64, TWeakObjectPtr<AActor>>& DebugDrawActorPair : DebugDrawActors)
		{
			const uint64 SkeletalMeshComponentId = DebugDrawActorPair.Key;
			if (AActor* DebugDrawActor = DebugDrawActorPair.Value.Get())
			{
				for (UActorComponent* ActorComponent : DebugDrawActor->GetInstanceComponents())
				{
					if (UPoseSearchMeshComponent* PoseSearchMeshComponent = Cast<UPoseSearchMeshComponent>(ActorComponent))
					{
						AnimationProvider->ReadSkeletalMeshPoseTimeline(SkeletalMeshComponentId, [&Frame, AnimationProvider, GameplayProvider, PoseSearchMeshComponent, &CachedPoseEntries](const IAnimationProvider::SkeletalMeshPoseTimeline& TimelineData, bool bHasCurves)
							{
								TimelineData.EnumerateEvents(Frame.StartTime, Frame.EndTime, [AnimationProvider, GameplayProvider, PoseSearchMeshComponent, &CachedPoseEntries](double InStartTime, double InEndTime, uint32 InDepth, const FSkeletalMeshPoseMessage& PoseMessage) -> TraceServices::EEventEnumerate
									{
										const FSkeletalMeshInfo* SkeletalMeshInfo = AnimationProvider->FindSkeletalMeshInfo(PoseMessage.MeshId);
										const FObjectInfo* SkeletalMeshObjectInfo = GameplayProvider->FindObjectInfo(PoseMessage.MeshId);
										if (!SkeletalMeshInfo || !SkeletalMeshObjectInfo)
										{
											return TraceServices::EEventEnumerate::Stop;
										}

										FCachedPoseData& CachedEntry = CachedPoseEntries.AddDefaulted_GetRef();
										CachedEntry.MeshComponent = PoseSearchMeshComponent;
										CachedEntry.MeshPathName = SkeletalMeshObjectInfo->PathName;
										AnimationProvider->GetSkeletalMeshComponentSpacePose(PoseMessage, *SkeletalMeshInfo, CachedEntry.ComponentWorldTransform, CachedEntry.ComponentSpaceTransforms);

										check(CachedEntry.ComponentWorldTransform.Equals(PoseMessage.ComponentToWorld));

										return TraceServices::EEventEnumerate::Stop;
									});
							});
						break;
					}
				}
			}
		}
	}

	// Apply cached pose data outside the lock (includes LoadSynchronous)
	for (FCachedPoseData& CachedEntry : CachedPoseEntries)
	{
		USkeletalMesh* SkeletalMesh = TSoftObjectPtr<USkeletalMesh>(FSoftObjectPath(CachedEntry.MeshPathName)).LoadSynchronous();
		if (SkeletalMesh)
		{
			CachedEntry.MeshComponent->SetSkinnedAssetAndUpdate(SkeletalMesh, true);
		}
		TArray<FTransform>& ComponentSpaceTransforms = CachedEntry.MeshComponent->GetEditableComponentSpaceTransforms();
		ComponentSpaceTransforms = MoveTemp(CachedEntry.ComponentSpaceTransforms);
		CachedEntry.MeshComponent->Initialize(CachedEntry.ComponentWorldTransform);
	}
}

void FDebuggerViewModel::OnUpdateSearchSelection(int32 InSearchId)
{
	if (InSearchId != InvalidSearchId)
	{
		// Find node in all motion matching states this frame
		ActiveMotionMatchingStateIdx = INDEX_NONE;
		const int32 NodesNum = MotionMatchingStates.Num();
		for (int32 i = 0; i < NodesNum; ++i)
		{
			if (MotionMatchingStates[i].GetSearchId() == InSearchId)
			{
				ActiveMotionMatchingStateIdx = i;
				break;
			}
		}
	}
}

void FDebuggerViewModel::SetVerbose(bool bVerbose)
{
	UPoseSearchDebuggerConfig::Get().bIsVerbose = bVerbose;
}

bool FDebuggerViewModel::IsVerbose() const
{
	return UPoseSearchDebuggerConfig::Get().bIsVerbose;
}

void FDebuggerViewModel::SetDrawQuery(bool bInDrawQuery)
{
	UPoseSearchDebuggerConfig::Get().bDrawQuery = bInDrawQuery;
}

bool FDebuggerViewModel::GetDrawQuery() const
{
	return UPoseSearchDebuggerConfig::Get().bDrawQuery;
}

void FDebuggerViewModel::SetDrawTrajectory(bool bInDrawTrajectory)
{
	UPoseSearchDebuggerConfig::Get().bDrawTrajectory = bInDrawTrajectory;
}

bool FDebuggerViewModel::GetDrawTrajectory() const
{
	return UPoseSearchDebuggerConfig::Get().bDrawTrajectory;
}

void FDebuggerViewModel::SetDrawHistory(bool bInDrawHistory)
{
	UPoseSearchDebuggerConfig::Get().bDrawHistory = bInDrawHistory;
}

bool FDebuggerViewModel::GetDrawHistory() const
{
	return UPoseSearchDebuggerConfig::Get().bDrawHistory;
}
	
} // namespace UE::PoseSearch
