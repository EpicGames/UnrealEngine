// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "PoseSearchMeshComponent.h"
#include "PoseSearch/PoseSearchMirrorDataCache.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

struct FInstancedStruct;
class IRewindDebugger;
class UPoseSearchDatabase;

namespace UE::PoseSearch
{

struct FTraceMotionMatchingStateMessage;

class FDebuggerViewModel : public TSharedFromThis<FDebuggerViewModel>
{
public:
	explicit FDebuggerViewModel(uint64 InAnimInstanceId);
	virtual ~FDebuggerViewModel();

	// Used for view callbacks
    const FTraceMotionMatchingStateMessage* GetMotionMatchingState() const;
	const TMap<uint64, TWeakObjectPtr<AActor>>& GetDebugDrawActors() const { return DebugDrawActors; }
	const TArray<FTraceMotionMatchingStateMessage>& GetMotionMatchingStates() const { return MotionMatchingStates; }

	int32 GetNodesNum() const;

	/** Update motion matching states for frame */
	void OnUpdate();
	
	/** Updates active motion matching state based on node selection */
	void OnUpdateSearchSelection(int32 InSearchId);

	void SetVerbose(bool bVerbose);
	bool IsVerbose() const;

	void SetDrawQuery(bool bInDrawQuery);
	bool GetDrawQuery() const;

	void SetDrawTrajectory(bool bInDrawTrajectory);
	bool GetDrawTrajectory() const;

	void SetDrawHistory(bool bInDrawHistory);
	bool GetDrawHistory() const;

private:

	/** List of all updated motion matching states per node */
	TArray<FTraceMotionMatchingStateMessage> MotionMatchingStates;
	
	/** Currently active motion matching state index based on node selection in the view */
	int32 ActiveMotionMatchingStateIdx = INDEX_NONE;

	/** Pointer to the active rewind debugger in the scene */
	TAttribute<const IRewindDebugger*> RewindDebugger;

	// @todo: rename it to AnimContextId
	/** Anim Instance associated with this debugger instance */
	uint64 AnimInstanceId = 0;

	/** Actor object populated by the timeline mapped from the MotionMatchingStates.SkeletalMeshComponentIds. used as input for additional bone transforms to draw channels */
	TMap<uint64, TWeakObjectPtr<AActor>> DebugDrawActors;
	
	/** Limits some public API */
	friend class FDebugger;
};

} // namespace UE::PoseSearch
