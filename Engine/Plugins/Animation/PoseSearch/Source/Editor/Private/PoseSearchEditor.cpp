// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchEditor.h"
#include "PoseSearchCustomization.h"
#include "PoseSearchDebugger.h"
#include "PoseSearchDatabaseEdMode.h"
#include "PoseSearchDatabaseEditorCommands.h"
#include "PoseSearchInteractionAssetEditor.h"

#include "Animation/AnimSequence.h"
#include "AssetToolsModule.h"
#include "Editor.h"
#include "IAnimationEditor.h"
#include "IPersonaToolkit.h"
#include "IPersonaPreviewScene.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Subsystems/AssetEditorSubsystem.h"

#include "Trace/PoseSearchTraceAnalyzer.h"
#include "Trace/PoseSearchTraceModule.h"
#include "PoseSearchColumnEditor.h"
#include "PoseSearchDebuggerTrackCreator.h"

DEFINE_LOG_CATEGORY(LogPoseSearchEditor);

#define LOCTEXT_NAMESPACE "PoseSearchEditorModule"

namespace UE::PoseSearch
{

//////////////////////////////////////////////////////////////////////////
// FPoseSearchEditorModule
class FEditorModule : public IPoseSearchEditorModuleInterface
{
public:
	void StartupModule() override;
	void ShutdownModule() override;

private:
	TArray<class IConsoleObject*> ConsoleCommands;
	
	/** Creates the track for the Rewind Debugger */
	TSharedPtr<FDebuggerTrackCreator> DebuggerTrackCreator;

	/** Enables dedicated PoseSearch trace module */
	TSharedPtr<FTraceModule> TraceModule;
	
private:
	void RegisterPropertyTypeCustomizations();
	void UnregisterCustomizations();
	void RegisterCustomClassLayout(FName ClassName, FOnGetDetailCustomizationInstance DetailLayoutDelegate);
	void RegisterCustomPropertyTypeLayout(FName PropertyTypeName, FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate);

private:
	/** List of registered class that we must unregister when the module shuts down */
	TSet<FName> RegisteredClassNames;
	TSet<FName> RegisteredPropertyTypes;
};

void FEditorModule::StartupModule()
{
	// Register Asset Editor Commands
	FDatabaseEditorCommands::Register();

	if (GIsEditor && !IsRunningCommandlet())
	{
		FDebugger::Initialize();
		TraceModule = MakeShared<FTraceModule>();
		DebuggerTrackCreator = MakeShared<FDebuggerTrackCreator>();
		
		IModularFeatures::Get().RegisterModularFeature(FDebuggerTrackCreator::ModularFeatureName, DebuggerTrackCreator.Get());
		IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, TraceModule.Get());
		
		// Register Ed Modes used by PoseSearchDatabase and PoseSearchInteractionAsset
		FEditorModeRegistry::Get().RegisterMode<FDatabaseEdMode>(FDatabaseEdMode::EdModeId, LOCTEXT("PoseSearchDatabaseEdModeName", "PoseSearchDatabase"));
		FEditorModeRegistry::Get().RegisterMode<FInteractionAssetEdMode>(FInteractionAssetEdMode::EdModeId, LOCTEXT("FPoseSearchInteractionAssetEdModeName", "PoseSearchInteractionAsset"));

		UE::PoseSearchEditor::RegisterPoseSearchChooserWidgets();
	}

	RegisterPropertyTypeCustomizations();

	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.NotifyCustomizationModuleChanged();
}

void FEditorModule::ShutdownModule()
{
	for (IConsoleObject* ConsoleCmd : ConsoleCommands)
	{
		IConsoleManager::Get().UnregisterConsoleObject(ConsoleCmd);
	}
	ConsoleCommands.Empty();

	// Unregister Ed Modes
	FEditorModeRegistry::Get().UnregisterMode(FDatabaseEdMode::EdModeId);
	FEditorModeRegistry::Get().UnregisterMode(FInteractionAssetEdMode::EdModeId);

	UnregisterCustomizations();

	// Unregister Asset Editor Commands
	FDatabaseEditorCommands::Unregister();

	if (DebuggerTrackCreator.Get() != nullptr)
	{
		IModularFeatures::Get().UnregisterModularFeature(FDebuggerTrackCreator::ModularFeatureName, DebuggerTrackCreator.Get());
	}

	if (TraceModule.Get() != nullptr)
	{
		IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, TraceModule.Get());
	}
	FDebugger::Shutdown();
}

void FEditorModule::RegisterCustomClassLayout(FName ClassName, FOnGetDetailCustomizationInstance DetailLayoutDelegate)
{
	check(ClassName != NAME_None);

	RegisteredClassNames.Add(ClassName);

	static FName PropertyEditor("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);
	PropertyModule.RegisterCustomClassLayout(ClassName, DetailLayoutDelegate);
}

void FEditorModule::RegisterCustomPropertyTypeLayout(FName PropertyTypeName, FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate)
{
	check(PropertyTypeName != NAME_None);

	RegisteredPropertyTypes.Add(PropertyTypeName);

	static FName PropertyEditor("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);
	PropertyModule.RegisterCustomPropertyTypeLayout(PropertyTypeName, PropertyTypeLayoutDelegate);
}

void FEditorModule::RegisterPropertyTypeCustomizations()
{
	RegisterCustomPropertyTypeLayout("PoseSearchDatabaseAnimationAsset",  FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPoseSearchDatabaseAnimationAssetCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("PoseSearchColumnRowReflection",     FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPoseSearchColumnRowReflectionCustomization::MakeInstance));
}

void FEditorModule::UnregisterCustomizations()
{
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		// Unregister all classes customized by name
		for (auto It = RegisteredClassNames.CreateConstIterator(); It; ++It)
		{
			if (It->IsValid())
			{
				PropertyModule.UnregisterCustomClassLayout(*It);
			}
		}

		// Unregister all structures
		for (auto It = RegisteredPropertyTypes.CreateConstIterator(); It; ++It)
		{
			if (It->IsValid())
			{
				PropertyModule.UnregisterCustomPropertyTypeLayout(*It);
			}
		}

		PropertyModule.NotifyCustomizationModuleChanged();
	}
}

} // namespace UE::PoseSearch

IMPLEMENT_MODULE(UE::PoseSearch::FEditorModule, PoseSearchEditor);

#undef LOCTEXT_NAMESPACE
