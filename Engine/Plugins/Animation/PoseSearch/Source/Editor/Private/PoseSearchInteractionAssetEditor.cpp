// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchInteractionAssetEditor.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AnimPreviewInstance.h"
#include "AssetEditorModeManager.h"
#include "Components/StaticMeshComponent.h"
#include "EngineUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/WorldSettings.h"
#include "Modules/ModuleManager.h"
#include "PoseSearch/PoseSearchConstraints.h"
#include "PoseSearch/PoseSearchInteractionAsset.h"
#include "PoseSearch/PoseSearchInteractionLibrary.h"
#include "PoseSearchEditor.h"
#include "PreviewProfileController.h"
#include "PropertyEditorModule.h"
#include "SSimpleTimeSlider.h"
#include "Styling/AppStyle.h"
#include "UnrealWidget.h"
#include "Viewports.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "PoseSearchInteractionAssetEditor"

namespace UE::PoseSearch
{

/////////////////////////////////////////////////
// class FInteractionAssetEdMode
bool FInteractionAssetPreviewActor::SpawnPreviewActor(UWorld* World, const UPoseSearchInteractionAsset* InteractionAsset, const FRole& Role)
{
	UAnimationAsset* PreviewAsset = InteractionAsset->GetAnimationAsset(Role);
	if (!PreviewAsset)
	{
		return false;
	}

	ActorRole = Role;

	FActorSpawnParameters Params;
	Params.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButAlwaysSpawn;

	for (int32 ActorIndex = 0; ActorIndex < NumActors; ++ActorIndex)
	{
		FAnimationAssetSampler& Sampler = Samplers[ActorIndex];

		const FTransform Origin = InteractionAsset->GetDebugWarpOrigin(GetRole(), ActorIndex == DebugActor);
		Sampler.Init(PreviewAsset, Origin, BlendParameters, FAnimationAssetSampler::DefaultRootTransformSamplingRate, true, false);

		const FTransform ActorTransform = Sampler.ExtractRootTransform(CurrentTime);

		TWeakObjectPtr<AActor>& ActorPtr = ActorPtrs[ActorIndex];

		ActorPtr = World->SpawnActor<AActor>(AActor::StaticClass(), FTransform::Identity, Params);
		ActorPtr->SetFlags(RF_Transient);

		UDebugSkelMeshComponent* Mesh = NewObject<UDebugSkelMeshComponent>(ActorPtr.Get());
		Mesh->RegisterComponentWithWorld(World);

		UAnimPreviewInstance* AnimInstance = NewObject<UAnimPreviewInstance>(Mesh);
		Mesh->PreviewInstance = AnimInstance;
		AnimInstance->InitializeAnimation();

		USkeleton* Skeleton = PreviewAsset->GetSkeleton();
		USkeletalMesh* PreviewMesh = InteractionAsset->GetPreviewMesh(Role);
		if (!PreviewMesh)
		{
			PreviewMesh = Skeleton->GetPreviewMesh(true);
		}
		Mesh->SetSkeletalMesh(PreviewMesh);
		Mesh->EnablePreview(true, PreviewAsset);

		AnimInstance->SetAnimationAsset(PreviewAsset, InteractionAsset->IsLooping(), 0.f);
		AnimInstance->SetBlendSpacePosition(BlendParameters);

		AnimInstance->PlayAnim(InteractionAsset->IsLooping(), 0.f);
		if (!ActorPtr->GetRootComponent())
		{
			ActorPtr->SetRootComponent(Mesh);
		}

		AnimInstance->SetPosition(Sampler.ToNormalizedTime(CurrentTime));
		AnimInstance->SetPlayRate(0.f);
		AnimInstance->SetBlendSpacePosition(BlendParameters);

		ActorPtr->SetActorTransform(ActorTransform);
	}
	return true;
}

void FInteractionAssetPreviewActor::UpdatePreviewActor(const UPoseSearchInteractionAsset* InteractionAsset, float PlayTime)
{
	if (UAnimationAsset* PreviewAsset = InteractionAsset->GetAnimationAsset(GetRole()))
	{
		bool bPlayTimeUpdated = false;

		float NewCurrentTime = 0.f;
		FAnimationRuntime::AdvanceTime(false, PlayTime, NewCurrentTime, Samplers[PreviewActor].GetPlayLength());

		if (!FMath::IsNearlyEqual(CurrentTime, NewCurrentTime))
		{
			CurrentTime = NewCurrentTime;
			bPlayTimeUpdated = true;
		}

		for (int32 ActorIndex = 0; ActorIndex < NumActors; ++ActorIndex)
		{
			const FTransform Origin = InteractionAsset->GetDebugWarpOrigin(GetRole(), ActorIndex == DebugActor);

			bool bSamplerReinitialized = false;
			FAnimationAssetSampler& Sampler = Samplers[ActorIndex];
			if (PreviewAsset != Sampler.GetAsset() || !Origin.Equals(Sampler.GetRootTransformOrigin()))
			{
				// reinitializing the Sampler if the PreviewAsset or the origin transform changed
				Sampler.Init(PreviewAsset, Origin, BlendParameters, FAnimationAssetSampler::DefaultRootTransformSamplingRate, true, false);
				bSamplerReinitialized = true;
			}

			TWeakObjectPtr<AActor>& ActorPtr = ActorPtrs[ActorIndex];
			if (ActorPtr != nullptr)
			{
				if (UDebugSkelMeshComponent* Mesh = Cast<UDebugSkelMeshComponent>(ActorPtr->GetRootComponent()))
				{
					USkeleton* Skeleton = PreviewAsset->GetSkeleton();
					USkeletalMesh* PreviewMesh = InteractionAsset->GetPreviewMesh(GetRole());
					if (!PreviewMesh)
					{
						PreviewMesh = Skeleton->GetPreviewMesh(true);
					}

					if (Mesh->GetSkeletalMeshAsset() != PreviewMesh)
					{
						Mesh->SetSkeletalMesh(PreviewMesh);
					}					

					if (UAnimPreviewInstance* AnimInstance = Mesh->PreviewInstance.Get())
					{
						bool bPreviewAssetChanged = false;
						if (AnimInstance->GetAnimationAsset() != PreviewAsset)
						{
							AnimInstance->SetAnimationAsset(PreviewAsset, InteractionAsset->IsLooping(), 0.f);
							bPreviewAssetChanged = true;
						}

						if (bPlayTimeUpdated || bSamplerReinitialized || bPreviewAssetChanged)
						{
							// SetPosition is in [0..1] range for blendspaces
							AnimInstance->SetPosition(Sampler.ToNormalizedTime(CurrentTime));
							AnimInstance->SetPlayRate(0.f);
							AnimInstance->SetBlendSpacePosition(BlendParameters);

							const FTransform ActorTransform = Sampler.ExtractRootTransform(CurrentTime);
							ActorPtr->SetActorTransform(ActorTransform);
						}
					}
				}
			}
		}
	}
}

void FInteractionAssetPreviewActor::Destroy()
{
	for (TWeakObjectPtr<AActor>& ActorPtr : ActorPtrs)
	{
		if (ActorPtr != nullptr)
		{
			ActorPtr->Destroy();
		}
		ActorPtr = nullptr;
	}
}


UAnimPreviewInstance* FInteractionAssetPreviewActor::GetAnimPreviewInstance()
{
	if (const AActor* Actor = ActorPtrs[DebugActor].Get())
	{
		if (const UDebugSkelMeshComponent* Mesh = Cast<UDebugSkelMeshComponent>(Actor->GetRootComponent()))
		{
			return Mesh->PreviewInstance.Get();
		}
	}
	
	return nullptr;
}

FTransform FInteractionAssetPreviewActor::GetDebugActorTransformFromSampler() const
{
	const FAnimationAssetSampler& Sampler = Samplers[DebugActor];
	const FTransform ActorTransform = Sampler.ExtractRootTransform(CurrentTime);
	return ActorTransform;
}

void FInteractionAssetPreviewActor::ForceDebugActorTransform(const FTransform& ActorTransform)
{
	TWeakObjectPtr<AActor>& ActorPtr = ActorPtrs[DebugActor];
	if (ActorPtr != nullptr)
	{
		ActorPtr->SetActorTransform(ActorTransform);
	}
}

/////////////////////////////////////////////////
// class FInteractionAssetViewModel
void FInteractionAssetViewModel::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(InteractionAssetPtr);
}

void FInteractionAssetViewModel::Initialize(UPoseSearchInteractionAsset* InteractionAsset, const TSharedRef<FInteractionAssetPreviewScene>& PreviewScene)
{
	InteractionAssetPtr = InteractionAsset;
	PreviewScenePtr = PreviewScene;
}

void FInteractionAssetViewModel::PreviewBackwardEnd()
{
	SetPlayTime(0.f, false);
}

void FInteractionAssetViewModel::PreviewBackwardStep()
{
	if (const UPoseSearchInteractionAsset* InteractionAsset = GetInteractionAsset())
	{
		SetPlayTime(PlayTime - StepDeltaTime, false);
	}
}

void FInteractionAssetViewModel::PreviewBackward()
{
	DeltaTimeMultiplier = -1.f;
}

void FInteractionAssetViewModel::PreviewPause()
{
	DeltaTimeMultiplier = 0.f;
}

void FInteractionAssetViewModel::PreviewForward()
{
	DeltaTimeMultiplier = 1.f;
}

void FInteractionAssetViewModel::PreviewForwardStep()
{
	if (const UPoseSearchInteractionAsset* InteractionAsset = GetInteractionAsset())
	{
		SetPlayTime(PlayTime + StepDeltaTime, false);
	}
}

void FInteractionAssetViewModel::PreviewForwardEnd()
{
	if (const UPoseSearchInteractionAsset* InteractionAsset = GetInteractionAsset())
	{
		// setting play time to a big number that will be clamped internally
		SetPlayTime(UE_MAX_FLT, false);
	}
}

UWorld* FInteractionAssetViewModel::GetWorld()
{
	check(PreviewScenePtr.IsValid());
	return PreviewScenePtr.Pin()->GetWorld();
}

void FInteractionAssetViewModel::Tick(float DeltaSeconds)
{
	const UPoseSearchInteractionAsset* InteractionAsset = GetInteractionAsset();
	if (!InteractionAsset)
	{
		PreviewPause();
		SetPlayTime(0.f, false);

		for (FInteractionAssetPreviewActor& PreviewActor : PreviewActors)
		{
			PreviewActor.Destroy();
		}

		PreviewActors.Reset();
	}
	else
	{
		PlayTime += DeltaSeconds * DeltaTimeMultiplier;

		const int32 NumRoles = InteractionAsset->GetNumRoles();
		FRoleToIndex InteractionAssetRoleToIndex;
		for (int32 RoleIndex = 0; RoleIndex < NumRoles; ++RoleIndex)
		{
			InteractionAssetRoleToIndex.Add(InteractionAsset->GetRole(RoleIndex)) = RoleIndex;
		}

		// iterating backwards because of the possible RemoveAtSwap 
		bool bAnyAddedOrRemovedActors = false;
		FRoleToIndex PreviewActorsRoleToIndex;
		for (int32 ActorIndex = PreviewActors.Num() - 1; ActorIndex >= 0; --ActorIndex)
		{
			FInteractionAssetPreviewActor& PreviewActor = PreviewActors[ActorIndex];
			if (!InteractionAssetRoleToIndex.Find(PreviewActor.GetRole()))
			{
				PreviewActor.Destroy();
				PreviewActors.RemoveAtSwap(ActorIndex, EAllowShrinking::No);
				bAnyAddedOrRemovedActors = true;
			}
			else
			{
				PreviewActorsRoleToIndex.Add(PreviewActor.GetRole());
			}
		}

		if (PreviewActors.Num() != InteractionAssetRoleToIndex.Num())
		{
			for (int32 RoleIndex = 0; RoleIndex < NumRoles; ++RoleIndex)
			{
				if (!PreviewActorsRoleToIndex.Find(InteractionAsset->GetRole(RoleIndex)))
				{
					FInteractionAssetPreviewActor PreviewActor;
					if (PreviewActor.SpawnPreviewActor(GetWorld(), InteractionAsset, InteractionAsset->GetRole(RoleIndex)))
					{
						PreviewActors.Add(MoveTemp(PreviewActor));
						bAnyAddedOrRemovedActors = true;
					}
				}
			}
		}

		if (bAnyAddedOrRemovedActors)
		{
			PreviewPause();
			SetPlayTime(0.f, false);
		}

		for (FInteractionAssetPreviewActor& PreviewActor : PreviewActors)
		{
			PreviewActor.UpdatePreviewActor(InteractionAsset, PlayTime);
		}

		FRoleToIndex PreviewActorRoleToIndex;
		PreviewActorRoleToIndex.Reserve(PreviewActors.Num());

		if (PreviewActors.Num() == NumRoles)
		{
			TArray<TObjectPtr<const UObject>, TInlineAllocator<PreallocatedRolesNum>> AnimContexts;
			AnimContexts.SetNum(NumRoles);

			TArray<FRole, TInlineAllocator<PreallocatedRolesNum>> Roles;
			Roles.SetNum(NumRoles);

			for (int32 PreviewActorIndex = 0; PreviewActorIndex < PreviewActors.Num(); ++PreviewActorIndex)
			{
				PreviewActorRoleToIndex.Add(PreviewActors[PreviewActorIndex].GetRole()) = PreviewActorIndex;

				FInteractionAssetPreviewActor& PreviewActor = PreviewActors[PreviewActorIndex];
				AnimContexts[PreviewActorIndex] = PreviewActor.GetAnimPreviewInstance();
				Roles[PreviewActorIndex] = PreviewActor.GetRole();
			}

			if (PreviewActorRoleToIndex.Num() == NumRoles)
			{
				// testing CalculateWarpTransforms
#if WITH_EDITORONLY_DATA
				if (InteractionAsset->bEnableDebugWarp)
				{
					TArray<FTransform, TInlineAllocator<PreallocatedRolesNum>> ActorTransforms;
					ActorTransforms.SetNum(NumRoles);
					for (int32 InteractionAssetRoleIndex = 0; InteractionAssetRoleIndex < NumRoles; ++InteractionAssetRoleIndex)
					{
						const FRole& InteractionAssetRole = InteractionAsset->GetRole(InteractionAssetRoleIndex);
						const int32 PreviewActorIndex = PreviewActorRoleToIndex[InteractionAssetRole];

						FInteractionAssetPreviewActor& PreviewActor = PreviewActors[PreviewActorIndex];
						check(PreviewActor.GetRole() == InteractionAssetRole);

						ActorTransforms[InteractionAssetRoleIndex] = PreviewActor.GetDebugActorTransformFromSampler();
					}

					TArray<FTransform, TInlineAllocator<PreallocatedRolesNum>> FullAlignedActorTransforms;
					FullAlignedActorTransforms.SetNum(NumRoles);
					const TConstArrayView<const UMirrorDataTable*> MirrorDataTables;

					InteractionAsset->CalculateWarpTransforms(PlayTime, 0.f, ActorTransforms, FullAlignedActorTransforms, MirrorDataTables, AnimContexts);

#if ENABLE_ANIM_DEBUG
					const UWorld* DebugWorld = GetWorld();
#endif // ENABLE_ANIM_DEBUG

					for (int32 InteractionAssetRoleIndex = 0; InteractionAssetRoleIndex < NumRoles; ++InteractionAssetRoleIndex)
					{
						const FRole& InteractionAssetRole = InteractionAsset->GetRole(InteractionAssetRoleIndex);
						const int32 PreviewActorIndex = PreviewActorRoleToIndex[InteractionAssetRole];

						FInteractionAssetPreviewActor& PreviewActor = PreviewActors[PreviewActorIndex];
						check(PreviewActor.GetRole() == InteractionAssetRole);
						FTransform DebugActorTransform;
						DebugActorTransform.Blend(ActorTransforms[InteractionAssetRoleIndex], FullAlignedActorTransforms[InteractionAssetRoleIndex], InteractionAsset->DebugWarpAmount);

#if ENABLE_ANIM_DEBUG
						// FullAlignedActorTransforms drawing 
						const FVector FullAlignedActorPosition = FullAlignedActorTransforms[InteractionAssetRoleIndex].GetTranslation();
						const FQuat FullAlignedActorRotation = FullAlignedActorTransforms[InteractionAssetRoleIndex].GetRotation();
						DrawDebugCircle(DebugWorld, FMatrix(FVector::ZAxisVector, FVector::XAxisVector, FVector::YAxisVector, FullAlignedActorPosition), 5.f, 16, FColorList::Yellow, false, 0.f, SDPG_Foreground, 0.f, false);
						DrawDebugLine(DebugWorld, FullAlignedActorPosition, FullAlignedActorPosition + FullAlignedActorRotation.RotateVector(FVector::XAxisVector) * 10, FColorList::Yellow, false, 0.f, SDPG_Foreground, 0.f);
#endif // ENABLE_ANIM_DEBUG

						PreviewActor.ForceDebugActorTransform(DebugActorTransform);
					}

#if ENABLE_ANIM_DEBUG
					if (InteractionAsset->DebugTimeOffset > UE_KINDA_SMALL_NUMBER)
					{
						InteractionAsset->CalculateWarpTransforms(PlayTime, InteractionAsset->DebugTimeOffset, ActorTransforms, FullAlignedActorTransforms, MirrorDataTables, AnimContexts);

						for (int32 InteractionAssetRoleIndex = 0; InteractionAssetRoleIndex < NumRoles; ++InteractionAssetRoleIndex)
						{
							// FullAlignedActorTransforms drawing 
							const FVector FullAlignedActorPosition = FullAlignedActorTransforms[InteractionAssetRoleIndex].GetTranslation();
							const FQuat FullAlignedActorRotation = FullAlignedActorTransforms[InteractionAssetRoleIndex].GetRotation();
							DrawDebugCircle(DebugWorld, FMatrix(FVector::ZAxisVector, FVector::XAxisVector, FVector::YAxisVector, FullAlignedActorPosition), 5.f, 16, FColorList::Pink, false, 0.f, SDPG_Foreground, 0.f, false);
							DrawDebugLine(DebugWorld, FullAlignedActorPosition, FullAlignedActorPosition + FullAlignedActorRotation.RotateVector(FVector::XAxisVector) * 10, FColorList::Pink, false, 0.f, SDPG_Foreground, 0.f);
						}
					}
#endif // ENABLE_ANIM_DEBUG
				}
#endif // WITH_EDITORONLY_DATA

				// extracting and drawing Constraints

				// @todo: fill up those parameters
				const FVector BlendParameters = FVector::ZeroVector;
				const bool bIsMirrored = false;

				TArray<FPoseSearchConstraint> Constraints;
				UPoseSearchInteractionLibrary::UpdateConstraints(Constraints, InteractionAsset, StepDeltaTime, PlayTime, bIsMirrored, BlendParameters, AnimContexts, Roles);

				if (const UWorld* World = GetWorld())
				{
					for (const FPoseSearchConstraint& Constraint : Constraints)
					{
						DrawDebugLine(World, Constraint.FromSocketTransform.GetTranslation(), Constraint.ToSocketTransform.GetTranslation(), FColor::Red, false, -1.f, SDPG_Foreground);
					}
				}
			}
		}
	}
}

TRange<double> FInteractionAssetViewModel::GetPreviewPlayRange() const
{
	constexpr double ViewRangeSlack = 0.2;
	if (const UPoseSearchInteractionAsset* InteractionAsset = GetInteractionAsset())
	{
		// @todo: add support for InteractionAsset contatining blend spaces
		FVector BlendParameters = FVector::ZeroVector;
		return TRange<double>(-ViewRangeSlack, InteractionAsset->GetPlayLength(BlendParameters) + ViewRangeSlack);
	}
	
	return TRange<double>(-ViewRangeSlack, ViewRangeSlack);
}

void FInteractionAssetViewModel::SetPlayTime(float NewPlayTime, bool bInTickPlayTime)
{
	if (const UPoseSearchInteractionAsset* InteractionAsset = GetInteractionAsset())
	{
		NewPlayTime = FMath::Max(NewPlayTime, 0.f);
		DeltaTimeMultiplier = bInTickPlayTime ? DeltaTimeMultiplier : 0.f;

		if (!FMath::IsNearlyEqual(PlayTime, NewPlayTime))
		{
			PlayTime = NewPlayTime;

			for (FInteractionAssetPreviewActor& PreviewActor : PreviewActors)
			{
				PreviewActor.UpdatePreviewActor(InteractionAsset, PlayTime);
			}
		}
	}
}

void FInteractionAssetViewModel::SetPreviewProperties(float AnimAssetTime, const FVector& AnimAssetBlendParameters, bool bAnimAssetPlaying)
{
	// @todo: add support for blend spaces (pass AnimAssetBlendParameters as input)
	SetPlayTime(AnimAssetTime, bAnimAssetPlaying);
}

/////////////////////////////////////////////////
// class FInteractionAssetEdMode
const FEditorModeID FInteractionAssetEdMode::EdModeId = TEXT("PoseSearchInteractionAssetEdMode");

void FInteractionAssetEdMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	FEdMode::Tick(ViewportClient, DeltaTime);

	if (FInteractionAssetViewportClient* InteractionAssetViewportClient = static_cast<FInteractionAssetViewportClient*>(ViewportClient))
	{
		// ensure we redraw even if PIE is active
		InteractionAssetViewportClient->Invalidate();

		if (!ViewModel)
		{
			ViewModel = InteractionAssetViewportClient->GetAssetEditor()->GetViewModel();
		}
	}

	if (ViewModel)
	{
		ViewModel->Tick(DeltaTime);
	}
}

void FInteractionAssetEdMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	FEdMode::Render(View, Viewport, PDI);
}

bool FInteractionAssetEdMode::AllowWidgetMove()
{
	return FEdMode::ShouldDrawWidget();
}

bool FInteractionAssetEdMode::ShouldDrawWidget() const
{
	return FEdMode::ShouldDrawWidget();
}

bool FInteractionAssetEdMode::GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	return FEdMode::GetCustomDrawingCoordinateSystem(InMatrix, InData);
}

/////////////////////////////////////////////////
// class FInteractionAssetViewportClient
FInteractionAssetViewportClient::FInteractionAssetViewportClient(
	const TSharedRef<FInteractionAssetPreviewScene>& InPreviewScene,
	const TSharedRef<SInteractionAssetViewport>& InViewport,
	const TSharedRef<FInteractionAssetEditor>& InAssetEditor)
	: FEditorViewportClient(nullptr, &InPreviewScene.Get(), StaticCastSharedRef<SEditorViewport>(InViewport))
	, PreviewScenePtr(InPreviewScene)
	, AssetEditorPtr(InAssetEditor)
{
	Widget->SetUsesEditorModeTools(ModeTools.Get());
	StaticCastSharedPtr<FAssetEditorModeManager>(ModeTools)->SetPreviewScene(&InPreviewScene.Get());
	ModeTools->SetDefaultMode(FInteractionAssetEdMode::EdModeId);

	SetRealtime(true);

	SetWidgetCoordSystemSpace(COORD_Local);
	ModeTools->SetWidgetMode(UE::Widget::WM_Translate);
}

void FInteractionAssetViewportClient::TrackingStarted(
	const struct FInputEventState& InInputState,
	bool bIsDraggingWidget,
	bool bNudge)
{
	ModeTools->StartTracking(this, Viewport);
}

void FInteractionAssetViewportClient::TrackingStopped()
{
	ModeTools->EndTracking(this, Viewport);
	Invalidate();
}

void FInteractionAssetViewportClient::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	FEditorViewportClient::Draw(View, PDI);
}

/////////////////////////////////////////////////
// class FInteractionAssetPreviewScene
FInteractionAssetPreviewScene::FInteractionAssetPreviewScene(ConstructionValues CVs, const TSharedRef<FInteractionAssetEditor>& Editor)
: FAdvancedPreviewScene(CVs)
, EditorPtr(Editor)
{
	// Disable killing actors outside of the world
	AWorldSettings* WorldSettings = GetWorld()->GetWorldSettings(true);
	WorldSettings->bEnableWorldBoundsChecks = false;

	// Spawn an owner for FloorMeshComponent so CharacterMovementComponent can detect it as a valid floor and slide 
	// along it
	{
		AActor* FloorActor = GetWorld()->SpawnActor<AActor>(AActor::StaticClass(), FTransform());
		check(FloorActor);

		static const FString NewName = FString(TEXT("FloorComponent"));
		FloorMeshComponent->Rename(*NewName, FloorActor);

		FloorActor->SetRootComponent(FloorMeshComponent);
	}
}

void FInteractionAssetPreviewScene::Tick(float InDeltaTime)
{
	FAdvancedPreviewScene::Tick(InDeltaTime);

	// Trigger Begin Play in this preview world.
	// This is needed for the CharacterMovementComponent to be able to switch to falling mode. 
	// See: UCharacterMovementComponent::StartFalling
	if (PreviewWorld && !PreviewWorld->GetBegunPlay())
	{
		for (FActorIterator It(PreviewWorld); It; ++It)
		{
			It->DispatchBeginPlay();
		}

		PreviewWorld->SetBegunPlay(true);
	}

	GetWorld()->Tick(LEVELTICK_All, InDeltaTime);
}

/////////////////////////////////////////////////
// class SInteractionAssetViewport
void SInteractionAssetViewport::Construct(
	const FArguments& InArgs,
	const FInteractionAssetPreviewRequiredArgs& InRequiredArgs)
{
	PreviewScenePtr = InRequiredArgs.PreviewScene;
	AssetEditorPtr = InRequiredArgs.AssetEditor;

	SEditorViewport::Construct(
		SEditorViewport::FArguments()
		.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
		.AddMetaData<FTagMetaData>(TEXT("AnimationTools.Viewport"))
	);
}

void SInteractionAssetViewport::BindCommands()
{
	SEditorViewport::BindCommands();
}

TSharedRef<FEditorViewportClient> SInteractionAssetViewport::MakeEditorViewportClient()
{
	ViewportClient = MakeShared<FInteractionAssetViewportClient>(
		PreviewScenePtr.Pin().ToSharedRef(),
		SharedThis(this),
		AssetEditorPtr.Pin().ToSharedRef());
	ViewportClient->ViewportType = LVT_Perspective;
	ViewportClient->bSetListenerPosition = false;
	ViewportClient->SetViewLocation(EditorViewportDefs::DefaultPerspectiveViewLocation);
	ViewportClient->SetViewRotation(EditorViewportDefs::DefaultPerspectiveViewRotation);

	return ViewportClient.ToSharedRef();
}

TSharedPtr<SWidget> SInteractionAssetViewport::BuildViewportToolbar()
{
	return SAssignNew(ViewportToolbar, SCommonEditorViewportToolbarBase, SharedThis(this));
}

TSharedRef<SEditorViewport> SInteractionAssetViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SInteractionAssetViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void SInteractionAssetViewport::OnFloatingButtonClicked()
{
}

/////////////////////////////////////////////////
// class SInteractionAssetPreview
void SInteractionAssetPreview::Construct(const FArguments& InArgs, const FInteractionAssetPreviewRequiredArgs& InRequiredArgs)
{
	SliderColor = InArgs._SliderColor;
	SliderScrubTime = InArgs._SliderScrubTime;
	SliderViewRange = InArgs._SliderViewRange;
	OnSliderScrubPositionChanged = InArgs._OnSliderScrubPositionChanged;

	OnBackwardEnd = InArgs._OnBackwardEnd;
	OnBackwardStep = InArgs._OnBackwardStep;
	OnBackward = InArgs._OnBackward;
	OnPause = InArgs._OnPause;
	OnForward = InArgs._OnForward;
	OnForwardStep = InArgs._OnForwardStep;
	OnForwardEnd = InArgs._OnForwardEnd;

	FSlimHorizontalToolBarBuilder ToolBarBuilder(
		TSharedPtr<const FUICommandList>(), 
		FMultiBoxCustomization::None, 
		nullptr, true);

	auto AddToolBarButton = [&ToolBarBuilder](FName ButtonImageName, FOnButtonClickedEvent& OnClicked)
		{
			ToolBarBuilder.AddToolBarWidget(
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
				.OnClicked_Lambda([&OnClicked]()
					{
						if (OnClicked.IsBound())
						{
							OnClicked.Execute();
							return FReply::Handled();
						}
						return FReply::Unhandled();
					})
				[
					SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						.Image(FAppStyle::Get().GetBrush(ButtonImageName))
				]);
		};

	//ToolBarBuilder.SetStyle(&FAppStyle::Get(), "PaletteToolBar");
	ToolBarBuilder.BeginSection("Preview");
	{
		AddToolBarButton("Animation.Backward_End", OnBackwardEnd);
		AddToolBarButton("Animation.Backward_Step", OnBackwardStep);
		AddToolBarButton("Animation.Backward", OnBackward);
		AddToolBarButton("Animation.Pause", OnPause);
		AddToolBarButton("Animation.Forward", OnForward);
		AddToolBarButton("Animation.Forward_Step", OnForwardStep);
		AddToolBarButton("Animation.Forward_End", OnForwardEnd);
	}

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SInteractionAssetViewport, InRequiredArgs)
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				ToolBarBuilder.MakeWidget()
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SSimpleTimeSlider)
				.ClampRangeHighlightSize(0.15f)
				.ClampRangeHighlightColor_Lambda([this]()
					{
						return SliderColor.Get();
					})
				.ScrubPosition_Lambda([this]()
					{
						return SliderScrubTime.Get();
					})
				.ViewRange_Lambda([this]()
					{
						return SliderViewRange.Get();
					})
				.ClampRange_Lambda([this]()
					{
						return SliderViewRange.Get();
					})
				.OnScrubPositionChanged_Lambda([this](double NewScrubTime, bool bIsScrubbing)
					{
						if (bIsScrubbing)
						{
							OnSliderScrubPositionChanged.ExecuteIfBound(NewScrubTime, bIsScrubbing);
						}
					})
			]
		]
	];
}

/////////////////////////////////////////////////
// class FInteractionAssetEditor
const FName PoseSearchInteractionAssetEditorAppName = FName(TEXT("PoseSearchInteractionAssetEditorApp"));

// Tab identifiers
struct FInteractionAssetEditorTabs
{
	static const FName AssetDetailsID;
	static const FName ViewportID;
};
const FName FInteractionAssetEditorTabs::AssetDetailsID(TEXT("PoseSearchInteractionAssetEditorAssetDetailsTabID"));
const FName FInteractionAssetEditorTabs::ViewportID(TEXT("PoseSearchInteractionAssetEditorViewportTabID"));

void FInteractionAssetEditor::InitAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UPoseSearchInteractionAsset* InteractionAsset)
{
	// Create Preview Scene
	if (!PreviewScene.IsValid())
	{
		PreviewScene = MakeShareable(
				new FInteractionAssetPreviewScene(
					FPreviewScene::ConstructionValues()
					.SetCreatePhysicsScene(false)
					.SetTransactional(false)
					.ForceUseMovementComponentInNonGameWorld(true),
					StaticCastSharedRef<FInteractionAssetEditor>(AsShared())));

		//Temporary fix for missing attached assets - MDW (Copied from FPersonaToolkit::CreatePreviewScene)
		PreviewScene->GetWorld()->GetWorldSettings()->SetIsTemporarilyHiddenInEditor(false);
	}

	// Create view model
	ViewModel = MakeShared<FInteractionAssetViewModel>();
	ViewModel->Initialize(InteractionAsset, PreviewScene.ToSharedRef());

	// Create viewport widget
	{
		FInteractionAssetPreviewRequiredArgs PreviewArgs(
			StaticCastSharedRef<FInteractionAssetEditor>(AsShared()),
			PreviewScene.ToSharedRef());
			
		PreviewWidget = SNew(SInteractionAssetPreview, PreviewArgs)
			.SliderColor(FLinearColor::Red)
			.SliderScrubTime_Lambda([this]()
				{
					return ViewModel->GetPlayTime();
				})
			.SliderViewRange_Lambda([this]() 
				{ 
					return ViewModel->GetPreviewPlayRange();
				})
			.OnSliderScrubPositionChanged_Lambda([this](float NewScrubPosition, bool bScrubbing)
				{
					ViewModel->SetPlayTime(NewScrubPosition, !bScrubbing);
				})
			.OnBackwardEnd_Raw(this, &FInteractionAssetEditor::PreviewBackwardEnd)
			.OnBackwardStep_Raw(this, &FInteractionAssetEditor::PreviewBackwardStep)
			.OnBackward_Raw(this, &FInteractionAssetEditor::PreviewBackward)
			.OnPause_Raw(this, &FInteractionAssetEditor::PreviewPause)
			.OnForward_Raw(this, &FInteractionAssetEditor::PreviewForward)
			.OnForwardStep_Raw(this, &FInteractionAssetEditor::PreviewForwardStep)
			.OnForwardEnd_Raw(this, &FInteractionAssetEditor::PreviewForwardEnd);
	}

	// asset details widget
	FDetailsViewArgs AssetDetailsArgs;
	AssetDetailsArgs.bHideSelectionTip = true;
	AssetDetailsArgs.NotifyHook = this;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	EditingAssetWidget = PropertyModule.CreateDetailView(AssetDetailsArgs);
	EditingAssetWidget->SetObject(InteractionAsset);

	// Define Editor Layout
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout =
	FTabManager::NewLayout("Standalone_PoseSearchInteractionAssetDatabaseEditor_Layout_v0.01")
		->AddArea
		(
			// Main application area
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
					->SetSizeCoefficient(0.4f)
					->AddTab(FInteractionAssetEditorTabs::AssetDetailsID, ETabState::OpenedTab)
					->SetHideTabWell(false)
			)
			->Split
			(
				FTabManager::NewStack()
					->SetSizeCoefficient(0.6f)
					->AddTab(FInteractionAssetEditorTabs::ViewportID, ETabState::OpenedTab)
					->SetHideTabWell(false)
			)
		);

	FAssetEditorToolkit::InitAssetEditor(
		Mode,
		InitToolkitHost,
		PoseSearchInteractionAssetEditorAppName,
		StandaloneDefaultLayout,
		true,
		true,
		InteractionAsset,
		false);

	RegenerateMenusAndToolbars();
}

void FInteractionAssetEditor::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(
		LOCTEXT("WorkspaceMenu_PoseSearchInteractionAssetEditor", "Pose Search Interaction Asset Editor"));
	TSharedRef<FWorkspaceItem> WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(
		FInteractionAssetEditorTabs::ViewportID,
		FOnSpawnTab::CreateSP(this, &FInteractionAssetEditor::SpawnTab_Viewport))
		.SetDisplayName(LOCTEXT("ViewportTab", "Viewport"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));
			
	InTabManager->RegisterTabSpawner(
		FInteractionAssetEditorTabs::AssetDetailsID,
		FOnSpawnTab::CreateSP(this, &FInteractionAssetEditor::SpawnTab_AssetDetails))
		.SetDisplayName(LOCTEXT("PoseSearchInteractionAssetDetailsTab", "Pose Search Interaction Asset Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
}

void FInteractionAssetEditor::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(FInteractionAssetEditorTabs::ViewportID);
	InTabManager->UnregisterTabSpawner(FInteractionAssetEditorTabs::AssetDetailsID);
}

FName FInteractionAssetEditor::GetToolkitFName() const
{
	return FName("PoseSearchInteractionAssetEditor");
}

FText FInteractionAssetEditor::GetBaseToolkitName() const
{
	return LOCTEXT("PoseSearchInteractionAssetEditorAppLabel", "Pose Search Interaction Asset Editor");
}

FText FInteractionAssetEditor::GetToolkitName() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("AssetName"), FText::FromString(GetNameSafe(GetInteractionAsset())));
	return FText::Format(LOCTEXT("PoseSearchInteractionAssetEditorToolkitName", "{AssetName}"), Args);
}

FLinearColor FInteractionAssetEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor::White;
}

FString FInteractionAssetEditor::GetWorldCentricTabPrefix() const
{
	return TEXT("PoseSearchInteractionAssetEditor");
}

void FInteractionAssetEditor::SetPreviewProperties(float AnimAssetTime, const FVector& AnimAssetBlendParameters, bool bAnimAssetPlaying)
{
	ViewModel->SetPreviewProperties(AnimAssetTime, AnimAssetBlendParameters, bAnimAssetPlaying);
}

TSharedRef<SDockTab> FInteractionAssetEditor::SpawnTab_Viewport(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FInteractionAssetEditorTabs::ViewportID);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab).Label(LOCTEXT("ViewportTab_Title", "Viewport"));

	if (PreviewWidget.IsValid())
	{
		SpawnedTab->SetContent(PreviewWidget.ToSharedRef());
	}

	return SpawnedTab;
}

TSharedRef<SDockTab> FInteractionAssetEditor::SpawnTab_AssetDetails(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FInteractionAssetEditorTabs::AssetDetailsID);

	return SNew(SDockTab)
		.Label(LOCTEXT("PoseSearchInteractionAsset_Details_Title", "Pose Search Interaction Asset Details"))
		[
			EditingAssetWidget.ToSharedRef()
		];
}

} // namespace UE::PoseSearch

#undef LOCTEXT_NAMESPACE
