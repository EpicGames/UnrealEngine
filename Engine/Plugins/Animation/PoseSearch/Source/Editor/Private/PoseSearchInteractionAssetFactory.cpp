// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchInteractionAssetFactory.h"
#include "PoseSearch/PoseSearchInteractionAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchInteractionAssetFactory)

#define LOCTEXT_NAMESPACE "PoseSearchInteractionAssetFactory"

UPoseSearchInteractionAssetFactory::UPoseSearchInteractionAssetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UPoseSearchInteractionAsset::StaticClass();
}

UObject* UPoseSearchInteractionAssetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UPoseSearchInteractionAsset>(InParent, Class, Name, Flags);
}

FString UPoseSearchInteractionAssetFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("NewPoseSearchInteractionAsset"));
}

#undef LOCTEXT_NAMESPACE
