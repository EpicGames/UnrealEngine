// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchMeshComponent.h"
#include "Engine/SkinnedAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchMeshComponent)

void UPoseSearchMeshComponent::Initialize(const FTransform& InComponentToWorld)
{
	SetComponentToWorld(InComponentToWorld);
	
	if (USkinnedAsset* CurrSkinnedAsset = GetSkinnedAsset())
	{
		const FReferenceSkeleton& SkeletalMeshRefSkeleton = CurrSkinnedAsset->GetRefSkeleton();

		// set up bone visibility states as this gets skipped since we allocate the component array before registration
		for (int32 BaseIndex = 0; BaseIndex < 2; ++BaseIndex)
		{
			BoneVisibilityStates[BaseIndex].SetNum(SkeletalMeshRefSkeleton.GetNum());
			for (int32 BoneIndex = 0; BoneIndex < SkeletalMeshRefSkeleton.GetNum(); BoneIndex++)
			{
				BoneVisibilityStates[BaseIndex][BoneIndex] = BVS_ExplicitlyHidden;
			}
		}

		Refresh();
	}
}

void UPoseSearchMeshComponent::Refresh()
{
	// Flip buffers once to copy the directly-written component space transforms
	bNeedToFlipSpaceBaseBuffers = true;
	bHasValidBoneTransform = false;
	FlipEditableSpaceBases();
	bHasValidBoneTransform = true;

	InvalidateCachedBounds();
	UpdateBounds();
	MarkRenderTransformDirty();
	MarkRenderDynamicDataDirty();
	MarkRenderStateDirty();
}
