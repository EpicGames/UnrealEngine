// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PoseableMeshComponent.h"
#include "PoseSearchMeshComponent.generated.h"

UCLASS()
class UPoseSearchMeshComponent : public UPoseableMeshComponent
{
	GENERATED_BODY()
public:

	void Refresh();
	void Initialize(const FTransform& InComponentToWorld);
};
