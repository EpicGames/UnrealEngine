// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchNormalizationSetFactory.h"
#include "PoseSearch/PoseSearchNormalizationSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchNormalizationSetFactory)

#define LOCTEXT_NAMESPACE "PoseSearchNormalizationSetFactory"

UPoseSearchNormalizationSetFactory::UPoseSearchNormalizationSetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UPoseSearchNormalizationSet::StaticClass();
}

UObject* UPoseSearchNormalizationSetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UPoseSearchNormalizationSet>(InParent, Class, Name, Flags);
}

FString UPoseSearchNormalizationSetFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("NewPoseSearchNormalizationSet"));
}

#undef LOCTEXT_NAMESPACE
