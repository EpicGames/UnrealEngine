// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchSchemaFactory.h"
#include "Animation/Skeleton.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "IContentBrowserSingleton.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "Widgets/Layout/SBorder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchSchemaFactory)

#define LOCTEXT_NAMESPACE "PoseSearchEditor"

UPoseSearchSchemaFactory::UPoseSearchSchemaFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UPoseSearchSchema::StaticClass();
}

void UPoseSearchSchemaFactory::OnTargetSkeletonSelected(const FAssetData& SelectedAsset)
{
	TargetSkeleton = Cast<USkeleton>(SelectedAsset.GetAsset());
	PickerWindow->RequestDestroyWindow();
}

bool UPoseSearchSchemaFactory::ConfigureProperties()
{
	// Load the content browser module to display an asset picker
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	FAssetPickerConfig AssetPickerConfig;
	
	/** The asset picker will only show skeletons */
	AssetPickerConfig.Filter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = true;

	/** The delegate that fires when an asset was selected */
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateUObject(this, &UPoseSearchSchemaFactory::OnTargetSkeletonSelected);

	/** The default view mode should be a list view */
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

	PickerWindow = SNew(SWindow)
	.Title(LOCTEXT("CreatePoseSearchSchemaOptions", "Pick Skeleton"))
	.ClientSize(FVector2D(500, 600))
	.SupportsMinimize(false) .SupportsMaximize(false)
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		]
	];

	GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
	PickerWindow.Reset();

	return TargetSkeleton != nullptr;
}

UObject* UPoseSearchSchemaFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UPoseSearchSchema* Schema = nullptr;
	if (TargetSkeleton)
	{
		Schema = NewObject<UPoseSearchSchema>(InParent, Class, Name, Flags);
		Schema->AddSkeleton(TargetSkeleton);
		Schema->AddDefaultChannels();
	}
	return Schema;
}

FString UPoseSearchSchemaFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("NewPoseSearchSchema"));
}

#undef LOCTEXT_NAMESPACE
