// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPoseSearchDatabaseViewport.h"
#include "Framework/Application/SlateApplication.h"
#include "PoseSearchDatabaseViewportClient.h"
#include "PoseSearchDatabaseViewModel.h"
#include "PoseSearchDatabaseEditor.h"
#include "SPoseSearchDatabaseViewportToolbar.h"
#include "PoseSearchDatabaseEditorCommands.h"

#include "SSimpleTimeSlider.h"
#include "Viewports.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

namespace UE::PoseSearch
{
	void SDatabaseViewport::Construct(
		const FArguments& InArgs,
		const FDatabasePreviewRequiredArgs& InRequiredArgs)
	{
		PreviewScenePtr = InRequiredArgs.PreviewScene;
		AssetEditorPtr = InRequiredArgs.AssetEditor;

		SEditorViewport::Construct(
			SEditorViewport::FArguments()
			.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
			.AddMetaData<FTagMetaData>(TEXT("AnimationTools.Viewport"))
		);
	}

	void SDatabaseViewport::OnToggleDisplayRootMotionSpeed()
	{
		AssetEditorPtr.Pin()->GetViewModelSharedPtr().ToSharedRef()->ToggleDisplayRootMotionSpeed();
		Invalidate();
	}

	bool SDatabaseViewport::IsDisplayRootMotionSpeedChecked() const
	{
		return AssetEditorPtr.Pin()->GetViewModelSharedPtr().ToSharedRef()->IsDisplayRootMotionSpeedChecked();
	}

	void SDatabaseViewport::OnToggleQuantizeAnimationToPoseData()
	{
		AssetEditorPtr.Pin()->GetViewModelSharedPtr().ToSharedRef()->ToggleQuantizeAnimationToPoseData();
		Invalidate();
	}

	bool SDatabaseViewport::IsQuantizeAnimationToPoseDataChecked() const
	{
		return AssetEditorPtr.Pin()->GetViewModelSharedPtr().ToSharedRef()->IsQuantizeAnimationToPoseDataChecked();
	}

	void SDatabaseViewport::OnToggleShowBones()
	{
		AssetEditorPtr.Pin()->GetViewModelSharedPtr().ToSharedRef()->ToggleShowBones();
		Invalidate();
	}

	bool SDatabaseViewport::IsShowBonesChecked() const
	{
		return AssetEditorPtr.Pin()->GetViewModelSharedPtr().ToSharedRef()->IsShowBonesChecked();
	}

	void SDatabaseViewport::OnToggleDisplayBlockTransition()
	{
		AssetEditorPtr.Pin()->GetViewModelSharedPtr().ToSharedRef()->ToggleDisplayBlockTransition();
		Invalidate();
	}

	bool SDatabaseViewport::IsDisplayBlockTransitionChecked() const
	{
		return AssetEditorPtr.Pin()->GetViewModelSharedPtr().ToSharedRef()->IsDisplayBlockTransitionChecked();
	}

	void SDatabaseViewport::OnToggleDisplayEventData()
	{
		AssetEditorPtr.Pin()->GetViewModelSharedPtr().ToSharedRef()->ToggleDisplayEventData();
		Invalidate();
	}

	bool SDatabaseViewport::IsDisplayEventDataChecked() const
	{
		return AssetEditorPtr.Pin()->GetViewModelSharedPtr().ToSharedRef()->IsDisplayEventDataChecked();
	}

	void SDatabaseViewport::BindCommands()
	{
		SEditorViewport::BindCommands();

		const FDatabaseEditorCommands& Commands = FDatabaseEditorCommands::Get();

		CommandList->MapAction(
			Commands.ShowDisplayRootMotionSpeed,
			FExecuteAction::CreateSP(this, &SDatabaseViewport::OnToggleDisplayRootMotionSpeed),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDatabaseViewport::IsDisplayRootMotionSpeedChecked));

		CommandList->MapAction(
			Commands.ShowQuantizeAnimationToPoseData,
			FExecuteAction::CreateSP(this, &SDatabaseViewport::OnToggleQuantizeAnimationToPoseData),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDatabaseViewport::IsQuantizeAnimationToPoseDataChecked));

		CommandList->MapAction(
			Commands.ShowBones,
			FExecuteAction::CreateSP(this, &SDatabaseViewport::OnToggleShowBones),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDatabaseViewport::IsShowBonesChecked));
		
		CommandList->MapAction(
			Commands.ShowDisplayBlockTransition,
			FExecuteAction::CreateSP(this, &SDatabaseViewport::OnToggleDisplayBlockTransition),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDatabaseViewport::IsDisplayBlockTransitionChecked));

		CommandList->MapAction(
			Commands.ShowDisplayEventData,
			FExecuteAction::CreateSP(this, &SDatabaseViewport::OnToggleDisplayEventData),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDatabaseViewport::IsDisplayEventDataChecked));
	}

	TSharedRef<FEditorViewportClient> SDatabaseViewport::MakeEditorViewportClient()
	{
		ViewportClient = MakeShared<FDatabaseViewportClient>(
			PreviewScenePtr.Pin().ToSharedRef(),
			SharedThis(this),
			AssetEditorPtr.Pin().ToSharedRef());
		ViewportClient->ViewportType = LVT_Perspective;
		ViewportClient->bSetListenerPosition = false;
		ViewportClient->SetViewLocation(EditorViewportDefs::DefaultPerspectiveViewLocation);
		ViewportClient->SetViewRotation(EditorViewportDefs::DefaultPerspectiveViewRotation);

		return ViewportClient.ToSharedRef();
	}

	TSharedPtr<SWidget> SDatabaseViewport::BuildViewportToolbar()
	{
		return SAssignNew(ViewportToolbar, SPoseSearchDatabaseViewportToolBar, SharedThis(this));
	}


	TSharedRef<SEditorViewport> SDatabaseViewport::GetViewportWidget()
	{
		return SharedThis(this);
	}

	TSharedPtr<FExtender> SDatabaseViewport::GetExtenders() const
	{
		TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
		return Result;
	}

	void SDatabaseViewport::OnFloatingButtonClicked()
	{
	}

	void SDatabasePreview::Construct(const FArguments& InArgs, const FDatabasePreviewRequiredArgs& InRequiredArgs)
	{
		SliderColor = InArgs._SliderColor;
		SliderScrubTime = InArgs._SliderScrubTime;
		SliderViewRange = InArgs._SliderViewRange;
		OnSliderScrubPositionChanged = InArgs._OnSliderScrubPositionChanged;

		OnBackwardEnd = InArgs._OnBackwardEnd;
		OnBackwardStep = InArgs._OnBackwardStep;
		OnBackward = InArgs._OnBackward;
		OnPause = InArgs._OnPause;
		OnForward = InArgs._OnForward;
		OnForwardStep = InArgs._OnForwardStep;
		OnForwardEnd = InArgs._OnForwardEnd;

		FSlimHorizontalToolBarBuilder ToolBarBuilder(
			TSharedPtr<const FUICommandList>(), 
			FMultiBoxCustomization::None, 
			nullptr, true);

		auto AddToolBarButton = [&ToolBarBuilder](FName ButtonImageName, FOnButtonClickedEvent& OnClicked)
			{
				ToolBarBuilder.AddToolBarWidget(
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
					.OnClicked_Lambda([&OnClicked]()
						{
							if (OnClicked.IsBound())
							{
								OnClicked.Execute();
								return FReply::Handled();
							}
							return FReply::Unhandled();
						})
					[
						SNew(SImage)
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
							.Image(FAppStyle::Get().GetBrush(ButtonImageName))
					]);
			};

		//ToolBarBuilder.SetStyle(&FAppStyle::Get(), "PaletteToolBar");
		ToolBarBuilder.BeginSection("Preview");
		{
			AddToolBarButton("Animation.Backward_End", OnBackwardEnd);
			AddToolBarButton("Animation.Backward_Step", OnBackwardStep);
			AddToolBarButton("Animation.Backward", OnBackward);
			AddToolBarButton("Animation.Pause", OnPause);
			AddToolBarButton("Animation.Forward", OnForward);
			AddToolBarButton("Animation.Forward_Step", OnForwardStep);
			AddToolBarButton("Animation.Forward_End", OnForwardEnd);
		}

		ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SDatabaseViewport, InRequiredArgs)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					ToolBarBuilder.MakeWidget()
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SSimpleTimeSlider)
					.ClampRangeHighlightSize(0.15f)
					.ClampRangeHighlightColor_Lambda([this]()
						{
							return SliderColor.Get();
						})
					.ScrubPosition_Lambda([this]()
						{
							return SliderScrubTime.Get();
						})
					.ViewRange_Lambda([this]()
						{
							return SliderViewRange.Get();
						})
					.ClampRange_Lambda([this]()
						{
							return SliderViewRange.Get();
						})
					.OnScrubPositionChanged_Lambda([this](double NewScrubTime, bool bIsScrubbing)
						{
							if (bIsScrubbing)
							{
								OnSliderScrubPositionChanged.ExecuteIfBound(NewScrubTime, bIsScrubbing);
							}
						})
				]
			]
		];
	}
}

