// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPoseSearchDatabaseViewportToolbar.h"
#include "SPoseSearchDatabaseViewport.h"
#include "PreviewProfileController.h"
#include "PoseSearchDatabaseEditorCommands.h"

#define LOCTEXT_NAMESPACE "PoseSearchDatabaseViewportToolBar"

namespace UE::PoseSearch
{
	void SPoseSearchDatabaseViewportToolBar::Construct(
		const FArguments& InArgs,
		TSharedPtr<SDatabaseViewport> InViewport)
	{
		SCommonEditorViewportToolbarBase::Construct(
			SCommonEditorViewportToolbarBase::FArguments()
			.AddRealtimeButton(false)
			.PreviewProfileController(MakeShared<FPreviewProfileController>()),
			InViewport);
	}

	TSharedRef<SWidget> SPoseSearchDatabaseViewportToolBar::GenerateShowMenu() const
	{
		GetInfoProvider().OnFloatingButtonClicked();

		TSharedRef<SEditorViewport> ViewportRef = GetInfoProvider().GetViewportWidget();

		const bool bInShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder ShowMenuBuilder(bInShouldCloseWindowAfterMenuSelection, ViewportRef->GetCommandList());
		{
			ShowMenuBuilder.BeginSection("Debug", LOCTEXT("ShowMenu_DebugLabel", "Debug"));
			const FDatabaseEditorCommands& Commands = FDatabaseEditorCommands::Get();
			ShowMenuBuilder.AddMenuEntry(Commands.ShowDisplayRootMotionSpeed);
			ShowMenuBuilder.AddMenuEntry(Commands.ShowQuantizeAnimationToPoseData);
			ShowMenuBuilder.AddMenuEntry(Commands.ShowBones);
			ShowMenuBuilder.AddMenuEntry(Commands.ShowDisplayBlockTransition);
			ShowMenuBuilder.AddMenuEntry(Commands.ShowDisplayEventData);
			ShowMenuBuilder.EndSection();
		}

		return ShowMenuBuilder.MakeWidget();
	}
}

#undef LOCTEXT_NAMESPACE
