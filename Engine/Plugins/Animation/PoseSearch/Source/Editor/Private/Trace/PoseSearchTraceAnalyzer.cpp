// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchTraceAnalyzer.h"

#include "Common/ProviderLock.h"
#include "HAL/LowLevelMemTracker.h"
#include "PoseSearchTraceProvider.h"
#include "PoseSearch/Trace/PoseSearchTraceLogger.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "PoseSearch/PoseSearchCustomVersion.h"

namespace UE::PoseSearch
{

FTraceAnalyzer::FTraceAnalyzer(TraceServices::IAnalysisSession& InSession, FTraceProvider& InTraceProvider)
: Session(InSession), TraceProvider(InTraceProvider)
{
}

void FTraceAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	FInterfaceBuilder& Builder = Context.InterfaceBuilder;
	
	Builder.RouteEvent(RouteId_MotionMatchingState, "PoseSearch", "MotionMatchingState");
	Builder.RouteEvent(RouteId_MotionMatchingState2, "PoseSearch", "MotionMatchingState2");
	Builder.RouteEvent(RouteId_MotionMatchingState3, "PoseSearch", "MotionMatchingState3");
}

bool FTraceAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/PoseSearch::FTraceAnalyzer"));

	TraceServices::FProviderEditScopeLock ProviderEditScope(TraceProvider);

	FPoseSearchCustomVersion::Type CustomVersionType = FPoseSearchCustomVersion::BeforeCustomVersionWasAdded;

	switch (RouteId)
	{
	case RouteId_MotionMatchingState:
		// Ensure we have not versions.
		break;
	case RouteId_MotionMatchingState2:
		// Ensure we have the version that matches this RouteId event.
		CustomVersionType = FPoseSearchCustomVersion::DeprecatedTrajectoryTypes;
		break;
	case RouteId_MotionMatchingState3:
		// Ensure we have the version that matches this RouteId event.
		CustomVersionType = FPoseSearchCustomVersion::AddedInterruptModeToDebugger;
		break;
	default:
		// Should not happen...
		checkNoEntry();
		break;
	}

	FTraceMotionMatchingStateMessage Message;
	Message.Input(CustomVersionType, Context.EventData.GetArrayView<uint8>("Data"));
	TraceProvider.AppendMotionMatchingState(Message, Context.EventTime.AsSeconds(Message.Cycle));

	return true;
}

} // namespace UE::PoseSearch
