// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Trace/Analyzer.h"

namespace TraceServices
{
class IAnalysisSession;
}

namespace UE::PoseSearch
{ 

class FTraceProvider;

/**
* Used to relay trace information to the pose search provider
*/
class FTraceAnalyzer : public Trace::IAnalyzer
{
public:
	FTraceAnalyzer(TraceServices::IAnalysisSession& InSession, FTraceProvider& InTraceProvider);

protected:
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	/** Event types for routing to the provider */
	enum : uint16
	{
		RouteId_MotionMatchingState,
		RouteId_MotionMatchingState2,
		RouteId_MotionMatchingState3,
	};

	TraceServices::IAnalysisSession& Session;
	FTraceProvider& TraceProvider;
};

} // namespace UE::PoseSearch
