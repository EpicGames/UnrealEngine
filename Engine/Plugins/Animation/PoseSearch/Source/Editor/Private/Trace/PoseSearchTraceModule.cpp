// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchTraceModule.h"
#include "PoseSearchTraceAnalyzer.h"
#include "PoseSearchTraceProvider.h"

namespace UE::PoseSearch
{

const FName FTraceModule::ModuleName("PoseSearchTrace");

void FTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("PoseSearch");
}

void FTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	// Add our provider and analyzer, starting our systems
	TSharedPtr<FTraceProvider> PoseSearchProvider = MakeShared<FTraceProvider>(InSession);
	InSession.AddProvider(FTraceProvider::ProviderName, PoseSearchProvider);
	InSession.AddAnalyzer(new FTraceAnalyzer(InSession, *PoseSearchProvider));
}

void FTraceModule::GetLoggers(TArray<const TCHAR*>& OutLoggers)
{
	OutLoggers.Add(TEXT("PoseSearch"));
}

void FTraceModule::GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
{
}

} // namespace UE::PoseSearch
