// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraceServices/ModuleService.h"

namespace UE::PoseSearch
{

/**
* Module used for loading our PoseSearch tracing systems
*/
class FTraceModule : public TraceServices::IModule
{
public:
	FTraceModule() = default;
	virtual ~FTraceModule() = default;
	
	const static FName ModuleName;

protected:
	virtual void GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo) override;
	virtual void OnAnalysisBegin(TraceServices::IAnalysisSession& InSession) override;
	virtual void GetLoggers(TArray<const TCHAR*>& OutLoggers) override;
	virtual void GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory) override;
	virtual const TCHAR* GetCommandLineArgument() override { return TEXT("posesearchtrace"); }
};

} // namespace UE::PoseSearch