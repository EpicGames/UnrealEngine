// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchTraceProvider.h"

namespace UE::PoseSearch
{

thread_local TraceServices::FProviderLock::FThreadLocalState GPoseSearchProviderLockState;

const FName FTraceProvider::ProviderName("PoseSearchTraceProvider");

FTraceProvider::FTraceProvider(TraceServices::IAnalysisSession& InSession) : Session(InSession)
{
}

bool FTraceProvider::ReadMotionMatchingStateTimeline(uint64 InAnimInstanceId, int32 InSearchId, TFunctionRef<void(const FMotionMatchingStateTimeline&)> Callback) const
{
	ReadAccessCheck();
	return MotionMatchingStateTimelineStorage.ReadTimeline(InAnimInstanceId, InSearchId, Callback);
}

bool FTraceProvider::EnumerateMotionMatchingStateTimelines(uint64 InAnimInstanceId, TFunctionRef<void(const FMotionMatchingStateTimeline&)> Callback) const
{
	ReadAccessCheck();
	return MotionMatchingStateTimelineStorage.EnumerateSearchTimelines(InAnimInstanceId, Callback);
}

void FTraceProvider::AppendMotionMatchingState(const FTraceMotionMatchingStateMessage& InMessage, double InTime)
{
	EditAccessCheck();

	TSharedRef<TraceServices::TPointTimeline<FTraceMotionMatchingStateMessage>> Timeline = MotionMatchingStateTimelineStorage.GetTimeline(Session, InMessage.AnimInstanceId, InMessage.GetSearchId());
	Timeline->AppendEvent(InTime, InMessage);

	{
		TraceServices::FAnalysisSessionEditScope SessionEditScope(Session);
		Session.UpdateDurationSeconds(InTime);
	}
}

} // namespace UE::PoseSearch
