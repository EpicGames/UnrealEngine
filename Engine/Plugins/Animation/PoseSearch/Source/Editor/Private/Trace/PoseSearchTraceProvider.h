// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/ProviderLock.h"
#include "Model/PointTimeline.h"
#include "PoseSearch/Trace/PoseSearchTraceLogger.h"
#include "TraceServices/Model/AnalysisSession.h"

namespace UE::PoseSearch
{

extern thread_local TraceServices::FProviderLock::FThreadLocalState GPoseSearchProviderLockState;

/**
 * Provider to the widgets for pose search functionality, largely mimicking FAnimationProvider
 */
class FTraceProvider : public TraceServices::IProvider, public TraceServices::IEditableProvider
{
public:
	explicit FTraceProvider(TraceServices::IAnalysisSession& InSession);

	//~ IProvider (read lock)
	virtual void BeginRead() const override
	{
		Lock.BeginRead(GPoseSearchProviderLockState);
	}
	virtual void EndRead() const override
	{
		Lock.EndRead(GPoseSearchProviderLockState);
	}
	virtual void ReadAccessCheck() const override
	{
		Lock.ReadAccessCheck(GPoseSearchProviderLockState);
	}

	//~ IEditableProvider (edit lock)
	virtual void BeginEdit() const override
	{
		Lock.BeginWrite(GPoseSearchProviderLockState);
	}
	virtual void EndEdit() const override
	{
		Lock.EndWrite(GPoseSearchProviderLockState);
	}
	virtual void EditAccessCheck() const override
	{
		Lock.WriteAccessCheck(GPoseSearchProviderLockState);
	}

	using FMotionMatchingStateTimeline = TraceServices::ITimeline<FTraceMotionMatchingStateMessage>;
	
	/** Read the node-relative timeline info of the AnimInstance and provide it via callback */
	bool ReadMotionMatchingStateTimeline(uint64 InAnimInstanceId, int32 InSearchId, TFunctionRef<void(const FMotionMatchingStateTimeline&)> Callback) const;
	
	/** Enumerate all node timelines on the AnimInstance and provide them via callback */
	bool EnumerateMotionMatchingStateTimelines(uint64 InAnimInstanceId, TFunctionRef<void(const FMotionMatchingStateTimeline&)> Callback) const;

	/** Append to the timeline on our object */
	void AppendMotionMatchingState(const FTraceMotionMatchingStateMessage& InMessage, double InTime);

	
	static const FName ProviderName;

private:
	/**
	 * Convenience struct for creating timelines per message type.
	 * We store an array of timelines for every possible AnimInstance Id that gets appended.
	 */
	template <class MessageType, class TimelineType = TraceServices::TPointTimeline<MessageType>>
	struct TTimelineStorage
	{
		// Map the timelines as AnimInstanceId -> SearchId -> Timeline
		using FSearchIdToTimelineMap = TMap<int32, uint32>;
		using FAnimInstanceToTimelineMap = TMap<uint64, FSearchIdToTimelineMap>;

		/** Retrieves the timeline for internal use, creating it if it does not exist */
		TSharedRef<TimelineType> GetTimeline(TraceServices::IAnalysisSession& InSession, uint64 InAnimInstanceId, int32 InSearchId)
		{
			const uint32* TimelineIndex = nullptr;
			
			FSearchIdToTimelineMap* SearchIdToTimelineMap = AnimInstanceIdToTimelines.Find(InAnimInstanceId);
			if (SearchIdToTimelineMap == nullptr)
			{
				// Create the timeline map if this is the first time accessing with this anim instance
				SearchIdToTimelineMap = &AnimInstanceIdToTimelines.Add(InAnimInstanceId, {});
			}
			else
			{
				// Anim instance already used, attempt to find the SearchId's timeline
				TimelineIndex = SearchIdToTimelineMap->Find(InSearchId);
			}

			if (TimelineIndex == nullptr)
			{
				// Append our timeline to the storage and our object + the storage index to the map
				TSharedRef<TimelineType> Timeline = MakeShared<TimelineType>(InSession.GetLinearAllocator());
				const uint32 NewIndex = Timelines.Add(Timeline);
				SearchIdToTimelineMap->Add(InSearchId, NewIndex);
				return Timeline;
			}

			return Timelines[*TimelineIndex];
		}

		bool ReadSearchTimeline(const FSearchIdToTimelineMap* SearchIdToTimelineMap, int32 InSearchId, TFunctionRef<void(const TraceServices::ITimeline<MessageType>&)> Callback) const
		{
			const uint32* TimelineIndex = SearchIdToTimelineMap->Find(InSearchId);
			if (TimelineIndex == nullptr || !Timelines.IsValidIndex(*TimelineIndex))
			{
				return false;
			}

			Callback(*Timelines[*TimelineIndex]);
			return true;
		}

		/** Retrieve a timeline from an anim instance + node and execute the callback */
		bool ReadTimeline(uint64 InAnimInstanceId, int32 InSearchId, TFunctionRef<void(const TraceServices::ITimeline<MessageType>&)> Callback) const
		{
			const FSearchIdToTimelineMap* SearchIdToTimelineMap = AnimInstanceIdToTimelines.Find(InAnimInstanceId);
			if (SearchIdToTimelineMap == nullptr)
			{
				return false;
			}

			return ReadSearchTimeline(SearchIdToTimelineMap, InSearchId, Callback);
		}

		/** Enumerates timelines of all nodes given an AnimInstance */
		bool EnumerateSearchTimelines(uint64 InAnimInstanceId, TFunctionRef<void(const TraceServices::ITimeline<MessageType>&)> Callback) const
		{
			const FSearchIdToTimelineMap* SearchIdToTimelineMap = AnimInstanceIdToTimelines.Find(InAnimInstanceId);
			if (SearchIdToTimelineMap == nullptr)
			{
				return false;
			}

			bool bSuccess = true;
			for (const TPair<int32, uint32>& Pair : *SearchIdToTimelineMap)
			{
				bSuccess &= ReadSearchTimeline(SearchIdToTimelineMap, Pair.Key, Callback);
				if (!bSuccess)
				{
					break;
				}
			}

			return bSuccess;
		}

		TSet<int32> GetSearchIds(uint64 InAnimInstanceId) const
		{
			TSet<int32> SearchIds;
			if (!AnimInstanceIdToTimelines.Find(InAnimInstanceId))
			{
				return SearchIds;
			}
			const FSearchIdToTimelineMap& SearchIdToTimelineMap = AnimInstanceIdToTimelines[InAnimInstanceId];

			// For each node that has a timeline, add the node to the set
			for(const TTuple<int32, uint32>& Pair : SearchIdToTimelineMap)
			{
				SearchIds.Add(Pair.Key);
			}
			
			return SearchIds;
		}

		/** Maps AnimInstanceIds to a map of SearchIds to timelines. AnimInstanceId -> SearchId -> Timeline */
		FAnimInstanceToTimelineMap AnimInstanceIdToTimelines;

		/** Timelines per node */
		TArray<TSharedRef<TimelineType>> Timelines;
	};

	// Storage for each message type
	struct FMotionMatchingStateTimelineStorage : TTimelineStorage<FTraceMotionMatchingStateMessage>
	{
	} MotionMatchingStateTimelineStorage;


	TraceServices::IAnalysisSession& Session;
	mutable TraceServices::FProviderLock Lock;
};
} // namespace UE::PoseSearch
