// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_BlendStack.h"
#include "PoseSearch/AnimNode_MotionMatching.h"
#include "Animation/AnimNodeFunctionRef.h"
#include "AnimGraphNode_MotionMatching.generated.h"


UCLASS(MinimalAPI)
class UAnimGraphNode_MotionMatching : public UAnimGraphNode_BlendStack_Base
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_MotionMatching Node;

	// UAnimGraphNode_Base interface
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetMenuCategory() const override;
	virtual UScriptStruct* GetTimePropertyStruct() const override;
	virtual void OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	virtual void AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const override;
	virtual void GetBoundFunctionsInfo(TArray<TPair<FName, FName>>& InOutBindingsInfo) override;
	virtual bool ReferencesFunction(const FName& InFunctionName, const UStruct* InScope) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_BlendStack_Base interface
	virtual FAnimNode_BlendStack_Standalone* GetBlendStackNode() const override { return (FAnimNode_BlendStack_Standalone*)(&Node); }
	// End of UAnimGraphNode_BlendStack_Base interface
	
	UPROPERTY(EditAnywhere, Category = "Functions|Motion Matching", meta=(FunctionReference, AllowFunctionLibraries, PrototypeFunction="/Script/AnimGraphRuntime.AnimExecutionContextLibrary.Prototype_ThreadSafeAnimUpdateCall", DefaultBindingName="OnStateExit"), DisplayName="On Motion Matching State Updated")
	FMemberReference OnMotionMatchingStateUpdatedFunction;
};
