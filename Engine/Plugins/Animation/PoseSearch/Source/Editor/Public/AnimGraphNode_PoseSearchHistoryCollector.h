// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_Base.h"
#include "CoreMinimal.h"
#include "PoseSearch/AnimNode_PoseSearchHistoryCollector.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_PoseSearchHistoryCollector.generated.h"

UCLASS(Abstract, BlueprintType)
class UAnimGraphNode_PoseSearchHistoryCollector_Base : public UAnimGraphNode_Base
{
	GENERATED_BODY()
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetMenuCategory() const override;

public:

	//~ Begin UK2Node Interface.
	virtual void ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const override;
	virtual void PostReconstructNode() override;
	virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;
	//~ End UK2Node Interface.
};

UCLASS(MinimalAPI)
class UAnimGraphNode_PoseSearchHistoryCollector : public UAnimGraphNode_PoseSearchHistoryCollector_Base
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_PoseSearchHistoryCollector Node;

	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
};

UCLASS(MinimalAPI)
class UAnimGraphNode_PoseSearchComponentSpaceHistoryCollector : public UAnimGraphNode_PoseSearchHistoryCollector_Base
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_PoseSearchComponentSpaceHistoryCollector Node;
	
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void CreateOutputPins() override;

};
