// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// @todo: move UMultiAnimAsset as well as IMultiAnimAssetEditor to Engine or a base plugin for multi character animation assets

#include "Toolkits/AssetEditorToolkit.h"

class IMultiAnimAssetEditor : public FAssetEditorToolkit
{
public:
	virtual void SetPreviewProperties(float AnimAssetTime, const FVector& AnimAssetBlendParameters, bool bAnimAssetPlaying) = 0;
};