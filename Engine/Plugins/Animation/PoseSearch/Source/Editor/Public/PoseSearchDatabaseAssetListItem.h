// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PoseSearchDatabaseAssetTreeNode.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FUICommandList;

namespace UE::PoseSearch
{
	class FDatabaseViewModel;
	class SDatabaseAssetTree;

	class SDatabaseAssetListItem : public STableRow<TSharedPtr<FDatabaseAssetTreeNode>>
	{
	public:
		void Construct(
			const FArguments& InArgs,
			const TSharedRef<FDatabaseViewModel>& InEditorViewModel,
			const TSharedRef<STableViewBase>& OwnerTable,
			TSharedRef<FDatabaseAssetTreeNode> InAssetTreeNode,
			TSharedRef<FUICommandList> InCommandList,
			TSharedPtr<SDatabaseAssetTree> InHierarchy);

	protected:
		FText GetName() const;
		TSharedRef<SWidget> GenerateItemWidget();

		const FSlateBrush* GetGroupBackgroundImage() const;
		void ConstructGroupItem(const TSharedRef<STableViewBase>& OwnerTable);
		void ConstructAssetItem(const TSharedRef<STableViewBase>& OwnerTable);

		virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
		
		EVisibility GetSelectedActorIconVisibility() const;

		void OnAssetPickerObjectChanged(const FAssetData& AssetData);
		FString GetAssetPickerObjectPath() const;
		bool GetAssetPickerIsEnabled() const;
		EVisibility GetAssetPickerCustomContentSlotVisibility() const;
		FText GetAssetPickerText() const;
		FSlateColor GetAssetPickerCustomContentSlotTextColor() const;
		
		FSlateColor GetLoopingColorAndOpacity() const;
		FText GetLoopingToolTip() const;
		
		FSlateColor GetRootMotionColorAndOpacity() const;
		FText GetRootMotionOptionToolTip() const;
		
		FText GetDisableReselectionToolTip() const;
		ECheckBoxState GetDisableReselectionChecked() const;
		void OnDisableReselectionChanged(ECheckBoxState NewCheckboxState);

		FText GetAssetEnabledToolTip() const;
		ECheckBoxState GetAssetEnabledChecked() const;
		void OnAssetIsEnabledChanged(ECheckBoxState NewCheckboxState);

		TWeakPtr<FDatabaseAssetTreeNode> WeakAssetTreeNode;
		TWeakPtr<FDatabaseViewModel> EditorViewModel;
		TWeakPtr<SDatabaseAssetTree> SkeletonView;
		
		FColor AssetTypeColor;
		TSharedPtr<SOverlay> AssetThumbnailOverlay;
	};
}

