// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearchDatabaseAssetListItem.h"
#include "DetailColumnSizeData.h"
#include "EditorUndoClient.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandList;

namespace UE::PoseSearch
{
	class FDatabaseViewModel;

	class SDatabaseAssetTree : public SCompoundWidget, public FSelfRegisteringEditorUndoClient
	{
	public:
		SLATE_BEGIN_ARGS(SDatabaseAssetTree) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<FDatabaseViewModel> InEditorViewModel);

		// SWidget interface
		virtual bool SupportsKeyboardFocus() const override;
		virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
		virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
		virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
		// End SWidget interface

		// Begin FEditorUndoClient interface
		virtual bool MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const override;
		virtual void PostUndo(bool bSuccess) override;
		virtual void PostRedo(bool bSuccess) override;
		// End FEditorUndoClient interface
		
		void RefreshTreeView(bool bIsInitialSetup = false, bool bRecoverSelection = false);
		void FinalizeTreeChanges(bool bRecoverSelection = false, bool bRefreshView = true);

		void SetSelectedItem(int32 SourceAssetIdx, bool bClearSelection);
		TArray<TSharedPtr<FDatabaseAssetTreeNode>> GetSelectedNodes() const { return TreeView->GetSelectedItems(); }
		
	protected:
		TWeakPtr<FDatabaseViewModel> EditorViewModel;

		/** command list we bind to */
		TSharedPtr<FUICommandList> CommandList;

		/** tree view widget */
		TSharedPtr<STreeView<TSharedPtr<FDatabaseAssetTreeNode>>> TreeView;
		TSharedPtr<SWidget> TreeViewDragAndDropSuggestion;
		TArray<TSharedPtr<FDatabaseAssetTreeNode>> RootNodes;

		TSharedRef<ITableRow> MakeTableRowWidget(
			TSharedPtr<FDatabaseAssetTreeNode> InItem,
			const TSharedRef<STableViewBase>& OwnerTable);
		void HandleGetChildrenForTree(
			TSharedPtr<FDatabaseAssetTreeNode> InNode, 
			TArray<TSharedPtr<FDatabaseAssetTreeNode>>& OutChildren);

		TOptional<EItemDropZone> OnCanAcceptDrop(
			const FDragDropEvent& DragDropEvent, 
			EItemDropZone DropZone, 
			TSharedPtr<FDatabaseAssetTreeNode> TargetItem);

		FReply OnAcceptDrop(
			const FDragDropEvent& DragDropEvent,
			EItemDropZone DropZone,
			TSharedPtr<FDatabaseAssetTreeNode> TargetItem);

		TSharedPtr<SWidget> CreateContextMenu();

		TSharedRef<SWidget> GenerateFilterBoxWidget();

		FText GetFilterText() const;
		void OnAssetFilterTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);

		void SetAssetFilterString(FString InString) { AssetFilterString = InString; }
		FString GetAssetFilterString() const { return AssetFilterString; }

		//String Filter For Database View
		FString AssetFilterString;

		void OnAddAsset(bool bFinalizeChanges = true);
		void OnDeleteAsset(TSharedPtr<FDatabaseAssetTreeNode> Node, bool bFinalizeChanges = true);
		void CreateCommandList();

		/** Removes existing selected component nodes from the tree*/
		bool CanDeleteNodes() const;
		void OnDeleteNodes();
		
		/** Copy selected nodes to clipboard */
		void OnCopySelectedNodesToClipboard() const;
		bool CanCopyToClipboard() const;

		/** Paste nodes from clipboard. Adds or overwrites curves (if identifiers collide) */
		void OnPasteNodesFromClipboard();
		bool CanPasteFromClipboard();

		/** Cut selected nodes to clipboard */
		void OnCutSelectedNodesToClipboard();
		bool CanCutToClipboard() const;
		
		void EnableSelectedNodes(bool bIsEnabled);
		void OnEnableNodes() { EnableSelectedNodes(true); }
		void OnDisableNodes() { EnableSelectedNodes(false); }
		void OnSetMirrorOptionForSelectedNodes(EPoseSearchMirrorOption InMirrorOption);
		void OnSetPoseReselectionForSelectedNodes(bool bIsEnabled);
		
		void OnConvertToBranchIn();

		friend SDatabaseAssetListItem;
		
	protected:
		// Called when an item is selected/deselected
		DECLARE_MULTICAST_DELEGATE_TwoParams(
			FOnSelectionChangedMulticaster, 
			const TArrayView<TSharedPtr<FDatabaseAssetTreeNode>>& /* InSelectedItems */,
			ESelectInfo::Type /* SelectInfo */);

		FOnSelectionChangedMulticaster OnSelectionChanged;

	public:
		typedef FOnSelectionChangedMulticaster::FDelegate FOnSelectionChanged;
		void RegisterOnSelectionChanged(const FOnSelectionChanged& Delegate);
		void UnregisterOnSelectionChanged(FDelegateUserObject Unregister);

		void RecoverSelection(const TArray<TSharedPtr<FDatabaseAssetTreeNode>>& PreviouslySelectedNodes);
	};
}

