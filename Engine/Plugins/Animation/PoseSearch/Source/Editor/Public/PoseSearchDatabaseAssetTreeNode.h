// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

enum class EPoseSearchMirrorOption;
class FUICommandList;
class ITableRow;
class STableViewBase;

namespace UE::PoseSearch
{
	class FDatabaseViewModel;
	class SDatabaseAssetTree;

	class FDatabaseAssetTreeNode : public TSharedFromThis<FDatabaseAssetTreeNode>
	{

	public:
		FDatabaseAssetTreeNode(int32 InSourceAssetIdx, const TSharedRef<FDatabaseViewModel>& InEditorViewModel);
		TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FDatabaseAssetTreeNode> InDatabaseAssetNode, TSharedRef<FUICommandList> InCommandList, TSharedPtr<SDatabaseAssetTree> InHierarchy);
		bool IsRootMotionEnabled() const;
		bool IsLooping() const;
		EPoseSearchMirrorOption GetMirrorOption() const;
		
		int32 SourceAssetIdx = INDEX_NONE;
		TSharedPtr<FDatabaseAssetTreeNode> Parent;
		TArray<TSharedPtr<FDatabaseAssetTreeNode>> Children;
		TWeakPtr<FDatabaseViewModel> EditorViewModel;
	};
}

