// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Views/STreeView.h"

class UPoseSearchFeatureChannel;

namespace UE::PoseSearch
{
	class FDatabaseViewModel;
	typedef TSharedPtr<class FChannelItem> FChannelItemPtr;
	typedef STreeView<FChannelItemPtr> SChannelItemsTreeView;
	struct FSearchIndex;

	class SDatabaseDataDetails : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS( SDatabaseDataDetails ) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& Args, TSharedRef<FDatabaseViewModel> InEditorViewModel);
		void Reconstruct();

	private:
		static void RebuildChannelItemsTreeRecursively(TArray<FChannelItemPtr>& ChannelItems, TConstArrayView<TObjectPtr<UPoseSearchFeatureChannel>> Channels, const FSearchIndex& SearchIndex);
		static void RebuildChannelItemsStats(TArray<FChannelItemPtr>& ChannelItems);
		static void TrackExpandedItems(const TArray<FChannelItemPtr>& ChannelItems, TMap<const FString, bool>& ExpandedItems);
		static void SetExpandedItems(TArray<FChannelItemPtr>& ChannelItems, const TMap<const FString, bool>& ExpandedItems, SChannelItemsTreeView* ChannelItemsTreeView);

		TSharedPtr<SChannelItemsTreeView> ChannelItemsTreeView;
		TArray<FChannelItemPtr> ChannelItems;
		TWeakPtr<FDatabaseViewModel> EditorViewModel;
	};
} // namespace UE::PoseSearch