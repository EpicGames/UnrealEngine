// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdMode.h"


namespace UE::PoseSearch
{
	class FDatabaseViewModel;

	class FDatabaseEdMode : public FEdMode
	{
	public:

		const static FEditorModeID EdModeId;

		FDatabaseEdMode();
		virtual ~FDatabaseEdMode();

		virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
		virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
		virtual bool HandleClick(
			FEditorViewportClient* InViewportClient,
			HHitProxy* HitProxy,
			const FViewportClick& Click) override;
		virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
		virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
		virtual bool InputDelta(
			FEditorViewportClient* InViewportClient,
			FViewport* InViewport,
			FVector& InDrag,
			FRotator& InRot,
			FVector& InScale) override;
		virtual bool InputKey(
			FEditorViewportClient* ViewportClient,
			FViewport* Viewport,
			FKey Key,
			EInputEvent Event) override;
		virtual bool AllowWidgetMove() override;
		virtual bool ShouldDrawWidget() const override;
		virtual FVector GetWidgetLocation() const override;
		virtual bool GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData) override;
		virtual bool GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData) override;
		virtual bool RequiresLegacyViewportInteractions() const override { return false; }

	private:
		FDatabaseViewModel* ViewModel = nullptr;
	};
}
