// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Misc/NotifyHook.h"

class IDetailsView;
class FToolBarBuilder;
class UPoseSearchDatabase;
class UPoseSearchDatabaseStatistics;

namespace UE::PoseSearch
{
	class FDatabaseAssetTreeNode;
	class FDatabasePreviewScene;
	class FDatabaseViewModel;
	class SDatabaseAssetTree;
	class SDatabaseDataDetails;
	class SDatabasePreview;
	class SPoseSearchDatabaseAssetBrowser;

	class FDatabaseEditor : public FAssetEditorToolkit, public FNotifyHook
	{
	public:

		void InitAssetEditor(
			const EToolkitMode::Type Mode,
			const TSharedPtr<IToolkitHost>& InitToolkitHost,
			UPoseSearchDatabase* DatabaseAsset);
		virtual ~FDatabaseEditor();

		virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
		virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
		virtual FName GetToolkitFName() const override;
		virtual FText GetBaseToolkitName() const override;
		virtual FText GetToolkitName() const override;
		virtual FLinearColor GetWorldCentricTabColorScale() const override;
		virtual FString GetWorldCentricTabPrefix() const override;

		const UPoseSearchDatabase* GetPoseSearchDatabase() const;
		UPoseSearchDatabase* GetPoseSearchDatabase();
		FDatabaseViewModel* GetViewModel() const { return ViewModel.Get(); }
		TSharedPtr<FDatabaseViewModel> GetViewModelSharedPtr() const { return ViewModel; }
		
		void SetSelectedPoseIdx(int32 PoseIdx, bool bDrawQuery, TConstArrayView<float> InQueryVector);
		void SetDrawQueryVector(bool bValue);
		
		void PreviewBackwardEnd();
		void PreviewBackwardStep();
		void PreviewBackward();
		void PreviewPause();
		void PreviewForward();
		void PreviewForwardStep();
		void PreviewForwardEnd();

	private:

		TSharedRef<SDockTab> SpawnTab_Viewport(const FSpawnTabArgs& Args);
		TSharedRef<SDockTab> SpawnTab_AssetDetails(const FSpawnTabArgs& Args);
		TSharedRef<SDockTab> SpawnTab_PreviewSettings(const FSpawnTabArgs& Args);
		TSharedRef<SDockTab> SpawnTab_AssetTreeView(const FSpawnTabArgs& Args);
		TSharedRef<SDockTab> SpawnTab_SelectionDetails(const FSpawnTabArgs& Args);
		TSharedRef<SDockTab> SpawnTab_StatisticsOverview(const FSpawnTabArgs& Args) const;
		TSharedRef<SDockTab> SpawnTab_DataDetails(const FSpawnTabArgs& Args) const;
		TSharedRef<SDockTab> SpawnTab_AssetBrowser(const FSpawnTabArgs& Args) const;
		
		void OnFinishedChangingSelectionProperties(const FPropertyChangedEvent& PropertyChangedEvent);
		void OnAssetTreeSelectionChanged(
			const TArrayView<TSharedPtr<FDatabaseAssetTreeNode>>& SelectedItems,
			ESelectInfo::Type SelectionType);

		void RefreshStatisticsWidgetInformation();
		void RefreshEditor();

		struct FSelectionWidget
		{
			TSharedPtr<IDetailsView> DetailView;
			TArray<TWeakObjectPtr<UObject>> SelectedReflections;
		};
		FSelectionWidget& GetInitializedSelectionWidget();
		
		TSharedPtr<SDatabasePreview> PreviewWidget;

		TSharedPtr<SDatabaseAssetTree> AssetTreeWidget;

		TSharedPtr<SDatabaseDataDetails> DataDetails;

		TSharedPtr<SPoseSearchDatabaseAssetBrowser> AssetBrowserWidget;
		
		TSharedPtr<IDetailsView> EditingAssetWidget;

		TSharedPtr<SVerticalBox> DetailsContainer;
		FSelectionWidget SelectionWidget;

		TSharedPtr<IDetailsView> StatisticsOverviewWidget;
		UPoseSearchDatabaseStatistics* CachedStatistics = nullptr;
		
		TSharedPtr<FDatabasePreviewScene> PreviewScene;

		TSharedPtr<FDatabaseViewModel> ViewModel;
	};
}

