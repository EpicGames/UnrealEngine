// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"


namespace UE::PoseSearch
{
	class FDatabaseEditorCommands : public TCommands<FDatabaseEditorCommands>
	{
	public:
		FDatabaseEditorCommands()
			: TCommands<FDatabaseEditorCommands>(
				TEXT("PoseSearchDatabaseEditor"),
				NSLOCTEXT("Contexts", "PoseSearchDatabase", "Pose Search Database"),
				NAME_None,
				FAppStyle::GetAppStyleSetName())
		{
		}

		virtual void RegisterCommands() override;

	public:
		TSharedPtr<FUICommandInfo> ShowDisplayRootMotionSpeed;
		TSharedPtr<FUICommandInfo> ShowQuantizeAnimationToPoseData;
		TSharedPtr<FUICommandInfo> ShowBones;
		TSharedPtr<FUICommandInfo> ShowDisplayBlockTransition;
		// Experimental, this feature might be removed without warning, not for production use
		TSharedPtr<FUICommandInfo> ShowDisplayEventData;
	};
}

