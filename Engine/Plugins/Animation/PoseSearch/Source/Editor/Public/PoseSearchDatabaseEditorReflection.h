// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearchDatabaseEditorReflection.generated.h"


namespace UE::PoseSearch
{
	class FDatabaseAssetTreeNode;
	class SDatabaseAssetTree;
}


UCLASS()
class UPoseSearchDatabaseReflectionBase : public UObject
{
	GENERATED_BODY()

public:
	int32 SourceAssetIdx = INDEX_NONE;
};

UCLASS()
class UPoseSearchDatabaseAnimationAssetReflection : public UPoseSearchDatabaseReflectionBase
{
	GENERATED_BODY()

public:
	UPROPERTY(Transient, EditAnywhere, Category = "Selected Animation Asset")
	FPoseSearchDatabaseAnimationAsset DatabaseAnimationAsset;
};

USTRUCT()
struct FPoseSearchDatabaseMemoryStats
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Stats")
	FText EstimatedDatabaseSize;

	UPROPERTY(VisibleAnywhere, Category = "Stats")
	FText ValuesSize;

	UPROPERTY(VisibleAnywhere, Category = "Stats")
	FText PCAValuesSize;

	UPROPERTY(VisibleAnywhere, Category = "Stats")
	FText KDTreeSize;

	UPROPERTY(VisibleAnywhere, Category = "Stats")
	FText PoseMetadataSize;
	
	UPROPERTY(VisibleAnywhere, Category = "Stats")
	FText AssetsSize;
	
	void Initialize(const UPoseSearchDatabase* PoseSearchDatabase);
};

UCLASS()
class UPoseSearchDatabaseStatistics : public UObject
{
	GENERATED_BODY()

public:
	
	// Number of Animation Sequences in the database.
	UPROPERTY(VisibleAnywhere, Category = "General Information")
	uint32 AnimationSequences;

	// Number of total animation poses in frames in the database.
	UPROPERTY(VisibleAnywhere, Category = "General Information")
	uint32 TotalAnimationPosesInFrames;

	// Number of total animation poses in time in the database.
	UPROPERTY(VisibleAnywhere, Category = "General Information")
	FText TotalAnimationPosesInTime;

	// Amount of animation frames that are searchable in the database (this will exclude frames that have been removed using Sampling Range).
	UPROPERTY(VisibleAnywhere, Category = "General Information")
	uint32 SearchableFrames;

	// Amount of animation in time that are searchable in the database (this will exclude time that have been removed using Sampling Range).
	UPROPERTY(VisibleAnywhere, Category = "General Information")
	FText SearchableTime;

	// Amount of animation frames (database poses) of pose features data that has been pruned out because close to each other more than PosePruningSimilarityThreshold
	UPROPERTY(VisibleAnywhere, Category = "General Information")
	uint32 PrunedFrames;

	// Amount of animation frames projected in PCA space (database poses) of pose PCA feature data (used to construct the kdtree) that has been pruned out because close to each other more than PCAValuesPruningSimilarityThreshold
	UPROPERTY(VisibleAnywhere, Category = "General Information")
	uint32 PrunedPCAFrames;

	// Cardinality for the database schema (how many floats per pose to store the pose features data)
	UPROPERTY(VisibleAnywhere, Category = "General Information")
	uint32 SchemaCardinality;

	// Average speed of the characters trajectory across all animations in the database.
	UPROPERTY(VisibleAnywhere, Category = "Kinematic Information")
	FText AverageSpeed;

	// Highest speed of the characters trajectory across all animations in the database.
	UPROPERTY(VisibleAnywhere, Category = "Kinematic Information")
	FText MaxSpeed;

	// The average acceleration of the characters trajectory across all the animations in the database.
	UPROPERTY(VisibleAnywhere, Category = "Kinematic Information")
	FText AverageAcceleration;

	// The max acceleration of the characters trajectory across all the animations in the database.
	UPROPERTY(VisibleAnywhere, Category = "Kinematic Information")
	FText MaxAcceleration;

	// When Pose Search Mode is set to PCAKDTree this value represents how well the variance of the dataset is explained within the chosen Number Of Principal Components:
	// the higher, the better the quality (statistically more significant result) out of the kdtree search.
	UPROPERTY(VisibleAnywhere, Category = "Principal Component Analysis Information", meta = (Units = "Percent"))
	float ExplainedVariance;

	// aggregated total memory used by this database (including animation assets).
	UPROPERTY(VisibleAnywhere, Category = "Memory Information")
	FText EstimatedDatabaseSize;

	// partial memory size used to store the pose feature vectors.
	UPROPERTY(VisibleAnywhere, Category = "Memory Information")
	FText ValuesSize;

	// partial memory size used to store the pose feature vectors in PCA space.
	UPROPERTY(VisibleAnywhere, Category = "Memory Information")
	FText PCAValuesSize;

	// partial memory size used by the kdtree.
	UPROPERTY(VisibleAnywhere, Category = "Memory Information")
	FText KDTreeSize;

	// partial memory size used by the vptree.
	UPROPERTY(VisibleAnywhere, Category = "Memory Information")
	FText VPTreeSize;
		
	// partial memory size used to store database metadata.
	UPROPERTY(VisibleAnywhere, Category = "Memory Information")
	FText PoseMetadataSize;
	
	// partial memory size used to animation data sub ranges, mirror state, blend parameters.
	UPROPERTY(VisibleAnywhere, Category = "Memory Information")
	FText AssetsSize;

	// Experimental, this feature might be removed without warning, not for production use
	// partial memory size used for events.
	UPROPERTY(VisibleAnywhere, Category = "Memory Information")
	FText EventDataSize;

	// Experimental, this feature might be removed without warning, not for production use
	// partial memory size used by the cooked animation assets.
	UPROPERTY(VisibleAnywhere, Category = "Memory Information")
 	FText SourceAnimAssetsSizeCookedEstimate;

	/** Initialize statistics given a database */
	void Initialize(const UPoseSearchDatabase* PoseSearchDatabase);
};