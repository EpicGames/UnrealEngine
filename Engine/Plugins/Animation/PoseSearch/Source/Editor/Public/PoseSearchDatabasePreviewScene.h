// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AdvancedPreviewScene.h"

namespace UE::PoseSearch
{
	class FDatabaseEditor;

	class FDatabasePreviewScene : public FAdvancedPreviewScene
	{
	public:

		FDatabasePreviewScene(
			ConstructionValues CVs,
			const TSharedRef<FDatabaseEditor>& Editor);
		~FDatabasePreviewScene() = default;

		virtual void Tick(float InDeltaTime) override;

		TSharedRef<FDatabaseEditor> GetEditor() const
		{
			return EditorPtr.Pin().ToSharedRef();
		}

	private:

		/** The asset editor we are embedded in */
		TWeakPtr<FDatabaseEditor> EditorPtr;
	};
}

