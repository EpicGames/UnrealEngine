// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StructUtils/InstancedStruct.h"
#include "MovieSceneFwd.h"
#include "PoseSearch/PoseSearchAssetSampler.h"
#include "PoseSearch/PoseSearchMirrorDataCache.h"
#include "PoseSearch/PoseSearchRole.h"
#include "Animation/TrajectoryTypes.h"
#include "PoseSearchDatabasePreviewScene.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "UObject/GCObject.h"

class UWorld;
class UPoseSearchDatabase;
class UAnimPreviewInstance;
class UDebugSkelMeshComponent;
class UAnimComposite;
class UAnimSequence;
class UBlendSpace;
class UMirrorDataTable;
class UMultiAnimAsset;

namespace UE::PoseSearch
{
	class FDatabaseAssetTreeNode;
	struct FSearchIndexAsset;
	class SDatabaseDataDetails;

	struct FDatabasePreviewActor
	{
	public:
		bool SpawnPreviewActor(UWorld* World, const UPoseSearchDatabase* PoseSearchDatabase, int32 IndexAssetIdx, const FRole& Role, const FTransform& SamplerRootTransformOrigin, int32 PoseIdxForTimeOffset = INDEX_NONE);
		void UpdatePreviewActor(const UPoseSearchDatabase* PoseSearchDatabase, float PlayTime, bool bQuantizeAnimationToPoseData);
		static bool DrawPreviewActors(TConstArrayView<FDatabasePreviewActor> PreviewActors, const UPoseSearchDatabase* PoseSearchDatabase, bool bDisplayRootMotionSpeed, bool bDisplayBlockTransition, bool bDisplayEventData, TConstArrayView<float> QueryVector);

		void Destroy();

		UDebugSkelMeshComponent* GetDebugSkelMeshComponent();
		const UDebugSkelMeshComponent* GetDebugSkelMeshComponent() const;
		const UAnimPreviewInstance* GetAnimPreviewInstance() const;
		const FAnimationAssetSampler& GetSampler() const { return Sampler; }

		const AActor* GetActor() const { return ActorPtr.Get(); }
		int32 GetIndexAssetIndex() const { return IndexAssetIndex; }
		int32 GetCurrentPoseIndex() const { return CurrentPoseIndex; }
		float GetPlayTimeOffset() const { return PlayTimeOffset; }
		
	private:
		UAnimPreviewInstance* GetAnimPreviewInstanceInternal();

		TWeakObjectPtr<AActor> ActorPtr;
		int32 IndexAssetIndex = INDEX_NONE;
		int32 CurrentPoseIndex = INDEX_NONE;
		float PlayTimeOffset = 0.f;
		float CurrentTime = 0.f;
		float QuantizedTime = 0.f;

		FAnimationAssetSampler Sampler;
		FTransformTrajectory Trajectory;
		TArray<float> TrajectorySpeed;

		FRole ActorRole = DefaultRole;
	};

	class FDatabaseViewModel : public TSharedFromThis<FDatabaseViewModel>, public FGCObject
	{
	public:
		// ~ FGCObject interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override { return TEXT("FPoseSearchDatabaseViewModel"); }

		void Initialize(UPoseSearchDatabase* InPoseSearchDatabase, const TSharedRef<FDatabasePreviewScene>& InPreviewScene, const TSharedRef<SDatabaseDataDetails>& InDatabaseDataDetails);

		void RemovePreviewActors();
		void BuildSearchIndex();

		void PreviewBackwardEnd();
		void PreviewBackwardStep();
		void PreviewBackward();
		void PreviewPause();
		void PreviewForward();
		void PreviewForwardStep();
		void PreviewForwardEnd();

		UPoseSearchDatabase* GetPoseSearchDatabase() { return PoseSearchDatabasePtr.Get(); }
		const UPoseSearchDatabase* GetPoseSearchDatabase() const { return PoseSearchDatabasePtr.Get(); }
		void OnPreviewActorClassChanged();

		void Tick(float DeltaSeconds);

		const TArray<TArray<FDatabasePreviewActor>>& GetPreviewActors() const { return PreviewActors; }
		TArray<TArray<FDatabasePreviewActor>>& GetPreviewActors() { return PreviewActors; }

		void ToggleDisplayRootMotionSpeed() { bDisplayRootMotionSpeed = !bDisplayRootMotionSpeed;	}
		bool IsDisplayRootMotionSpeedChecked() const { return bDisplayRootMotionSpeed; }

		void ToggleQuantizeAnimationToPoseData() { bQuantizeAnimationToPoseData = !bQuantizeAnimationToPoseData; }
		bool IsQuantizeAnimationToPoseDataChecked() const { return bQuantizeAnimationToPoseData; }

		void ToggleShowBones() { bShowBones = !bShowBones; }
		bool IsShowBonesChecked() const { return bShowBones; }

		void ToggleDisplayBlockTransition() { bDisplayBlockTransition = !bDisplayBlockTransition; }
		bool IsDisplayBlockTransitionChecked() const { return bDisplayBlockTransition; }

		void ToggleDisplayEventData() { bDisplayEventData = !bDisplayEventData; }
		bool IsDisplayEventDataChecked() const { return bDisplayEventData; }

		void AddAnimationAssetToDatabase(UObject* AnimationAsset);
		bool DeleteFromDatabase(int32 AnimationAssetIndex);

		void SetDisableReselection(int32 AnimationAssetIndex, bool bEnabled);
		bool IsDisableReselection(int32 AnimationAssetIndex) const;

		void SetIsEnabled(int32 AnimationAssetIndex, bool bEnabled);
		bool IsEnabled(int32 AnimationAssetIndex) const;

		bool SetAnimationAsset(int32 AnimationAssetIndex, UObject* AnimAsset);

		void SetMirrorOption(int32 AnimationAssetIndex, EPoseSearchMirrorOption InMirrorOption);
		EPoseSearchMirrorOption GetMirrorOption(int32 AnimationAssetIndex);
		
		int32 SetSelectedNode(int32 PoseIdx, bool bClearSelection, bool bDrawQuery, TConstArrayView<float> InQueryVector);
		void SetSelectedNodes(const TArrayView<TSharedPtr<FDatabaseAssetTreeNode>>& InSelectedNodes);
		void ProcessSelectedActor(AActor* Actor);
		
		TConstArrayView<float> GetQueryVector() const { return QueryVector; }
		void SetDrawQueryVector(bool bValue);
		bool ShouldDrawQueryVector() const { return bDrawQueryVector && !bIsEditorSelection; }

		const FSearchIndexAsset* GetSelectedActorIndexAsset() const;

		TRange<double> GetPreviewPlayRange() const;

		void SetPlayTime(float NewPlayTime, bool bInTickPlayTime);
		float GetPlayTime() const;
		bool IsEditorSelection() const { return bIsEditorSelection; }
		bool GetAnimationTime(int32 SourceAssetIdx, float& CurrentPlayTime, FVector& BlendParameters) const;

	private:
		UWorld* GetWorld();

		float PlayTime = 0.f;
		float DeltaTimeMultiplier = 1.f;
		float StepDeltaTime = 1.f / 30.f;

		/** Scene asset being viewed and edited by this view model. */
		TWeakObjectPtr<UPoseSearchDatabase> PoseSearchDatabasePtr;

		/** Weak pointer to the PreviewScene */
		TWeakPtr<FDatabasePreviewScene> PreviewScenePtr;

		/** Weak pointer to the SDatabaseDataDetails */
		TWeakPtr<SDatabaseDataDetails> DatabaseDataDetails;

		/** Actors to be displayed in the preview viewport */
		TArray<TArray<FDatabasePreviewActor>> PreviewActors;
		
		/** From zero to the play length of the longest preview */
		float MaxPreviewPlayLength = 0.f;
		float MinPreviewPlayLength = 0.f;

		bool bIsEditorSelection = true;
		bool bDrawQueryVector = false;
		TArray<float> QueryVector;

		/** Is animation debug draw enabled */
		bool bDisplayRootMotionSpeed = false;
		bool bQuantizeAnimationToPoseData = false;
		bool bShowBones = false;
		bool bDisplayBlockTransition = false;
		bool bDisplayEventData = false;

		int32 SelectedActorIndexAssetIndex = INDEX_NONE;
	};
}
