// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorViewportClient.h"


namespace UE::PoseSearch
{
	class FDatabasePreviewScene;
	class SDatabaseViewport;
	class FDatabaseEditor;

	class FDatabaseViewportClient : public FEditorViewportClient
	{
	public:

		FDatabaseViewportClient(
			const TSharedRef<FDatabasePreviewScene>& InPreviewScene,
			const TSharedRef<SDatabaseViewport>& InViewport,
			const TSharedRef<FDatabaseEditor>& InAssetEditor);
		virtual ~FDatabaseViewportClient() = default;

		// ~FEditorViewportClient interface
		virtual void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
		virtual void TrackingStarted(const struct FInputEventState& InInputState, bool bIsDragging, bool bNudge) override;
		virtual void TrackingStopped() override;
		// ~End of FEditorViewportClient interface

		/** Get the preview scene we are viewing */
		TSharedRef<FDatabasePreviewScene> GetPreviewScene() const
		{
			return PreviewScenePtr.Pin().ToSharedRef();
		}

		TSharedRef<FDatabaseEditor> GetAssetEditor() const
		{
			return AssetEditorPtr.Pin().ToSharedRef();
		}

	private:

		/** Preview scene we are viewing */
		TWeakPtr<FDatabasePreviewScene> PreviewScenePtr;

		/** Asset editor we are embedded in */
		TWeakPtr<FDatabaseEditor> AssetEditorPtr;
	};
}
