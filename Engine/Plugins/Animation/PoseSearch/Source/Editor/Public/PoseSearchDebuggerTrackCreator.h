// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IRewindDebuggerTrackCreator.h"

#define UE_API POSESEARCHEDITOR_API

namespace UE::PoseSearch
{

class FDebuggerTrackCreator : public RewindDebugger::IRewindDebuggerTrackCreator
{
private:
	virtual UE_API FName GetTargetTypeNameInternal() const override;
	virtual FName GetNameInternal() const override { return "PoseSearchDebugger"; }
	virtual UE_API void  GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const override;
	virtual UE_API TSharedPtr<RewindDebugger::FRewindDebuggerTrack> CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const override;
	virtual UE_API bool HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const override;
	virtual int32 GetSortOrderPriorityInternal() const override { return 10; };
};

} // UE::PoseSearch

#undef UE_API

