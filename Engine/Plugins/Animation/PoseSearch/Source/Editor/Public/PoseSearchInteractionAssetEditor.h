// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedPreviewScene.h"
#include "CoreMinimal.h"
#include "EditorViewportClient.h"
#include "EdMode.h"
#include "IMultiAnimAssetEditor.h"
#include "Misc/NotifyHook.h"
#include "PoseSearch/PoseSearchAssetSampler.h"
#include "PoseSearch/PoseSearchRole.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "SEditorViewport.h"

class UAnimPreviewInstance;
class UPoseSearchInteractionAsset;
class UDebugSkelMeshComponent;

namespace UE::PoseSearch
{

class FInteractionAssetPreviewScene;
class FInteractionAssetEditor;
class SInteractionAssetPreview;
class SInteractionAssetViewport;

// Experimental, this feature might be removed without warning, not for production use
struct FInteractionAssetPreviewActor
{
public:
	bool SpawnPreviewActor(UWorld* World, const UPoseSearchInteractionAsset* InteractionAsset, const UE::PoseSearch::FRole& Role);
	void UpdatePreviewActor(const UPoseSearchInteractionAsset* InteractionAsset, float PlayTime);
	void Destroy();
	UAnimPreviewInstance* GetAnimPreviewInstance();
	const UE::PoseSearch::FRole& GetRole() const { return ActorRole; }
	FTransform GetDebugActorTransformFromSampler() const;
	void ForceDebugActorTransform(const FTransform& ActorTransform);

private:
	enum { PreviewActor, DebugActor, NumActors };
	TWeakObjectPtr<AActor> ActorPtrs[NumActors];
	UE::PoseSearch::FAnimationAssetSampler Samplers[NumActors]; // used to keep track of the animation position / orientation

	float CurrentTime = 0.f;
	UE::PoseSearch::FRole ActorRole = UE::PoseSearch::DefaultRole;
	
	// @todo: add support for blend spaces
	FVector BlendParameters = FVector::ZeroVector;
};

// Experimental, this feature might be removed without warning, not for production use
class FInteractionAssetViewModel : public TSharedFromThis<FInteractionAssetViewModel>, public FGCObject
{
public:
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("FInteractionAssetViewModel"); }
	void Initialize(UPoseSearchInteractionAsset* InteractionAsset, const TSharedRef<FInteractionAssetPreviewScene>& PreviewScene);
	void PreviewBackwardEnd();
	void PreviewBackwardStep();
	void PreviewBackward();
	void PreviewPause();
	void PreviewForward();
	void PreviewForwardStep();
	void PreviewForwardEnd();
	const UPoseSearchInteractionAsset* GetInteractionAsset() const { return InteractionAssetPtr.Get(); }
	void Tick(float DeltaSeconds);
	TConstArrayView<FInteractionAssetPreviewActor> GetPreviewActors() const { return PreviewActors; }
	TRange<double> GetPreviewPlayRange() const;
	void SetPlayTime(float NewPlayTime, bool bInTickPlayTime);
	float GetPlayTime() const { return PlayTime; }
	void SetPreviewProperties(float AnimAssetTime, const FVector& AnimAssetBlendParameters, bool bAnimAssetPlaying);

private:
	UWorld* GetWorld();

	float PlayTime = 0.f;
	float DeltaTimeMultiplier = 1.f;
	float StepDeltaTime = 1.f / 30.f;

	/** asset being viewed and edited by this view model. */
	TWeakObjectPtr<UPoseSearchInteractionAsset> InteractionAssetPtr;
	/** Weak pointer to the PreviewScene */
	TWeakPtr<FInteractionAssetPreviewScene> PreviewScenePtr;
	/** Actors to be displayed in the preview viewport */
	TArray<FInteractionAssetPreviewActor> PreviewActors;
};

// Experimental, this feature might be removed without warning, not for production use
class FInteractionAssetEdMode : public FEdMode
{
public:
	const static FEditorModeID EdModeId;

	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	virtual bool AllowWidgetMove() override;
	virtual bool ShouldDrawWidget() const override;
	virtual bool GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }

private:
	FInteractionAssetViewModel* ViewModel = nullptr;
};

// Experimental, this feature might be removed without warning, not for production use
class FInteractionAssetViewportClient : public FEditorViewportClient
{
public:
	FInteractionAssetViewportClient(const TSharedRef<FInteractionAssetPreviewScene>& InPreviewScene, const TSharedRef<SInteractionAssetViewport>& InViewport, const TSharedRef<FInteractionAssetEditor>& InAssetEditor);
	virtual void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	virtual void TrackingStarted(const struct FInputEventState& InInputState, bool bIsDragging, bool bNudge) override;
	virtual void TrackingStopped() override;

	/** Get the preview scene we are viewing */
	TSharedRef<FInteractionAssetPreviewScene> GetPreviewScene() const { return PreviewScenePtr.Pin().ToSharedRef(); }
	TSharedRef<FInteractionAssetEditor> GetAssetEditor() const { return AssetEditorPtr.Pin().ToSharedRef();	}

private:
	/** Preview scene we are viewing */
	TWeakPtr<FInteractionAssetPreviewScene> PreviewScenePtr;
	/** Asset editor we are embedded in */
	TWeakPtr<FInteractionAssetEditor> AssetEditorPtr;
};

// Experimental, this feature might be removed without warning, not for production use
class FInteractionAssetPreviewScene : public FAdvancedPreviewScene
{
public:
	FInteractionAssetPreviewScene(ConstructionValues CVs, const TSharedRef<FInteractionAssetEditor>& Editor);
	virtual void Tick(float InDeltaTime) override;
	TSharedRef<FInteractionAssetEditor> GetEditor() const { return EditorPtr.Pin().ToSharedRef(); }

private:
	/** The asset editor we are embedded in */
	TWeakPtr<FInteractionAssetEditor> EditorPtr;
};

// Experimental, this feature might be removed without warning, not for production use
struct FInteractionAssetPreviewRequiredArgs
{
	FInteractionAssetPreviewRequiredArgs(const TSharedRef<FInteractionAssetEditor>& InAssetEditor, const TSharedRef<FInteractionAssetPreviewScene>& InPreviewScene)
		: AssetEditor(InAssetEditor)
		, PreviewScene(InPreviewScene)
	{
	}

	TSharedRef<FInteractionAssetEditor> AssetEditor;
	TSharedRef<FInteractionAssetPreviewScene> PreviewScene;
};

// Experimental, this feature might be removed without warning, not for production use
class SInteractionAssetViewport : public SEditorViewport, public ICommonEditorViewportToolbarInfoProvider
{
public:
	SLATE_BEGIN_ARGS(SInteractionAssetViewport) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, const FInteractionAssetPreviewRequiredArgs& InRequiredArgs);
	virtual TSharedRef<class SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;

protected:
	virtual void BindCommands() override;
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;

	/** The viewport toolbar */
	TSharedPtr<SCommonEditorViewportToolbarBase> ViewportToolbar;
	/** Viewport client */
	TSharedPtr<FInteractionAssetViewportClient> ViewportClient;
	/** The preview scene that we are viewing */
	TWeakPtr<FInteractionAssetPreviewScene> PreviewScenePtr;
	/** Asset editor we are embedded in */
	TWeakPtr<FInteractionAssetEditor> AssetEditorPtr;
};

// Experimental, this feature might be removed without warning, not for production use
class SInteractionAssetPreview : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_TwoParams(FOnScrubPositionChanged, double, bool)
	DECLARE_DELEGATE(FOnButtonClickedEvent)

	SLATE_BEGIN_ARGS(SInteractionAssetPreview) {}
		SLATE_ATTRIBUTE(FLinearColor, SliderColor);
		SLATE_ATTRIBUTE(double, SliderScrubTime);
		SLATE_ATTRIBUTE(TRange<double>, SliderViewRange);
		SLATE_EVENT(FOnScrubPositionChanged, OnSliderScrubPositionChanged);
		SLATE_EVENT(FOnButtonClickedEvent, OnBackwardEnd);
		SLATE_EVENT(FOnButtonClickedEvent, OnBackwardStep);
		SLATE_EVENT(FOnButtonClickedEvent, OnBackward);
		SLATE_EVENT(FOnButtonClickedEvent, OnPause);
		SLATE_EVENT(FOnButtonClickedEvent, OnForward);
		SLATE_EVENT(FOnButtonClickedEvent, OnForwardStep);
		SLATE_EVENT(FOnButtonClickedEvent, OnForwardEnd);
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, const FInteractionAssetPreviewRequiredArgs& InRequiredArgs);

protected:
	TAttribute<FLinearColor> SliderColor;
	TAttribute<double> SliderScrubTime;
	TAttribute<TRange<double>> SliderViewRange = TRange<double>(0.0, 1.0);
	FOnScrubPositionChanged OnSliderScrubPositionChanged;
	FOnButtonClickedEvent OnBackwardEnd;
	FOnButtonClickedEvent OnBackwardStep;
	FOnButtonClickedEvent OnBackward;
	FOnButtonClickedEvent OnPause;
	FOnButtonClickedEvent OnForward;
	FOnButtonClickedEvent OnForwardStep;
	FOnButtonClickedEvent OnForwardEnd;
};

// Experimental, this feature might be removed without warning, not for production use
class FInteractionAssetEditor : public IMultiAnimAssetEditor, public FNotifyHook
{
public:
	void InitAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UPoseSearchInteractionAsset* InteractionAsset);

	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual void SetPreviewProperties(float AnimAssetTime, const FVector& AnimAssetBlendParameters, bool bAnimAssetPlaying) override;

	const UPoseSearchInteractionAsset* GetInteractionAsset() const { return ViewModel.IsValid() ? ViewModel->GetInteractionAsset() : nullptr; }
	FInteractionAssetViewModel* GetViewModel() const { return ViewModel.Get(); }
	void PreviewBackwardEnd() { ViewModel->PreviewBackwardEnd(); }
	void PreviewBackwardStep() { ViewModel->PreviewBackwardStep();	}
	void PreviewBackward() { ViewModel->PreviewBackward(); }
	void PreviewPause() { ViewModel->PreviewPause(); }
	void PreviewForward() { ViewModel->PreviewForward(); }
	void PreviewForwardStep() { ViewModel->PreviewForwardStep(); }
	void PreviewForwardEnd() { ViewModel->PreviewForwardEnd(); }

private:
	TSharedRef<SDockTab> SpawnTab_Viewport(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_AssetDetails(const FSpawnTabArgs& Args);

	TSharedPtr<SInteractionAssetPreview> PreviewWidget;
	TSharedPtr<IDetailsView> EditingAssetWidget;
	TSharedPtr<FInteractionAssetPreviewScene> PreviewScene;
	TSharedPtr<FInteractionAssetViewModel> ViewModel;
};

} // UE::PoseSearch

