// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "PoseSearchNormalizationSetFactory.generated.h"

#define UE_API POSESEARCHEDITOR_API

UCLASS(MinimalAPI)
class UPoseSearchNormalizationSetFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// UFactory interface
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	UE_API virtual FString GetDefaultNewAssetName() const override;
	// End of UFactory interface
};

#undef UE_API
