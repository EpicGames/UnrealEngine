// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "PoseSearchSchemaFactory.generated.h"

#define UE_API POSESEARCHEDITOR_API

struct FAssetData;
class SWindow;

UCLASS(MinimalAPI)
class UPoseSearchSchemaFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<class USkeleton> TargetSkeleton;

	TSharedPtr<SWindow> PickerWindow;

	// UFactory interface
	UE_API virtual bool ConfigureProperties() override;
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	UE_API virtual FString GetDefaultNewAssetName() const override;
	// End of UFactory interface

	UE_API void OnTargetSkeletonSelected(const FAssetData& SelectedAsset);
};

#undef UE_API
