// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SEditorViewport.h"
#include "SCommonEditorViewportToolbarBase.h"


namespace UE::PoseSearch
{
	class FDatabasePreviewScene;
	class FDatabaseViewportClient;
	class FDatabaseEditor;
	class SPoseSearchDatabaseViewportToolBar;

	struct FDatabasePreviewRequiredArgs
	{
		FDatabasePreviewRequiredArgs(
			const TSharedRef<FDatabaseEditor>& InAssetEditor,
			const TSharedRef<FDatabasePreviewScene>& InPreviewScene)
			: AssetEditor(InAssetEditor)
			, PreviewScene(InPreviewScene)
		{
		}

		TSharedRef<FDatabaseEditor> AssetEditor;

		TSharedRef<FDatabasePreviewScene> PreviewScene;
	};

	class SDatabaseViewport : public SEditorViewport, public ICommonEditorViewportToolbarInfoProvider
	{
	public:
		SLATE_BEGIN_ARGS(SDatabaseViewport) {}
		SLATE_END_ARGS();

		void Construct(const FArguments& InArgs, const FDatabasePreviewRequiredArgs& InRequiredArgs);
		virtual ~SDatabaseViewport() = default;

		// ~ICommonEditorViewportToolbarInfoProvider interface
		virtual TSharedRef<class SEditorViewport> GetViewportWidget() override;
		virtual TSharedPtr<FExtender> GetExtenders() const override;
		virtual void OnFloatingButtonClicked() override;
		// ~End of ICommonEditorViewportToolbarInfoProvider interface

	protected:

		// ~SEditorViewport interface
		virtual void BindCommands() override;
		virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
		virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
		// ~End of SEditorViewport interface

		void OnToggleDisplayRootMotionSpeed();
		bool IsDisplayRootMotionSpeedChecked() const;
		void OnToggleQuantizeAnimationToPoseData();
		bool IsQuantizeAnimationToPoseDataChecked() const;
		void OnToggleShowBones();
		bool IsShowBonesChecked() const;
		void OnToggleDisplayBlockTransition();
		bool IsDisplayBlockTransitionChecked() const;
		// Experimental, this feature might be removed without warning, not for production use
		void OnToggleDisplayEventData();
		// Experimental, this feature might be removed without warning, not for production use
		bool IsDisplayEventDataChecked() const;

		/** The viewport toolbar */
		TSharedPtr<SPoseSearchDatabaseViewportToolBar> ViewportToolbar;

		/** Viewport client */
		TSharedPtr<FDatabaseViewportClient> ViewportClient;

		/** The preview scene that we are viewing */
		TWeakPtr<FDatabasePreviewScene> PreviewScenePtr;

		/** Asset editor we are embedded in */
		TWeakPtr<FDatabaseEditor> AssetEditorPtr;
	};

	class SDatabasePreview : public SCompoundWidget
	{
	public:
		DECLARE_DELEGATE_TwoParams(FOnScrubPositionChanged, double, bool)
		DECLARE_DELEGATE(FOnButtonClickedEvent)

		SLATE_BEGIN_ARGS(SDatabasePreview) {}
			SLATE_ATTRIBUTE(FLinearColor, SliderColor);
			SLATE_ATTRIBUTE(double, SliderScrubTime);
			SLATE_ATTRIBUTE(TRange<double>, SliderViewRange);
			SLATE_EVENT(FOnScrubPositionChanged, OnSliderScrubPositionChanged);
			SLATE_EVENT(FOnButtonClickedEvent, OnBackwardEnd);
			SLATE_EVENT(FOnButtonClickedEvent, OnBackwardStep);
			SLATE_EVENT(FOnButtonClickedEvent, OnBackward);
			SLATE_EVENT(FOnButtonClickedEvent, OnPause);
			SLATE_EVENT(FOnButtonClickedEvent, OnForward);
			SLATE_EVENT(FOnButtonClickedEvent, OnForwardStep);
			SLATE_EVENT(FOnButtonClickedEvent, OnForwardEnd);
		SLATE_END_ARGS();

		void Construct(const FArguments& InArgs, const FDatabasePreviewRequiredArgs& InRequiredArgs);

	protected:
		TAttribute<FLinearColor> SliderColor;
		TAttribute<double> SliderScrubTime;
		TAttribute<TRange<double>> SliderViewRange = TRange<double>(0.0, 1.0);
		FOnScrubPositionChanged OnSliderScrubPositionChanged;
		FOnButtonClickedEvent OnBackwardEnd;
		FOnButtonClickedEvent OnBackwardStep;
		FOnButtonClickedEvent OnBackward;
		FOnButtonClickedEvent OnPause;
		FOnButtonClickedEvent OnForward;
		FOnButtonClickedEvent OnForwardStep;
		FOnButtonClickedEvent OnForwardEnd;
	};
}
