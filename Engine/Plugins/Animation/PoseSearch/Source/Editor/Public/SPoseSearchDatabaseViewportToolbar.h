// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SCommonEditorViewportToolbarBase.h"


namespace UE::PoseSearch
{
	class SDatabaseViewport;

	class SPoseSearchDatabaseViewportToolBar : public SCommonEditorViewportToolbarBase
	{
	public:
		SLATE_BEGIN_ARGS(SPoseSearchDatabaseViewportToolBar)
		{}

		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedPtr<SDatabaseViewport> InViewport);

		// ~SCommonEditorViewportToolbarBase interface
		virtual TSharedRef<SWidget> GenerateShowMenu() const override;
		// ~End of SCommonEditorViewportToolbarBase interface
	};
}
