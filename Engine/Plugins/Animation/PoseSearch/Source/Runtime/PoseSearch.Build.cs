// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PoseSearch : ModuleRules
{
	public PoseSearch(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		AddEngineThirdPartyPrivateStaticDependencies(Target,
			"Eigen",
			"nanoflann"
		);

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AnimationCore",
				"AnimGraphRuntime",
				"BlendStack",
				"Chooser",
				"Core",
				"CoreUObject",
				"Engine",
				"TraceLog"
			}
		);

		PrivateDependencyModuleNames.AddRange(
		new string[]{
			"AnimationWarpingRuntime",
			"DeveloperSettings",
			"GameplayTags",
			"MotionWarping",
			"RewindDebuggerRuntimeInterface"
		}
		);

		if (Target.bCompileAgainstEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AnimationModifiers",
					"AnimationBlueprintLibrary",
					"DerivedDataCache",
					"RigVM",
					"RigVMDeveloper",
					"UnrealEd"
				}
			);
		}
	}
}
