// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/AnimNode_MotionMatching.h"
#include "Animation/AnimInertializationSyncScope.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNode_Inertialization.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimStats.h"
#include "Animation/BlendSpace.h"
#include "BoneControllers/AnimNode_OffsetRootBone.h"
#include "Components/SkeletalMeshComponent.h"
#include "PoseSearch/AnimNode_PoseSearchHistoryCollector.h"
#include "PoseSearch/MultiAnimAsset.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchDerivedData.h"
#include "PoseSearch/PoseSearchInteractionLibrary.h"
#include "PoseSearch/PoseSearchInteractionUtils.h"
#include "PoseSearch/PoseSearchInteractionValidator.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearch/Trace/PoseSearchTraceLogger.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_MotionMatching)

#define LOCTEXT_NAMESPACE "AnimNode_MotionMatching"

#if ENABLE_ANIM_DEBUG
namespace UE::Private
{
	enum EPlayRateState : int8
	{
		Disabled = 0,
		Enabled = 1,
		PoseSearchOnly = 2
	};
} // namespace UE::Private

static bool GVarAnimNodeMotionMatchingDrawQuery = false;
static FAutoConsoleVariableRef CVarAnimNodeMotionMatchingDrawQuery(TEXT("a.AnimNode.MotionMatching.DebugDrawQuery"), GVarAnimNodeMotionMatchingDrawQuery, TEXT("Draw input query"));

static bool GVarAnimNodeMotionMatchingDrawCurResult = false;
static FAutoConsoleVariableRef CVarAnimNodeMotionMatchingDrawCurResult(TEXT("a.AnimNode.MotionMatching.DebugDrawCurResult"), GVarAnimNodeMotionMatchingDrawCurResult, TEXT("Draw current result"));

static bool GVarAnimNodeMotionMatchingDrawInfo = false;
static FAutoConsoleVariableRef CVarAnimNodeMotionMatchingDrawInfo(TEXT("a.AnimNode.MotionMatching.DebugDrawInfo"), GVarAnimNodeMotionMatchingDrawInfo, TEXT("Draw info like current databases and asset"));

static bool GVarAnimNodeMotionMatchingDrawInfoVerbose = true;
static FAutoConsoleVariableRef CVarAnimNodeMotionMatchingDrawInfoVerbose(TEXT("a.AnimNode.MotionMatching.DebugDrawInfoVerbose"), GVarAnimNodeMotionMatchingDrawInfoVerbose, TEXT("Draw additional info like blend stack"));

static float GVarAnimNodeMotionMatchingDrawInfoHeight = 50.f;
static FAutoConsoleVariableRef CVarAnimNodeMotionMatchingDrawInfoHeight(TEXT("a.AnimNode.MotionMatching.DebugDrawInfoHeight"), GVarAnimNodeMotionMatchingDrawInfoHeight, TEXT("Vertical offset for DebugDrawInfo"));

static int32 GVarAnimNodeMotionMatchingPlayRateEnabled = UE::Private::EPlayRateState::Enabled;
static FAutoConsoleVariableRef CVarAnimNodeMotionMatchingPlayRateEnabled(
	TEXT("a.AnimNode.MotionMatching.DebugPlayRateEnabled"), 
	GVarAnimNodeMotionMatchingPlayRateEnabled,
	TEXT("Toggles if PlayRate is used in motion matching. Same as setting PlayRate to (1,1) when disabled.\n")
	TEXT("0: Completely disable PlayRate usage.\n")
	TEXT("1: Enable all usages of PlayRate.\n")
	TEXT("2: Enable PlayRate in PoseSeach only (Not used in actual playback).\n")
);
#endif

/////////////////////////////////////////////////////
// FAnimNode_MotionMatching

void FAnimNode_MotionMatching::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread);

	GetEvaluateGraphExposedInputs().Execute(Context);

	FAnimNode_BlendStack_Standalone::Initialize_AnyThread(Context);
	MotionMatchingState.Reset();
}

void FAnimNode_MotionMatching::UpdateAssetPlayer(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(UpdateAssetPlayer);

	QUICK_SCOPE_CYCLE_COUNTER(STAT_PoseSearch_UpdateAssetPlayer);

	using namespace UE::PoseSearch;

	check(Context.AnimInstanceProxy);

	GetEvaluateGraphExposedInputs().Execute(Context);

	// synchronizing with GetAccumulatedTime or resetting MotionMatchingState, and conditionally resetting FAnimNode_BlendStack_Standalone
	if (bResetOnBecomingRelevant && UpdateCounter.HasEverBeenUpdated() && !UpdateCounter.WasSynchronizedCounter(Context.AnimInstanceProxy->GetUpdateCounter()))
	{
		// If we just became relevant and haven't been initialized yet, then reset motion matching state, otherwise update the asset time using the player node.
		MotionMatchingState.Reset();
		FAnimNode_BlendStack_Standalone::Reset();
	}
	else if (MotionMatchingState.SearchResult.SelectedDatabase == nullptr || MotionMatchingState.SearchResult.SelectedDatabase->Schema == nullptr)
	{
	}
#if WITH_EDITOR
	else if (EAsyncBuildIndexResult::Success != FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(MotionMatchingState.SearchResult.SelectedDatabase.Get(), ERequestAsyncBuildFlag::ContinueRequest))
	{
		// MotionMatchingState.SearchResult.Database is indexing, and it's not safe to use its previous index properties cached in MotionMatchingState
		MotionMatchingState.Reset();
	}
#endif // WITH_EDITOR

	UpdateCounter.SynchronizeWith(Context.AnimInstanceProxy->GetUpdateCounter());

	// If the Database property hasn't been overridden, set it as the only database to search.
	if (!bOverrideDatabaseInput)
	{
		DatabasesToSearch.Reset();
		if (Database != nullptr)
		{
			DatabasesToSearch.Add(Database);
		}
	}

#if ENABLE_ANIM_DEBUG
	if (GVarAnimNodeMotionMatchingDrawInfo)
	{
		FString DebugInfo = FString::Printf(TEXT("NextUpdateInterruptMode(%s)\n"), *UEnum::GetValueAsString(NextUpdateInterruptMode));
		DebugInfo += FString::Printf(TEXT("Current Database(%s)\n"), *GetNameSafe(MotionMatchingState.SearchResult.SelectedDatabase.Get()));
		DebugInfo += FString::Printf(TEXT("Current Asset(%s)\n"), *GetNameSafe(GetAnimAsset()));
		if (GVarAnimNodeMotionMatchingDrawInfoVerbose)
		{
			DebugInfo += FString::Printf(TEXT("Databases to search:\n"));
			for (const UPoseSearchDatabase* DatabaseToSearch : DatabasesToSearch)
			{
				DebugInfo += FString::Printf(TEXT("  %s\n"), *GetNameSafe(DatabaseToSearch));
			}
			DebugInfo += FString::Printf(TEXT("Blend Stack:\n"));
			for (const FBlendStackAnimPlayer& AnimPlayer : AnimPlayers)
			{
				DebugInfo += FString::Printf(TEXT("  %s [time:%.2f|playrate:%.2f]\n"), *GetNameSafe(AnimPlayer.GetAnimationAsset()), AnimPlayer.GetAccumulatedTime(), AnimPlayer.GetPlayRate());
			}
		}
		Context.AnimInstanceProxy->AnimDrawDebugInWorldMessage(DebugInfo, FVector::UpVector * GVarAnimNodeMotionMatchingDrawInfoHeight, FColor::Yellow, 1.f /*TextScale*/);
	}
#endif // ENABLE_ANIM_DEBUG

	const IPoseHistory* PoseHistory = nullptr;
	if (const FPoseHistoryProvider* PoseHistoryProvider = Context.GetMessage<FPoseHistoryProvider>())
	{
		PoseHistory = PoseHistoryProvider->GetPoseHistoryPtr();
	}

	if (!PoseHistory)
	{
		UE_LOGF(LogPoseSearch, Error, "FAnimNode_MotionMatching::UpdateAssetPlayer, missing IPoseHistory");
	}
	else
	{
		const UE::AnimationWarping::FRootOffsetProvider* RootOffsetProvider = Context.GetMessage<UE::AnimationWarping::FRootOffsetProvider>();
		MeshWithoutOffset = Context.AnimInstanceProxy->GetComponentTransform();
		MeshWithOffset = RootOffsetProvider ? RootOffsetProvider->GetRootTransform() : MeshWithoutOffset;

		const UObject* AnimContext = Context.AnimInstanceProxy->GetAnimInstanceObject();
		check(AnimContext);
		// MotionMatchInteraction synchronizes and integrates MotionMatchingState.SearchResult internally, so there's no need to update it with the blendstack information
		UPoseSearchInteractionLibrary::MotionMatchInteraction(MotionMatchingState.SearchResult, Availabilities,
			AnimContext, FName(), PoseHistory, bValidateResultAgainstAvailabilities, bKeepInteractionAlive, BlendTime, Context.GetDeltaTime());

		// performing the regular single character motion matching search in case there's no MM interaction
		if (!MotionMatchingState.SearchResult.bIsInteraction)
		{
			// We adjust the motion matching state asset time to the current player node's asset time. This is done 
			// because the player node may have ticked more or less time than we expected due to variable dt or the 
			// dynamic playback rate adjustment and as such the motion matching state does not update by itself
			MotionMatchingState.SearchResult.SelectedAnim = GetAnimAsset();
			MotionMatchingState.SearchResult.SelectedTime = GetAccumulatedTime();
			MotionMatchingState.SearchResult.bIsMirrored = GetMirror();
			MotionMatchingState.SearchResult.BlendParameters = GetBlendParameters();

			FFloatInterval PoseSearchPlayRate = PlayRate;
#if ENABLE_ANIM_DEBUG
			bool bPoseSearchPlayRateEnabled = GVarAnimNodeMotionMatchingPlayRateEnabled == UE::Private::EPlayRateState::Enabled || GVarAnimNodeMotionMatchingPlayRateEnabled == UE::Private::EPlayRateState::PoseSearchOnly;
			PoseSearchPlayRate = bPoseSearchPlayRateEnabled ? PoseSearchPlayRate : FFloatInterval(1.f, 1.f);
#endif // ENABLE_ANIM_DEBUG

			const float DeltaTime = Context.GetDeltaTime();

			FSearchContext SearchContext(0.f, PoseJumpThresholdTime, EventToSearch.GetPlayRateOverriddenEvent(PoseSearchPlayRate));
			FChooserEvaluationContext ChooserEvaluationContext(const_cast<UObject*>(AnimContext));
			SearchContext.AddRole(GetCommonDefaultRole(DatabasesToSearch), &ChooserEvaluationContext, PoseHistory);

			// If we can't advance or enough time has elapsed since the last pose jump then search
			bool bSearch = true;
			FSearchResult SearchResult;
			if (MotionMatchingState.ElapsedPoseSearchTime < SearchThrottleTime)
			{
				SearchResult.InitFrom(MotionMatchingState.SearchResult);
				SearchContext.UpdateContinuingPoseSearchResult(SearchResult, SearchResult);
				const bool bCanAdvance = SearchContext.GetContinuingPoseSearchResult().PoseIdx != INDEX_NONE;
				bSearch = !bCanAdvance;
			}

			if (bSearch)
			{
				MotionMatchingState.ElapsedPoseSearchTime = 0.f;

				TArrayView<const UObject*> AssetsToSearch((const UObject**)FMemory_Alloca(DatabasesToSearch.Num() * sizeof(UObject*)), (int32)DatabasesToSearch.Num());
				for (int32 DatabaseToSearchIndex = 0; DatabaseToSearchIndex < DatabasesToSearch.Num(); ++DatabaseToSearchIndex)
				{
					AssetsToSearch[DatabaseToSearchIndex] = DatabasesToSearch[DatabaseToSearchIndex].Get();
				}

				FSearchResults_Single SearchResults;
				FPoseSearchContinuingProperties ContinuingProperties;
				ContinuingProperties.InitFrom(MotionMatchingState.SearchResult, NextUpdateInterruptMode);
				UPoseSearchLibrary::MotionMatch(SearchContext, AssetsToSearch, ContinuingProperties, SearchResults);
				SearchResult = SearchResults.GetBestResult();
			}
			else
			{
				MotionMatchingState.ElapsedPoseSearchTime += DeltaTime;

				// if we didn't perform a search we carry on with the continuing pose search result
				SearchResult.bIsContinuingPoseSearch = true;

#if UE_POSE_SEARCH_TRACE_ENABLED
				// in case we skipped the search, we still have to track we would have requested to evaluate DatabasesToSearch and SearchResult.Database
				for (const TObjectPtr<const UPoseSearchDatabase>& DatabaseToSearch : DatabasesToSearch)
				{
					SearchContext.Track(DatabaseToSearch.Get());
				}

				SearchContext.Track(SearchResult.Database.Get());
#endif // UE_POSE_SEARCH_TRACE_ENABLED
			}

			const float WantedPlayRate = CalculateWantedPlayRate(SearchResult, SearchContext, PoseSearchPlayRate, PoseHistory ? PoseHistory->GetTrajectorySpeedMultiplier() : 1.f, EventToSearch);
			MotionMatchingState.SearchResult.InitFrom(SearchResult, WantedPlayRate);

			if (const FPoseIndicesHistory* PoseIndicesHistory = PoseHistory->GetPoseIndicesHistory())
			{
				// const casting here is safe since we're in the thread owning the pose history, and it's the correct place to update the previously selected poses
				const_cast<FPoseIndicesHistory*>(PoseIndicesHistory)->Update(SearchResult, DeltaTime, PoseReselectHistory);
			}
		}

		float DesiredPlayRate = MotionMatchingState.SearchResult.WantedPlayRate * PlayRateMultiplier;
#if ENABLE_ANIM_DEBUG
		DesiredPlayRate = GVarAnimNodeMotionMatchingPlayRateEnabled == UE::Private::EPlayRateState::Enabled ? DesiredPlayRate : PlayRateMultiplier;
#endif // ENABLE_ANIM_DEBUG

		UE::Anim::FNodeFunctionCaller::CallFunction(GetOnUpdateMotionMatchingStateFunction(), Context, *this);

		// If a new pose is requested, blend into the new asset via BlendStackNode
		const bool bJumpToPose = MotionMatchingState.SearchResult.SelectedAnim && !MotionMatchingState.SearchResult.bIsContinuingPoseSearch;
		if (bJumpToPose)
		{
			const UPoseSearchDatabase* CurrentResultDatabase = MotionMatchingState.SearchResult.SelectedDatabase.Get();
			if (CurrentResultDatabase && CurrentResultDatabase->Schema)
			{
				if (UAnimationAsset* AnimationAsset = MotionMatchingState.SearchResult.GetAnimationAssetForRole())
				{
					bool bAbortBlend = false;
					if (!AnimPlayers.IsEmpty() && GetBlendspaceParametersDeltaThreshold() > 0.f && AnimationAsset->IsA<UBlendSpace>())
					{
						if (AnimPlayers[0].GetAnimationAsset() == AnimationAsset)
						{
							const FVector CurrentBlendParameters = AnimPlayers[0].GetBlendParameters();
							const FVector DesiredBlendParameters = GetBlendspaceParameters();
							const float Delta = (CurrentBlendParameters - DesiredBlendParameters).SizeSquared();
							if (Delta < FMath::Square(GetBlendspaceParametersDeltaThreshold()))
							{
								// If we haven't changed assets, and our currently playing blendspace xy is within threshold of change, then keep playing it.
								// Time differences should be OK because of the call to AdjustAssetTime before the search.
								bAbortBlend = true;
							}
						}
					}

					if (!bAbortBlend)
					{
						FAnimNode_BlendStack_Standalone::BlendTo(Context, AnimationAsset, MotionMatchingState.SearchResult.SelectedTime, MotionMatchingState.SearchResult.bLoop,
							MotionMatchingState.SearchResult.bIsMirrored, CurrentResultDatabase->Schema->GetMirrorDataTable(MotionMatchingState.SearchResult.Role), BlendTime,
							BlendProfile, BlendOption, bUseInertialBlend, NAME_None, MotionMatchingState.SearchResult.BlendParameters, DesiredPlayRate, 0,
							GetGroupName(), GetGroupRole(), GetGroupMethod(), GetOverridePositionWhenJoiningSyncGroupAsLeader());
					}
				}
				else
				{
					checkNoEntry();
				}
			}
		}

		const bool bDidBlendToRequestAnInertialBlend = bJumpToPose && bUseInertialBlend;
		UE::Anim::TOptionalScopedGraphMessage<UE::Anim::FAnimInertializationSyncScope> InertializationSync(bDidBlendToRequestAnInertialBlend, Context);

		FAnimNode_BlendStack_Standalone::UpdatePlayRate(DesiredPlayRate);
		FAnimNode_BlendStack_Standalone::UpdateBlendspaceParameters(GetBlendspaceUpdateMode(), GetBlendspaceParameters());
	}

	FAnimNode_BlendStack_Standalone::UpdateAssetPlayer(Context);

	NextUpdateInterruptMode = EPoseSearchInterruptMode::DoNotInterrupt;
}

void FAnimNode_MotionMatching::Evaluate_AnyThread(FPoseContext& Output)
{
	using namespace UE::PoseSearch;

	check(Output.AnimInstanceProxy);
	CheckInteractionThreadSafety(Output.AnimInstanceProxy->GetAnimInstanceObject());

	Super::Evaluate_AnyThread(Output);

	if (!bEnableWarp)
	{
		return;
	}

	// calculating the animation time we want the full alignment transforms from
	UAnimInstance* AnimInstance = Cast<UAnimInstance>(Output.AnimInstanceProxy->GetAnimInstanceObject());
	if (!AnimInstance)
	{
		UE_LOGF(LogPoseSearch, Error, "FAnimNode_MotionMatching::Evaluate_AnyThread couldn't find the AnimInstance!?");
		return;
	}

	const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get();

	if (!RootMotionProvider)
	{
		UE_LOGF(LogPoseSearch, Error, "FAnimNode_MotionMatching::Evaluate_AnyThread couldn't find the IAnimRootMotionProvider");
		return;
	}

	const UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(MotionMatchingState.SearchResult.SelectedAnim);
	if (!MultiAnimAsset)
	{
		return;
	}

	const int32 NumRoles = MultiAnimAsset->GetNumRoles();
	if (MotionMatchingState.SearchResult.ActorRootTransforms.Num() != NumRoles)
	{
		// warping is supported only for UMultiAnimAsset(s)
		return;
	}

	const int32 CurrentResultRoleIndex = GetRoleIndex(MultiAnimAsset, MotionMatchingState.SearchResult.Role);
	if (CurrentResultRoleIndex == INDEX_NONE)
	{
		return;
	}

	const UAnimationAsset* AnimationAsset = MultiAnimAsset->GetAnimationAsset(MotionMatchingState.SearchResult.Role);
	if (!AnimationAsset)
	{
		UE_LOGF(LogPoseSearch, Error, "FAnimNode_MotionMatching::Evaluate_AnyThread MultiAnimAsset %ls for Role %ls is invalid!", *GetNameSafe(MultiAnimAsset), *MotionMatchingState.SearchResult.Role.ToString());
		return;
	}

	TArray<FTransform, TInlineAllocator<PreallocatedRolesNum>> FullAlignedTransforms;
	FullAlignedTransforms.SetNum(NumRoles);
	CalculateFullAlignedTransforms(MotionMatchingState.SearchResult, MotionMatchingState.SearchResult.SelectedTime, 0.f, bWarpUsingRootBone, FullAlignedTransforms);

	// NoTe: keep in mind DeltaAlignment is relative to the previous execution frame so we still need to extract and and apply the current animation root motion transform to get to the current frame full aligned transform.
	const FTransform DeltaAlignment = CalculateDeltaAlignment(MeshWithoutOffset, MeshWithOffset, FullAlignedTransforms[CurrentResultRoleIndex], WarpingRotationRatio, WarpingTranslationRatio);
	
	FTransform RootMotionDelta;
	RootMotionProvider->ExtractRootMotion(Output.CustomAttributes, RootMotionDelta);

	const FTransform DeltaAlignmentWithRootMotion = DeltaAlignment * RootMotionDelta;
	RootMotionProvider->OverrideRootMotion(DeltaAlignmentWithRootMotion, Output.CustomAttributes);

#if ENABLE_VISUAL_LOG
	if (FVisualLogger::IsRecording())
	{
		static const TCHAR* LogName = TEXT("MotionMatching");

		for (int32 Index = 0; Index < NumRoles; ++Index)
		{
			const FTransform& ActorRootTransform = MotionMatchingState.SearchResult.ActorRootTransforms[Index];
			const FTransform& FullAlignedTransform = FullAlignedTransforms[Index];

			UE_VLOG_SEGMENT_THICK(AnimInstance, LogName, Display, FullAlignedTransform.GetLocation(), ActorRootTransform.GetLocation(), FColorList::Orange, 1.f, TEXT(""));
			UE_VLOG_SEGMENT_THICK(AnimInstance, LogName, Display, ActorRootTransform.GetLocation(), ActorRootTransform.GetLocation() + ActorRootTransform.GetRotation().GetForwardVector() * 35, FColorList::LightGrey, 3.f, TEXT(""));
			UE_VLOG_SEGMENT_THICK(AnimInstance, LogName, Display, FullAlignedTransform.GetLocation(), FullAlignedTransform.GetLocation() + FullAlignedTransform.GetRotation().GetForwardVector() * 30, FColorList::Orange, 2.f, TEXT(""));
		}

		UE_VLOG_SEGMENT_THICK(AnimInstance, LogName, Display, MeshWithOffset.GetLocation(), MeshWithOffset.GetLocation() + MeshWithOffset.GetRotation().GetForwardVector() * 35, FColorList::Blue, 3.f, TEXT(""));
		UE_VLOG_SEGMENT_THICK(AnimInstance, LogName, Display, MeshWithoutOffset.GetLocation(), MeshWithoutOffset.GetLocation() + MeshWithoutOffset.GetRotation().GetForwardVector() * 40, FColorList::Cyan, 4.f, TEXT(""));
	}
#endif
}

const FAnimNodeFunctionRef& FAnimNode_MotionMatching::GetOnUpdateMotionMatchingStateFunction() const
{
	return GET_ANIM_NODE_DATA(FAnimNodeFunctionRef, OnMotionMatchingStateUpdated);
}

void FAnimNode_MotionMatching::SetDatabaseToSearch(UPoseSearchDatabase* InDatabase, EPoseSearchInterruptMode InterruptMode)
{
	SetDatabasesToSearch(MakeArrayView(&InDatabase, 1), InterruptMode);
}

FVector FAnimNode_MotionMatching::GetEstimatedFutureRootMotionVelocity() const
{
	return MotionMatchingState.GetEstimatedFutureRootMotionVelocity();
}

void FAnimNode_MotionMatching::SetDatabasesToSearch(TConstArrayView<UPoseSearchDatabase*> InDatabases, EPoseSearchInterruptMode InterruptMode)
{
	DatabasesToSearch.Reset();
	for (UPoseSearchDatabase* InDatabase : InDatabases)
	{
		DatabasesToSearch.AddUnique(InDatabase);
	}
	NextUpdateInterruptMode = InterruptMode;
	bOverrideDatabaseInput = true;
}

void FAnimNode_MotionMatching::ResetDatabasesToSearch(EPoseSearchInterruptMode InterruptMode)
{
	DatabasesToSearch.Reset();
	bOverrideDatabaseInput = false;
	NextUpdateInterruptMode = InterruptMode;
}

void FAnimNode_MotionMatching::SetInterruptMode(EPoseSearchInterruptMode InterruptMode)
{
	NextUpdateInterruptMode = InterruptMode;
}

// FAnimNode_AssetPlayerBase interface
bool FAnimNode_MotionMatching::GetIgnoreForRelevancyTest() const
{
	return GET_ANIM_NODE_DATA(bool, bIgnoreForRelevancyTest);
}

bool FAnimNode_MotionMatching::SetIgnoreForRelevancyTest(bool bInIgnoreForRelevancyTest)
{
#if WITH_EDITORONLY_DATA
	bIgnoreForRelevancyTest = bInIgnoreForRelevancyTest;
#endif

	if(bool* bIgnoreForRelevancyTestPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bIgnoreForRelevancyTest))
	{
		*bIgnoreForRelevancyTestPtr = bInIgnoreForRelevancyTest;
		return true;
	}

	return false;
}

const FVector& FAnimNode_MotionMatching::GetBlendspaceParameters() const
{
	return GET_ANIM_NODE_DATA(FVector, BlendParameters);
}

float FAnimNode_MotionMatching::GetBlendspaceParametersDeltaThreshold() const
{
	return GET_ANIM_NODE_DATA(float, BlendParametersDeltaThreshold);
}

EBlendStack_BlendspaceUpdateMode FAnimNode_MotionMatching::GetBlendspaceUpdateMode() const
{
	return GET_ANIM_NODE_DATA(EBlendStack_BlendspaceUpdateMode, BlendspaceUpdateMode);
}

FName FAnimNode_MotionMatching::GetGroupName() const
{
	return GET_ANIM_NODE_DATA(FName, GroupName);
}

EAnimGroupRole::Type FAnimNode_MotionMatching::GetGroupRole() const
{
	return GET_ANIM_NODE_DATA(TEnumAsByte<EAnimGroupRole::Type>, GroupRole);
}

EAnimSyncMethod FAnimNode_MotionMatching::GetGroupMethod() const
{
	return GET_ANIM_NODE_DATA(EAnimSyncMethod, Method);
}

bool FAnimNode_MotionMatching::GetOverridePositionWhenJoiningSyncGroupAsLeader() const
{
	return GET_ANIM_NODE_DATA(bool, bOverridePositionWhenJoiningSyncGroupAsLeader);
}

bool FAnimNode_MotionMatching::IsLooping() const
{
	if (!AnimPlayers.IsEmpty())
	{
		return AnimPlayers[0].IsLooping();
	}
	return false;
}

bool FAnimNode_MotionMatching::SetGroupName(FName InGroupName)
{
#if WITH_EDITORONLY_DATA
	GroupName = InGroupName;
#endif
	if(FName* GroupNamePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(FName, GroupName))
	{
		*GroupNamePtr = InGroupName;
		return true;
	}
	return false;
}

bool FAnimNode_MotionMatching::SetGroupRole(EAnimGroupRole::Type InRole)
{
#if WITH_EDITORONLY_DATA
	GroupRole = InRole;
#endif
	
	if(TEnumAsByte<EAnimGroupRole::Type>* GroupRolePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(TEnumAsByte<EAnimGroupRole::Type>, GroupRole))
	{
		*GroupRolePtr = InRole;
		return true;
	}
	return false;
}

bool FAnimNode_MotionMatching::SetGroupMethod(EAnimSyncMethod InMethod)
{
#if WITH_EDITORONLY_DATA
	Method = InMethod;
#endif
	if(EAnimSyncMethod* MethodPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(EAnimSyncMethod, Method))
	{
		*MethodPtr = InMethod;
		return true;
	}
	return false;
}

bool FAnimNode_MotionMatching::SetOverridePositionWhenJoiningSyncGroupAsLeader(bool InOverridePositionWhenJoiningSyncGroupAsLeader)
{
#if WITH_EDITORONLY_DATA
	bOverridePositionWhenJoiningSyncGroupAsLeader = InOverridePositionWhenJoiningSyncGroupAsLeader;
#endif
	if(bool* bOverridePositionWhenJoiningSyncGroupAsLeaderPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bOverridePositionWhenJoiningSyncGroupAsLeader))
	{
		*bOverridePositionWhenJoiningSyncGroupAsLeaderPtr = InOverridePositionWhenJoiningSyncGroupAsLeader;
		return true;
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
