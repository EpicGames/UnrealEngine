// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/AnimNode_PoseSearchHistoryCollector.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNodeMessages.h"
#include "Animation/AnimStats.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkinnedAsset.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchHistoryAttribute.h"
#include "PoseSearch/PoseSearchInteractionValidator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_PoseSearchHistoryCollector)

#define LOCTEXT_NAMESPACE "AnimNode_PoseSearchHistoryCollector"

#if ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
static bool GVarAnimPoseHistoryDebugDrawBeforeCollection = false;
static FAutoConsoleVariableRef CVarAnimPoseHistoryDebugDrawBeforeCollection(TEXT("a.AnimNode.PoseHistory.DebugDrawBeforeCollection"), GVarAnimPoseHistoryDebugDrawBeforeCollection, TEXT("Debug Draw Pose History before collection"));
#endif // ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG

/////////////////////////////////////////////////////
// FAnimNode_PoseSearchHistoryCollector_Base

void FAnimNode_PoseSearchHistoryCollector_Base::OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance)
{
	PoseHistoryPtr = MakeShareable(new UE::PoseSearch::FGenerateTrajectoryPoseHistory());
}

void FAnimNode_PoseSearchHistoryCollector_Base::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	using namespace UE::PoseSearch;

	Super::Initialize_AnyThread(Context);

	check(Context.AnimInstanceProxy);
	const UObject* AnimContext = Context.AnimInstanceProxy->GetAnimInstanceObject();
	check(AnimContext);
	CheckInteractionThreadSafety(AnimContext);

	// reinitializing PoseHistoryPtr if invalid here. it could happen when recompiling animation 
	// blue prints when OnInitializeAnimInstance is not called
	if (!PoseHistoryPtr.IsValid())
	{
		PoseHistoryPtr = MakeShareable(new UE::PoseSearch::FGenerateTrajectoryPoseHistory());
	}

	PoseHistoryPtr->Initialize_AnyThread(PoseCount, SamplingInterval);

	FMemMark Mark(FMemStack::Get());
	UE::PoseSearch::FAIPComponentSpacePoseProvider ComponentSpacePoseProvider(Context.AnimInstanceProxy);
	if (ComponentSpacePoseProvider.GetSkeletonAsset())
	{
		PoseHistoryPtr->EvaluateComponentSpace_AnyThread(0.f, ComponentSpacePoseProvider, bStoreScales,
			RootBoneRecoveryTime, RootBoneTranslationRecoveryRatio, RootBoneRotationRecoveryRatio, true, true,
			GetRequiredBones(Context.AnimInstanceProxy), FBlendedCurve(), CollectedCurves
#if WITH_EDITORONLY_DATA
			, GetContextTransform(AnimContext, false)
#endif //WITH_EDITORONLY_DATA
		);
	}
}

TArray<FBoneIndexType> FAnimNode_PoseSearchHistoryCollector_Base::GetRequiredBones(const FAnimInstanceProxy* AnimInstanceProxy) const
{
	check(AnimInstanceProxy);
	return GetRequiredBones(AnimInstanceProxy->GetSkelMeshComponent());
}

TArray<FBoneIndexType> FAnimNode_PoseSearchHistoryCollector_Base::GetRequiredBones(const USkeletalMeshComponent* SkeletalMeshComponent) const
{
	TArray<FBoneIndexType> RequiredBones;
	if (!CollectedBones.IsEmpty() && SkeletalMeshComponent)
	{
		if (const USkinnedAsset* SkinnedAsset = SkeletalMeshComponent->GetSkinnedAsset())
		{
			if (const USkeleton* Skeleton = SkinnedAsset->GetSkeleton())
			{
				RequiredBones.Reserve(CollectedBones.Num());
				for (const FBoneReference& BoneReference : CollectedBones)
				{
					FBoneReference BoneReferenceCopy = BoneReference;
					if (BoneReferenceCopy.Initialize(Skeleton))
					{
						RequiredBones.AddUnique(BoneReferenceCopy.BoneIndex);
					}
				}
			}
		}
	}
	return RequiredBones;
}

void FAnimNode_PoseSearchHistoryCollector_Base::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread);
	check(Context.AnimInstanceProxy);

	Super::CacheBones_AnyThread(Context);

	bCacheBones = true;
}

void FAnimNode_PoseSearchHistoryCollector_Base::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	using namespace UE::PoseSearch;

	check(Context.AnimInstanceProxy);
	CheckInteractionThreadSafety(Context.AnimInstanceProxy->GetAnimInstanceObject());

	GetEvaluateGraphExposedInputs().Execute(Context);
	check(PoseHistoryPtr.IsValid());
	
	PoseHistoryPtr->bGenerateTrajectory = bGenerateTrajectory;
	if (PoseHistoryPtr->bGenerateTrajectory)
	{
		PoseHistoryPtr->TrajectoryDataSampling.NumHistorySamples = FMath::Max(PoseCount, TrajectoryHistoryCount);
		PoseHistoryPtr->TrajectoryDataSampling.SecondsPerHistorySample = SamplingInterval;
		PoseHistoryPtr->TrajectoryDataSampling.NumPredictionSamples = TrajectoryPredictionCount;
		PoseHistoryPtr->TrajectoryDataSampling.SecondsPerPredictionSample = PredictionSamplingInterval;
		PoseHistoryPtr->TrajectoryData = TrajectoryData;

		PoseHistoryPtr->GenerateTrajectory(Context.AnimInstanceProxy->GetAnimInstanceObject(), Context.GetDeltaTime());
	}
	else
	{
		PoseHistoryPtr->SetTrajectory(TransformTrajectory, TrajectorySpeedMultiplier);
	}

	PoseHistoryPtr->bIsTrajectoryGeneratedBeforePreUpdate = false;
	UpdateCounter.SynchronizeWith(Context.AnimInstanceProxy->GetUpdateCounter());
}

/////////////////////////////////////////////////////
// FAnimNode_PoseSearchHistoryCollector

void FAnimNode_PoseSearchHistoryCollector::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread);
	Super::Initialize_AnyThread(Context);
	Source.Initialize(Context);
}

void FAnimNode_PoseSearchHistoryCollector::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	Super::CacheBones_AnyThread(Context);
	Source.CacheBones(Context);
}

void FAnimNode_PoseSearchHistoryCollector::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread);
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(PoseSearchHistoryCollector, !IsInGameThread());

	using namespace UE::PoseSearch;

	Super::Evaluate_AnyThread(Output);
	Source.Evaluate(Output);

	check(Output.AnimInstanceProxy);
	const UObject* AnimContext = Output.AnimInstanceProxy->GetAnimInstanceObject();
	check(AnimContext);
	CheckInteractionThreadSafety(AnimContext);

	const bool bNeedsReset = bResetOnBecomingRelevant && UpdateCounter.HasEverBeenUpdated() && !UpdateCounter.WasSynchronizedCounter(Output.AnimInstanceProxy->GetUpdateCounter());

	FCSPose<FCompactPose> ComponentSpacePose;
	ComponentSpacePose.InitPose(Output.Pose);

	TArray<FBoneIndexType> RequiredBones;
	if (bCacheBones)
	{
		// array of skeleton bone indexes (encoded as FBoneIndexType)
		RequiredBones = GetRequiredBones(Output.AnimInstanceProxy);
	}

	if (FPoseHistory* PoseHistory = GetPoseHistoryPtr())
	{
#if ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
		if (GVarAnimPoseHistoryDebugDrawBeforeCollection)
		{
			FColor Color;
#if WITH_EDITORONLY_DATA
			Color = DebugColor.ToFColor(true);
#else // WITH_EDITORONLY_DATA
			Color = FColor::Red;
#endif // WITH_EDITORONLY_DATA
			PoseHistory->DebugDraw(*Output.AnimInstanceProxy, Color);
		}
#endif // ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG

		UE::PoseSearch::FComponentSpacePoseProvider ComponentSpacePoseProvider(ComponentSpacePose);
		PoseHistory->EvaluateComponentSpace_AnyThread(Output.AnimInstanceProxy->GetDeltaSeconds(), ComponentSpacePoseProvider, bStoreScales,
			RootBoneRecoveryTime, RootBoneTranslationRecoveryRatio, RootBoneRotationRecoveryRatio, bNeedsReset, bCacheBones,
			RequiredBones, Output.Curve, MakeConstArrayView(CollectedCurves)
#if WITH_EDITORONLY_DATA
			, GetContextTransform(AnimContext, false)
#endif //WITH_EDITORONLY_DATA
		);

		// Fill in the pose history into a custom attribute for access downstream
		if (FPoseHistoryAnimationAttribute* PoseHistoryAttribute = Output.CustomAttributes.FindOrAdd<FPoseHistoryAnimationAttribute>(UE::PoseSearch::PoseHistoryAttributeId))
		{
			PoseHistoryAttribute->PoseHistory = PoseHistory;
			PoseHistoryAttribute->ScopeObject = Output.GetAnimInstanceObject();
		}

#if ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
		if (!GVarAnimPoseHistoryDebugDrawBeforeCollection)
		{
			FColor Color;
#if WITH_EDITORONLY_DATA
			Color = DebugColor.ToFColor(true);
#else // WITH_EDITORONLY_DATA
			Color = FColor::Red;
#endif // WITH_EDITORONLY_DATA
			PoseHistory->DebugDraw(*Output.AnimInstanceProxy, Color);
		}
#endif // ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
	}

	bCacheBones = false;
}

void FAnimNode_PoseSearchHistoryCollector::GatherDebugData(FNodeDebugData& DebugData)
{
	Super::GatherDebugData(DebugData);
	Source.GatherDebugData(DebugData);
}

/////////////////////////////////////////////////////
// FAnimNode_PoseSearchComponentSpaceHistoryCollector

void FAnimNode_PoseSearchComponentSpaceHistoryCollector::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread);
	Super::Initialize_AnyThread(Context);
	Source.Initialize(Context);
}

void FAnimNode_PoseSearchComponentSpaceHistoryCollector::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	Super::CacheBones_AnyThread(Context);
	Source.CacheBones(Context);
}

void FAnimNode_PoseSearchComponentSpaceHistoryCollector::EvaluateComponentSpace_AnyThread(FComponentSpacePoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateComponentSpace_AnyThread);
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(PoseSearchComponentSpaceHistoryCollector, !IsInGameThread());

	using namespace UE::PoseSearch;

	Super::EvaluateComponentSpace_AnyThread(Output);
	Source.EvaluateComponentSpace(Output);

	check(Output.AnimInstanceProxy);
	const UObject* AnimContext = Output.AnimInstanceProxy->GetAnimInstanceObject();
	check(AnimContext);
	CheckInteractionThreadSafety(AnimContext);

	const bool bNeedsReset = bResetOnBecomingRelevant && UpdateCounter.HasEverBeenUpdated() && !UpdateCounter.WasSynchronizedCounter(Output.AnimInstanceProxy->GetUpdateCounter());

	TArray<FBoneIndexType> RequiredBones;
	if (bCacheBones)
	{
		// array of skeleton bone indexes (encoded as FBoneIndexType)
		RequiredBones = GetRequiredBones(Output.AnimInstanceProxy);
	}

	if (FPoseHistory* PoseHistory = GetPoseHistoryPtr())
	{
#if ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
		if (GVarAnimPoseHistoryDebugDrawBeforeCollection)
		{
			FColor Color;
#if WITH_EDITORONLY_DATA
			Color = DebugColor.ToFColor(true);
#else // WITH_EDITORONLY_DATA
			Color = FColor::Red;
#endif // WITH_EDITORONLY_DATA
			PoseHistory->DebugDraw(*Output.AnimInstanceProxy, Color);
		}
#endif // ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG

		UE::PoseSearch::FComponentSpacePoseProvider ComponentSpacePoseProvider(Output.Pose);
		PoseHistory->EvaluateComponentSpace_AnyThread(Output.AnimInstanceProxy->GetDeltaSeconds(), ComponentSpacePoseProvider, bStoreScales,
			RootBoneRecoveryTime, RootBoneTranslationRecoveryRatio, RootBoneRotationRecoveryRatio, bNeedsReset, bCacheBones,
			RequiredBones, Output.Curve, MakeConstArrayView(CollectedCurves)
#if WITH_EDITORONLY_DATA
			, GetContextTransform(AnimContext, false)
#endif //WITH_EDITORONLY_DATA
		);

		// Fill in the pose history into a custom attribute for access downstream
		if (FPoseHistoryAnimationAttribute* PoseHistoryAttribute = Output.CustomAttributes.FindOrAdd<FPoseHistoryAnimationAttribute>(UE::PoseSearch::PoseHistoryAttributeId))
		{
			PoseHistoryAttribute->PoseHistory = PoseHistory;
			PoseHistoryAttribute->ScopeObject = Output.GetAnimInstanceObject();
		}

#if ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
		if (!GVarAnimPoseHistoryDebugDrawBeforeCollection)
		{
			FColor Color;
#if WITH_EDITORONLY_DATA
			Color = DebugColor.ToFColor(true);
#else // WITH_EDITORONLY_DATA
			Color = FColor::Red;
#endif // WITH_EDITORONLY_DATA
			PoseHistory->DebugDraw(*Output.AnimInstanceProxy, Color);
		}
#endif // ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
	}

	bCacheBones = false;
}

void FAnimNode_PoseSearchHistoryCollector::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread);

	Super::Update_AnyThread(Context);
	UE::Anim::TScopedGraphMessage<UE::PoseSearch::FPoseHistoryProvider> ScopedMessage(Context, this);
	Source.Update(Context);
}

void FAnimNode_PoseSearchComponentSpaceHistoryCollector::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread);

	Super::Update_AnyThread(Context);
	UE::Anim::TScopedGraphMessage<UE::PoseSearch::FPoseHistoryProvider> ScopedMessage(Context, this);
	Source.Update(Context);
}

void FAnimNode_PoseSearchComponentSpaceHistoryCollector::GatherDebugData(FNodeDebugData& DebugData)
{
	Super::GatherDebugData(DebugData);
	Source.GatherDebugData(DebugData);
}

#undef LOCTEXT_NAMESPACE
