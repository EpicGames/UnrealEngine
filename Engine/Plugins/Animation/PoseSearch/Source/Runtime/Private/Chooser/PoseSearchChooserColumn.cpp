// Copyright Epic Games, Inc. All Rights Reserved.
#include "PoseSearch/Chooser/PoseSearchChooserColumn.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimSequence.h"
#include "Chooser.h"
#include "ChooserIndexArray.h"
#include "ChooserPropertyAccess.h"
#include "ChooserTrace.h"
#include "ChooserTypes.h"
#include "IChooserParameterBool.h"
#include "IChooserParameterEnum.h"
#include "IChooserParameterFloat.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchLibrary.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "Templates/Greater.h"
#if WITH_EDITOR
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchChooserColumn)

#define LOCTEXT_NAMESPACE "FPoseSearchColumn"

//#define ENSURE_MAPPING_VALIDATION(InExpression) ensure(InExpression)
#define ENSURE_MAPPING_VALIDATION(InExpression) 
//#define DO_ENSURE_CONSISTENCY(InExpression) ensure(InExpression)
#define DO_ENSURE_CONSISTENCY(InExpression) 

namespace UE::PoseSearch
{
	// searching for FChooserPlayerSettings in the Context.Params
	static const FChooserPlayerSettings* GetChooserPlayerSettings(const FChooserEvaluationContext& Context)
	{
		for (const FStructView& Param : Context.Params)
		{
			if (const FChooserPlayerSettings* ChooserPlayerSettings = Param.GetPtr<FChooserPlayerSettings>())
			{
				return ChooserPlayerSettings;
			}
		}
		return nullptr;
	}

	static const FInstancedStruct& GetResult(const UChooserTable* Chooser, int32 RowIndex)
	{
		check(Chooser);

#if WITH_EDITORONLY_DATA
		if (!Chooser->IsCookedData())
		{
			if (Chooser->ResultsStructs.IsValidIndex(RowIndex))
			{
				return Chooser->ResultsStructs[RowIndex];
			}
			return Chooser->FallbackResult;
		}
#endif // WITH_EDITORONLY_DATA
		
		if (Chooser->CookedResults.IsValidIndex(RowIndex))
		{
			return Chooser->CookedResults[RowIndex];
		}
		return Chooser->FallbackResult;
	}

#if WITH_EDITOR
	static UObject* GetReferencedObject(const UChooserTable* Chooser, int32 RowIndex)
	{
		const FInstancedStruct& Result = GetResult(Chooser, RowIndex);
		if (Result.IsValid())
		{
			return Result.Get<FObjectChooserBase>().GetReferencedObject();
		}
		return nullptr;
	}
#endif // WITH_EDITOR

	struct FPoseSearchColumnScratchArea
	{
		TArray<FSearchResult> SearchResults;
		const IPoseHistory* PoseHistory = nullptr;
		const FChooserPlayerSettings* ChooserPlayerSettings = nullptr;
		TSharedPtr<UE::PoseSearch::FPoseHistory, ESPMode::ThreadSafe> PoseHistoryPtr;

#if DO_CHECK
		const FPoseSearchColumn* DebugOwner = nullptr;
#endif // DO_CHECK
	};

	FArchive& operator<<(FArchive& Ar, FActiveColumnCost& ActiveColumnCost)
	{
		Ar << ActiveColumnCost.RowIndex;
		Ar << ActiveColumnCost.RowCost;
		return Ar;
	}

	typedef TArray<FPoseSearchColumn*, TInlineAllocator<16>> FPoseSearchColumnPtrs;
	typedef TSet<const UChooserTable*, DefaultKeyFuncs<const UChooserTable*>, TInlineSetAllocator<32>> FVisitedChoosers;
	typedef TFunction<void(const FInstancedStruct& ResultsStruct, const UChooserTable* Chooser, int32 RowIndex, const FPoseSearchColumnPtrs& PoseSearchColumnPtrs)> FProcessRow;

	static void IterateChooserRows(UChooserTable* Chooser, FVisitedChoosers& VisitedChoosers, const FProcessRow& ProcessRow);

	static void ProcessResultsStruct(const FInstancedStruct& ResultsStruct, const UChooserTable* Chooser, int32 RowIndex, FVisitedChoosers& VisitedChoosers, const FPoseSearchColumnPtrs& PoseSearchColumnPtrs, const FProcessRow& ProcessRow)
	{
		if (ResultsStruct.IsValid())
		{
			if (const FNestedChooser* NestedChooser = ResultsStruct.GetPtr<FNestedChooser>())
			{
				if (NestedChooser->Chooser)
				{
					IterateChooserRows(NestedChooser->Chooser, VisitedChoosers, ProcessRow);
				}
			}
			else
			{
				ProcessRow(ResultsStruct, Chooser, RowIndex, PoseSearchColumnPtrs);
			}
		}
	}

	static void IterateChooserRows(UChooserTable* Chooser, FVisitedChoosers& VisitedChoosers, const FProcessRow& ProcessRow)
	{
		if (!ensure(Chooser))
		{
			return;
		}

		bool bAlreadyVisited = false;
		VisitedChoosers.Add(Chooser, &bAlreadyVisited);
		if (bAlreadyVisited)
		{
			return;
		}

		int32 NumRows = 0;
#if WITH_EDITORONLY_DATA
		if (!Chooser->IsCookedData())
		{
			NumRows = Chooser->ResultsStructs.Num();
		}
		else
#endif
		{
			NumRows = Chooser->CookedResults.Num();
		}

		FPoseSearchColumnPtrs PoseSearchColumnPtrs;
		for (FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
		{
			if (FPoseSearchColumn* PoseSearchColumn = ColumnData.GetMutablePtr<FPoseSearchColumn>())
			{
				// adding only columns that has already been upgraded / correcly set up
				if (NumRows == PoseSearchColumn->GetNumRows())
				{
#if WITH_EDITOR
					// making sure Initialize has been called on PoseSearchColumn
					ensure(PoseSearchColumn->GetChooser());
#endif // WITH_EDITOR
					PoseSearchColumnPtrs.Add(PoseSearchColumn);
				}
			}
		}

#if WITH_EDITORONLY_DATA
		if (!Chooser->IsCookedData())
		{
			for (int32 ResultIndex = 0; ResultIndex < Chooser->ResultsStructs.Num(); ++ResultIndex)
			{
				if (!Chooser->IsRowDisabled(ResultIndex))
				{
					ProcessResultsStruct(Chooser->ResultsStructs[ResultIndex], Chooser, ResultIndex, VisitedChoosers, PoseSearchColumnPtrs, ProcessRow);
				}
			}
		}
		else
#endif
		{
			for (int32 ResultIndex = 0; ResultIndex < Chooser->CookedResults.Num(); ++ResultIndex)
			{
				ProcessResultsStruct(Chooser->CookedResults[ResultIndex], Chooser, ResultIndex, VisitedChoosers, PoseSearchColumnPtrs, ProcessRow);
			}
		}

		if (Chooser->FallbackResult.IsValid())
		{
			ProcessResultsStruct(Chooser->FallbackResult, Chooser, ChooserColumn_SpecialIndex_Fallback, VisitedChoosers, PoseSearchColumnPtrs, ProcessRow);
		}
	}

	static bool CheckConsistency(const UChooserTable* Chooser)
	{
		bool bSuccess = true;
#if WITH_EDITOR
		if (!Chooser)
		{
			bSuccess = false;
		}
		else if (!Chooser->GetRootChooser())
		{
			bSuccess = false;
		}
		else
		{
			FVisitedChoosers VisitedChoosers;
			IterateChooserRows(const_cast<UChooserTable*>(Chooser->GetRootChooser()), VisitedChoosers,
				[&bSuccess](const FInstancedStruct& ResultsStruct, const UChooserTable* Chooser, int32 RowIndex, const FPoseSearchColumnPtrs& PoseSearchColumnPtrs)
				{
					for (FPoseSearchColumn* PoseSearchColumnPtr : PoseSearchColumnPtrs)
					{
						if (UPoseSearchDatabase* Database = PoseSearchColumnPtr->GetRowValue(RowIndex).Database)
						{
							if (Database->GetOuter() != Chooser->GetRootChooser())
							{
								bSuccess = false;
							}
						}
					}
				});
		}
#endif // WITH_EDITOR
		return bSuccess;
	}

	FMapping::FMapping(UChooserTable* Chooser)
	{
		Init(Chooser);
	}

	void FMapping::Init(UChooserTable* Chooser)
	{
		Map.Reset();

		if (Chooser)
		{
			check(Chooser == Chooser->GetRootChooser());
			FVisitedChoosers VisitedChoosers;
			IterateChooserRows(Chooser, VisitedChoosers,
				[this](const FInstancedStruct& ResultsStruct, const UChooserTable* Chooser, int32 RowIndex, const FPoseSearchColumnPtrs& PoseSearchColumnPtrs)
				{
					for (FPoseSearchColumn* PoseSearchColumnPtr : PoseSearchColumnPtrs)
					{
						UPoseSearchDatabase* Database = PoseSearchColumnPtr->GetRowValue(RowIndex).Database;
						ensure(!Database || Database->GetOuter() == Chooser->GetRootChooser());

						FTableCells& TableCells = Map.FindOrAdd(Database);
						FTableCell TableCell = { PoseSearchColumnPtr, RowIndex };
						if (ensure(!TableCells.Contains(TableCell)))
						{
							TableCells.Add(TableCell);
						}
					}
				});
		}
	}

	bool FMapping::Equals(const FMapping& Other) const
	{
		return Map.OrderIndependentCompareEqual(Other.Map);
	}

	int32 FMapping::GetNumIndexesInDatabase(const UPoseSearchDatabase* Database) const
	{
		if (const FTableCells* TableCells = Map.Find(Database))
		{
			return TableCells->Num();
		}
		return 0;
	}

	int32 FMapping::FindIndexInDatabase(int32 RowIndex, const FPoseSearchColumn* Column, const UPoseSearchDatabase* Database) const
	{
		if (const FTableCells* TableCells = Map.Find(Database))
		{
			for (int32 IndexInDatabase = 0; IndexInDatabase < TableCells->Num(); ++IndexInDatabase)
			{
				const FTableCell& TableCell = (*TableCells)[IndexInDatabase];
				if (TableCell.Column == Column && TableCell.RowIndex == RowIndex)
				{
					return IndexInDatabase;
				}
			}
		}
		return INDEX_NONE;
	}

	const FPoseSearchDatabaseAnimationAsset* FMapping::FindDatabaseAnimationAsset(int32 IndexInDatabase, const UPoseSearchDatabase* Database) const
	{
		const FPoseSearchDatabaseAnimationAsset* DatabaseAnimationAsset = nullptr;
		if (ensure(IndexInDatabase >= 0 && Database))
		{
			if (const FTableCells* TableCells = Map.Find(Database))
			{
				if (TableCells->IsValidIndex(IndexInDatabase))
				{
					const FTableCell& TableCell = (*TableCells)[IndexInDatabase];
					DatabaseAnimationAsset = &TableCell.Column->GetRowValue(TableCell.RowIndex).Data;
#if WITH_EDITOR
					if (DatabaseAnimationAsset)
					{
						if (const UChooserTable* Chooser = TableCell.Column->GetChooser())
						{
							if (!ensure(GetReferencedObject(Chooser, TableCell.RowIndex) == DatabaseAnimationAsset->AnimAsset))
							{
								UE_LOGF(LogPoseSearch, Error, "FMapping::FindDatabaseAnimationAsset - Mismatch between chooser table '%ls' with root '%ls' and Database '%ls' for row %d", *Chooser->GetName(), *Chooser->GetRootChooser()->GetName(), *Database->GetName(), TableCell.RowIndex);
							}
						}
					}
#endif // WITH_EDITOR
				}
			}
		}
		return DatabaseAnimationAsset;
	}

#if WITH_EDITORONLY_DATA
	void FMapping::SetAnimationAssetAt(const FPoseSearchDatabaseAnimationAsset& AnimationAsset, int32 IndexInDatabase, const UPoseSearchDatabase* Database)
	{
		if (ensure(IndexInDatabase >= 0 && Database))
		{
			if (FTableCells* TableCells = Map.Find(Database))
			{
				if (TableCells->IsValidIndex(IndexInDatabase))
				{
					FTableCell& TableCell = (*TableCells)[IndexInDatabase];

					DO_ENSURE_CONSISTENCY(CheckConsistency(TableCell.Column->GetChooser()));

					FPoseSearchDatabaseAnimationAsset& DatabaseAnimationAsset = TableCell.Column->EditRowValue(TableCell.RowIndex).Data;
					DatabaseAnimationAsset = AnimationAsset;
					// enforcing RowValue.Data.AnimAsset to be equal to ReferencedObject
					DatabaseAnimationAsset.AnimAsset = GetReferencedObject(TableCell.Column->GetChooser(), TableCell.RowIndex);

					DO_ENSURE_CONSISTENCY(CheckConsistency(TableCell.Column->GetChooser()));
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA

	int32 FMapping::GetRowIndex(int32 IndexInDatabase, const UPoseSearchDatabase* Database) const
	{
		if (ensure(IndexInDatabase >= 0 && Database))
		{
			if (const FTableCells* TableCells = Map.Find(Database))
			{
				if (IndexInDatabase < TableCells->Num())
				{
					return (*TableCells)[IndexInDatabase].RowIndex;
				}
			}
		}
		ensure(false);
		return INDEX_NONE;
	}

	void FMapping::ApplyMappingToAllColumns()
	{
		for (TPair<UPoseSearchDatabase*, FTableCells>& Pair : Map)
		{
			for (FTableCell& TableCell : Pair.Value)
			{
				TableCell.Column->UpdateMapping(*this);
			}
		}
	}

	FMapping* FMapping::FindMapping(UChooserTable* Chooser)
	{
		if (Chooser)
		{
			for (FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
			{
				if (FPoseSearchColumn* PoseSearchColumn = ColumnData.GetMutablePtr<FPoseSearchColumn>())
				{
					return &PoseSearchColumn->GetMapping();
				}
			}

#if WITH_EDITORONLY_DATA
			if (!Chooser->IsCookedData())
			{
				for (int32 ResultIndex = 0; ResultIndex < Chooser->ResultsStructs.Num(); ++ResultIndex)
				{
					if (!Chooser->IsRowDisabled(ResultIndex))
					{
						if (const FNestedChooser* NestedChooser = Chooser->ResultsStructs[ResultIndex].GetPtr<FNestedChooser>())
						{
							if (FMapping* Mapping = FindMapping(NestedChooser->Chooser))
							{
								return Mapping;
							}
						}
					}
				}
			}
			else
#endif
			{
				for (int32 ResultIndex = 0; ResultIndex < Chooser->CookedResults.Num(); ++ResultIndex)
				{
					if (const FNestedChooser* NestedChooser = Chooser->CookedResults[ResultIndex].GetPtr<FNestedChooser>())
					{
						if (FMapping* Mapping = FindMapping(NestedChooser->Chooser))
						{
							return Mapping;
						}
					}
				}
			}

			if (Chooser->FallbackResult.IsValid())
			{
				if (const FNestedChooser* NestedChooser = Chooser->FallbackResult.GetPtr<FNestedChooser>())
				{
					if (FMapping* Mapping = FindMapping(NestedChooser->Chooser))
					{
						return Mapping;
					}
				}
			}
		}

		return nullptr;
	}
} // namespace UE::PoseSearch

FPoseSearchColumn::FPoseSearchColumn(const FPoseSearchColumn& Other)
	: FChooserColumnBase(Other)
	, RowValues(Other.RowValues)
	, FallbackValue(Other.FallbackValue)
	, Chooser(Other.Chooser)
	// no need to copy Mapping since it has to be recomputed, because it holds references to Other.Chooser!
	, Mapping()
	, PoseReselectHistory(Other.PoseReselectHistory)
	, MaxNumberOfResults(Other.MaxNumberOfResults)
	, InterruptMode(Other.InterruptMode)
	, InputValue(Other.InputValue)
	, OutputStartTime(Other.OutputStartTime)
	, OutputMirror(Other.OutputMirror)
	, OutputForceBlendTo(Other.OutputForceBlendTo)
	, OutputCost(Other.OutputCost)
{
	// has Other column been properly initialized?
	ensure(Other.Chooser);

	// resetting the references of RowValues and FallbackValue Database(s) to null, to avoid referencing Other.Chooser nested objects,
	// and expecting CopyRow and CopyFallback APIs to set them accordingly
	for (FPoseSearchColumnRow& RowValue : RowValues)
	{
		RowValue.Database = nullptr;
	}
	FallbackValue.Database = nullptr;
}

FPoseSearchColumn& FPoseSearchColumn::operator=(const FPoseSearchColumn& Other)
{
	if (this != &Other)
	{
		this->~FPoseSearchColumn();
		new(this) FPoseSearchColumn(Other);
	}
	return *this;
}

const FPoseSearchColumnRow& FPoseSearchColumn::GetRowValue(int32 RowIndex) const
{
	using namespace UE::PoseSearch;
	if (RowIndex == ChooserColumn_SpecialIndex_Fallback)
	{
		return FallbackValue;
	}
	if (RowValues.IsValidIndex(RowIndex))
	{
		return RowValues[RowIndex];
	}
	return FallbackValue;
}

FPoseSearchColumnRow& FPoseSearchColumn::EditRowValue(int32 RowIndex)
{
	using namespace UE::PoseSearch;
	if (RowIndex == ChooserColumn_SpecialIndex_Fallback)
	{
		return FallbackValue;
	}
	if (ensure(RowValues.IsValidIndex(RowIndex)))
	{
		return RowValues[RowIndex];
	}
	return FallbackValue;
}

void FPoseSearchColumn::PostLoad()
{
	using namespace UE::PoseSearch;

#if WITH_EDITORONLY_DATA
	
	// upgrading InternalDatabase_DEPRECATED into their corresponding Databases
	if (InternalDatabase_DEPRECATED)
	{
		// InternalDatabase_DEPRECATED was mapped with this indices conventions:
		// RowIndex -> DatabaseAssetIndex
		// 0 -> ChooserColumn_SpecialIndex_Fallback
		// RowIndex -> RowIndex + 1

		ensure(!Chooser);
		Chooser = Cast<UChooserTable>(InternalDatabase_DEPRECATED->GetOuter());
		ensure(Chooser);

		InternalDatabase_DEPRECATED->Modify();
		const int32 NumRows = InternalDatabase_DEPRECATED->GetNumAnimationAssets() - 1;
		ensure(NumRows >= 0);

		RowValues.SetNum(NumRows);
		for (int32 RowIndex = 0; RowIndex < NumRows; ++RowIndex)
		{
			if (const FPoseSearchDatabaseAnimationAsset* DatabaseAnimationAsset = InternalDatabase_DEPRECATED->GetDatabaseAnimationAsset(RowIndex + 1))
			{
				if (DatabaseAnimationAsset->GetAnimationAsset())
				{
					RowValues[RowIndex].Data = *DatabaseAnimationAsset;
					RowValues[RowIndex].Database = InternalDatabase_DEPRECATED;
					// enforcing Data.AnimAsset to be equal to ReferencedObject
					RowValues[RowIndex].Data.AnimAsset = GetReferencedObject(Chooser, RowIndex);
				}
			}
		}

		if (const FPoseSearchDatabaseAnimationAsset* DatabaseAnimationAsset = InternalDatabase_DEPRECATED->GetDatabaseAnimationAsset(0))
		{
			if (DatabaseAnimationAsset->GetAnimationAsset())
			{
				FallbackValue.Data = *DatabaseAnimationAsset;
				FallbackValue.Database = InternalDatabase_DEPRECATED;
				// enforcing Data.AnimAsset to be equal to ReferencedObject
				FallbackValue.Data.AnimAsset = GetReferencedObject(Chooser, ChooserColumn_SpecialIndex_Fallback);
			}
		}

		// removing all the AnimationAssets from InternalDatabase_DEPRECATED
		const int32 NumAnimationAssetsToRemove = InternalDatabase_DEPRECATED->GetNumAnimationAssets();
		for (int32 AnimationAssetIndex = NumAnimationAssetsToRemove - 1; AnimationAssetIndex >= 0; --AnimationAssetIndex)
		{
			InternalDatabase_DEPRECATED->RemoveAnimationAssetAt(AnimationAssetIndex);
		}

		// changing InternalDatabase_DEPRECATED outer to RootChooser
		UChooserTable* RootChooser = Chooser->GetRootChooser();
		const FName DatabaseName = *("PSD_" + Chooser->GetName());
		const FName DatabaseUniqueName = MakeUniqueObjectName(RootChooser, UPoseSearchDatabase::StaticClass(), DatabaseName);
		InternalDatabase_DEPRECATED->Rename(*DatabaseUniqueName.ToString(), RootChooser);
		RootChooser->AddNestedObject(InternalDatabase_DEPRECATED);

		InternalDatabase_DEPRECATED->PostEditChange();
		
		InternalDatabase_DEPRECATED = nullptr;
	}
#endif // WITH_EDITORONLY_DATA

	if (Chooser)
	{
		FMapping(Chooser->GetRootChooser()).ApplyMappingToAllColumns();
	}

	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
}

UPoseSearchDatabase* FPoseSearchColumn::GetDatabase(int32 RowIndex) const
{
	UPoseSearchDatabase* Database = GetRowValue(RowIndex).Database;
	ensure(!Database || Database->GetOuter() == Chooser->GetRootChooser());
	return Database;
}

#if WITH_EDITOR
void FPoseSearchColumn::SwitchDatabase(UPoseSearchDatabase* NewDatabase, int32 RowIndex)
{
	using namespace UE::PoseSearch;
	
	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));

	UChooserTable* RootChooser = Chooser->GetRootChooser();
	TObjectPtr<UPoseSearchDatabase>& Database = EditRowValue(RowIndex).Database;
	ensure(!Database || Database->GetOuter() == RootChooser);
	ensure(!NewDatabase || NewDatabase->GetOuter() == RootChooser);
	if (Database != NewDatabase)
	{
		if (Database)
		{
			Database->Modify();
		}
		if (NewDatabase)
		{
			NewDatabase->Modify();
		}

		Database = NewDatabase;
		FMapping(RootChooser).ApplyMappingToAllColumns();

		if (Database)
		{
			Database->PostEditChange();
		}
		if (NewDatabase)
		{
			NewDatabase->PostEditChange();
		}
	}

	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
}

void FPoseSearchColumn::InsertAnimationAssetAt(const FPoseSearchDatabaseAnimationAsset& AnimationAsset, int32 AnimationAssetIndex, UChooserTable* Chooser, const UPoseSearchDatabase* Database)
{
	// @todo: implement me! Without this method, we're not able to add a chooser row by adding a row in the related UPoseSearchDatabase.

	using namespace UE::PoseSearch;
	//FMapping(Chooser->GetRootChooser()).ApplyMappingToAllColumns();
	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
}

void FPoseSearchColumn::RemoveAnimationAssetAt(int32 AnimationAssetIndex, UChooserTable* Chooser, const UPoseSearchDatabase* Database)
{
	// @todo: implement me! Without this method, we're not able to delete a chooser row by deleting a row in the related UPoseSearchDatabase.

	using namespace UE::PoseSearch;
	//FMapping(Chooser->GetRootChooser()).ApplyMappingToAllColumns();
	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
}

void FPoseSearchColumn::SetAnimationAssetAt(const FPoseSearchDatabaseAnimationAsset& AnimationAsset, int32 AnimationAssetIndex, UChooserTable* Chooser, const UPoseSearchDatabase* Database)
{
	using namespace UE::PoseSearch;
	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
	if (FMapping* Mapping = FMapping::FindMapping(Chooser))
	{
		ENSURE_MAPPING_VALIDATION(Mapping->Equals(FMapping(Chooser->GetRootChooser())));
		Mapping->SetAnimationAssetAt(AnimationAsset, AnimationAssetIndex, Database);
	}
	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
}
#endif // WITH_EDITOR

const FPoseSearchDatabaseAnimationAsset* FPoseSearchColumn::GetDatabaseAnimationAsset(int32 AnimationAssetIndex, UChooserTable* Chooser, const UPoseSearchDatabase* Database)
{
	using namespace UE::PoseSearch;
	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
	if (FMapping* Mapping = FMapping::FindMapping(Chooser))
	{
		ENSURE_MAPPING_VALIDATION(Mapping->Equals(FMapping(Chooser->GetRootChooser())));
		return Mapping->FindDatabaseAnimationAsset(AnimationAssetIndex, Database);
	}
	return nullptr;
}

int32 FPoseSearchColumn::GetNumAnimationAssets(UChooserTable* Chooser, const UPoseSearchDatabase* Database)
{
	using namespace UE::PoseSearch;
	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
	if (FMapping* Mapping = FMapping::FindMapping(Chooser))
	{
		ENSURE_MAPPING_VALIDATION(Mapping->Equals(FMapping(Chooser->GetRootChooser())));
		return Mapping->GetNumIndexesInDatabase(Database);
	}
	return 0;
}

void FPoseSearchColumn::UpdateMapping(UE::PoseSearch::FMapping& NewMapping)
{
	using namespace UE::PoseSearch;
	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
	Mapping = NewMapping;
	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
}

void FPoseSearchColumn::Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut, TArrayView<uint8> ScratchArea) const
{
	using namespace UE::PoseSearch;

	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));

	ENSURE_MAPPING_VALIDATION(Mapping.Equals(FMapping(Chooser->GetRootChooser())));

	ensure(ScratchArea.Num() == GetScratchAreaSize());

	FMemMark Mark(FMemStack::Get());

	FPoseSearchColumnScratchArea* PoseSearchColumnScratchArea = reinterpret_cast<FPoseSearchColumnScratchArea*>(ScratchArea.begin());
#if DO_CHECK
	ensure(PoseSearchColumnScratchArea->DebugOwner == this);
	ensure(PoseSearchColumnScratchArea->SearchResults.IsEmpty());
#endif // DO_CHECK

	FPoseHistoryReference PoseHistoryReference;
	if (const FChooserParameterPoseHistoryBase* PoseHistoryParameter = InputValue.GetPtr<FChooserParameterPoseHistoryBase>())
	{
		FPoseSearchHistory PoseSearchHistory;
		if (PoseHistoryParameter->GetValue(Context, PoseHistoryReference))
		{
			PoseSearchColumnScratchArea->PoseHistory = PoseHistoryReference.PoseHistory.Get();
		}
		else if (PoseHistoryParameter->GetValue(Context, PoseSearchHistory))
		{
			// @todo: avoid allocations...
			PoseSearchColumnScratchArea->PoseHistoryPtr = MakeShareable(new FPoseHistory());
			PoseSearchColumnScratchArea->PoseHistoryPtr->EditPoseSearchHistory() = MoveTemp(PoseSearchHistory);
			PoseSearchColumnScratchArea->PoseHistory = PoseSearchColumnScratchArea->PoseHistoryPtr.Get();
		}
	}

	PoseSearchColumnScratchArea->ChooserPlayerSettings = GetChooserPlayerSettings(Context);
	if (!PoseSearchColumnScratchArea->PoseHistory && PoseSearchColumnScratchArea->ChooserPlayerSettings && PoseSearchColumnScratchArea->ChooserPlayerSettings->AnimationUpdateContext)
	{
		if (const FPoseHistoryProvider* PoseHistoryProvider = PoseSearchColumnScratchArea->ChooserPlayerSettings->AnimationUpdateContext->GetMessage<FPoseHistoryProvider>())
		{
			PoseSearchColumnScratchArea->PoseHistory = PoseHistoryProvider->GetPoseHistoryPtr();
		}
	}

#if WITH_EDITOR
	TArray<FActiveColumnCost> TracedActiveColumnCosts;
#endif // WITH_EDITOR

	if (!PoseSearchColumnScratchArea->PoseHistory)
	{
		UE_LOGF(LogPoseSearch, Error, "FPoseSearchColumn::Filter, missing IPoseHistory when filtering chooser table '%ls' with root '%ls'", *Chooser->GetName(), *Chooser->GetRootChooser()->GetName());
	}
	else
	{
		ensure(Chooser);
		
		FStackDatabaseToAssetIndexes AssetIndexesToConsiderPerDatabase;
		for (const FChooserIndexArray::FIndexData& IndexData : IndexListIn)
		{
			const int32 RowIndex = IndexData.Index;

			if (UPoseSearchDatabase* Database = GetDatabase(RowIndex))
			{
				const int32 IndexInDatabase = Mapping.FindIndexInDatabase(RowIndex, this, Database);
				if (IndexInDatabase != INDEX_NONE)
				{
					AssetIndexesToConsiderPerDatabase.FindOrAdd(Database).Add(IndexInDatabase);
				}
			}
		}

		if (!AssetIndexesToConsiderPerDatabase.IsEmpty())
		{
			// collecting and sorting DatabasesToSearch to have a deterministic search
			bool bRoleSet = false;
			FRole Role = DefaultRole;

			TArray<const UObject*> DatabasesToSearch;
			for (TPair<const UPoseSearchDatabase*, FStackAssetIndexes>& Pair : AssetIndexesToConsiderPerDatabase)
			{
				if (Pair.Key->Schema)
				{
					if (!bRoleSet)
					{
						Role = Pair.Key->Schema->GetDefaultRole();
						bRoleSet = true;
					}
#if !NO_LOGGING
					else if (Role != Pair.Key->Schema->GetDefaultRole())
					{
						UE_LOGF(LogPoseSearch, Error, "FPoseSearchColumn::Filter inconsistent Roles across the databases referenced by chooser table '%ls' with root '%ls' (expected '%ls', found '%ls')", *Chooser->GetName(), *Chooser->GetRootChooser()->GetName(), *Role.ToString(), *Pair.Key->Schema->GetDefaultRole().ToString());
					}
#endif // !NO_LOGGING
					DatabasesToSearch.Add(Pair.Key);
				}
			}
			DatabasesToSearch.Sort();

			const FFloatInterval PoseJumpThresholdTime(0.f, 0.f);
			FSearchContext SearchContext(0.f, PoseJumpThresholdTime, FPoseSearchEvent());

			SearchContext.AddRole(Role, &Context, PoseSearchColumnScratchArea->PoseHistory);
			SearchContext.SetAssetIndexesToConsiderPerDatabase(&AssetIndexesToConsiderPerDatabase);

			FPoseSearchContinuingProperties ContinuingProperties;
			if (PoseSearchColumnScratchArea->ChooserPlayerSettings)
			{
				ContinuingProperties.PlayingAsset = PoseSearchColumnScratchArea->ChooserPlayerSettings->PlayingAsset;
				ContinuingProperties.PlayingAssetAccumulatedTime = PoseSearchColumnScratchArea->ChooserPlayerSettings->PlayingAssetAccumulatedTime;
				ContinuingProperties.bIsPlayingAssetMirrored = PoseSearchColumnScratchArea->ChooserPlayerSettings->bIsPlayingAssetMirrored;
				ContinuingProperties.PlayingAssetBlendParameters = PoseSearchColumnScratchArea->ChooserPlayerSettings->PlayingAssetBlendParameters;

				uint8 InterruptModeValue = 0;
				if (InterruptMode.IsValid() && InterruptMode.Get<FChooserParameterEnumBase>().GetValue(Context, InterruptModeValue))
				{
					ContinuingProperties.InterruptMode = (EPoseSearchInterruptMode)InterruptModeValue;
				}
			}

			FSearchResult BestSearchResult;
			if (MaxNumberOfResults == 1)
			{
				FSearchResults_Single SearchResults;
				UPoseSearchLibrary::MotionMatch(SearchContext, DatabasesToSearch, ContinuingProperties, SearchResults);
#if DO_CHECK
				SearchResults.IterateOverSearchResults([&AssetIndexesToConsiderPerDatabase](FSearchResult& SearchResult)
					{
						if (const FSearchIndexAsset* SearchIndexAsset = SearchResult.GetSearchIndexAsset())
						{
							const FStackAssetIndexes* AssetIndexesToConsider = AssetIndexesToConsiderPerDatabase.Find(SearchResult.Database.Get());
							ensure(AssetIndexesToConsider && AssetIndexesToConsider->Contains(SearchIndexAsset->GetSourceAssetIdx()));
						}
						return false;
					});
#endif // DO_CHECK

				BestSearchResult = SearchResults.GetBestResult();

				if (const FSearchIndexAsset* SearchIndexAsset = BestSearchResult.GetSearchIndexAsset())
				{
					const int32 RowIndex = Mapping.GetRowIndex(SearchIndexAsset->GetSourceAssetIdx(), BestSearchResult.Database.Get());
					ensure(RowIndex >= 0);
					PoseSearchColumnScratchArea->SearchResults.Add(BestSearchResult);
					IndexListOut.Push({ static_cast<uint32>(RowIndex), BestSearchResult.PoseCost });
#if WITH_EDITOR
					TracedActiveColumnCosts.Add({ RowIndex, BestSearchResult.PoseCost });
#endif // WITH_EDITOR
				}				
			}
			else
			{
				FSearchResults_AssetBests SearchResults;
				UPoseSearchLibrary::MotionMatch(SearchContext, DatabasesToSearch, ContinuingProperties, SearchResults);
#if DO_CHECK
				SearchResults.IterateOverSearchResults([&AssetIndexesToConsiderPerDatabase](FSearchResult& SearchResult)
					{
						if (const FSearchIndexAsset* SearchIndexAsset = SearchResult.GetSearchIndexAsset())
						{
							const FStackAssetIndexes* AssetIndexesToConsider = AssetIndexesToConsiderPerDatabase.Find(SearchResult.Database.Get());
							ensure(AssetIndexesToConsider && AssetIndexesToConsider->Contains(SearchIndexAsset->GetSourceAssetIdx()));
						}
						return false;
					});
#endif // DO_CHECK

				BestSearchResult = SearchResults.GetBestResult();
				if (MaxNumberOfResults > 0)
				{
					// keeping only up to MaxNumberOfResults results
					SearchResults.Shrink(MaxNumberOfResults);
				}

				for (const FChooserIndexArray::FIndexData& IndexData : IndexListIn)
				{
					const uint32 RowIndex = IndexData.Index;
					if (UPoseSearchDatabase* Database = GetDatabase(RowIndex))
					{
						const int32 DatabaseAssetIndex = Mapping.FindIndexInDatabase(RowIndex, this, Database);
						if (const FSearchResult* FoundSearchResult = SearchResults.FindSearchResultFor(Database, DatabaseAssetIndex))
						{
							PoseSearchColumnScratchArea->SearchResults.Add(*FoundSearchResult);
							IndexListOut.Push({ RowIndex, FoundSearchResult->PoseCost });
#if WITH_EDITOR
							TracedActiveColumnCosts.Add({ static_cast<int32>(RowIndex), FoundSearchResult->PoseCost });
#endif // WITH_EDITOR
						}
					}
				}
			}
		}
	}

#if WITH_EDITOR
	// @todo: improve how we trace chooser values, without relying on GetDebugName, since it's not a unique identifier
	FString DebugName;
	if (const FChooserParameterBase* ChooserParameterBase = InputValue.GetPtr<FChooserParameterBase>())
	{
		DebugName = ChooserParameterBase->GetDebugName();
	}
	else
	{
		DebugName = "FPoseSearchColumn";
	}
	TRACE_CHOOSER_VALUE(Context, ToCStr(DebugName), TracedActiveColumnCosts);
	if (Context.DebuggingInfo.bCurrentDebugTarget)
	{
		ActiveColumnCosts = TracedActiveColumnCosts;
	}
#endif // WITH_EDITOR
}


void FPoseSearchColumn::SetOutputs(FChooserEvaluationContext& Context, int RowIndex, TArrayView<uint8> ScratchArea) const
{
	using namespace UE::PoseSearch;

	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));

	FMemMark Mark(FMemStack::Get());

	// set an arbitrary "high" cost value so that cost threshold implementations can still wait for the pose search to find a good match
	float CostValue = 100.f;
	float StartTimeValue = 0.f;
	bool bMirrorValue = false;
	bool bForceBlendToValue = false;

	ensure(ScratchArea.Num() == GetScratchAreaSize());
	FPoseSearchColumnScratchArea* PoseSearchColumnScratchArea = reinterpret_cast<FPoseSearchColumnScratchArea*>(ScratchArea.begin());
#if DO_CHECK
	ensure(PoseSearchColumnScratchArea->DebugOwner == this);
#endif // DO_CHECK

	FSearchResult BestSearchResult;
	if (RowIndex == ChooserColumn_SpecialIndex_Fallback)
	{
		if (PoseSearchColumnScratchArea->PoseHistory)
		{
			// NoTe: PoseSearchColumnScratchArea->SearchResults can be NOT empty in case we had other columns after this FPoseSearchColumn,
			// invalidating ALL the possible results, so we have to perform another search with the fallback row

			// do the search again only with the fallback row..
			FStackDatabaseToAssetIndexes AssetIndexesToConsiderPerDatabase;
			if (UPoseSearchDatabase* Database = GetDatabase(RowIndex))
			{
				const int32 IndexInDatabase = Mapping.FindIndexInDatabase(RowIndex, this, Database);
				// also pruning out eventual nested chooser or invalid assets not supported by the database
				if (IndexInDatabase != INDEX_NONE && Database->GetAnimationAsset(IndexInDatabase))
				{
					AssetIndexesToConsiderPerDatabase.Add(Database).Add(IndexInDatabase);

					const FFloatInterval PoseJumpThresholdTime(0.f, 0.f);
					const UObject* DatabasesToSearch[] = { Database };
					FSearchContext SearchContext(0.f, PoseJumpThresholdTime, FPoseSearchEvent());

					const FRole Role = Database->Schema ? Database->Schema->GetDefaultRole() : DefaultRole;
					SearchContext.AddRole(Role, &Context, PoseSearchColumnScratchArea->PoseHistory);
					SearchContext.SetAssetIndexesToConsiderPerDatabase(&AssetIndexesToConsiderPerDatabase);

					FPoseSearchContinuingProperties ContinuingProperties;
					if (PoseSearchColumnScratchArea->ChooserPlayerSettings)
					{
						ContinuingProperties.PlayingAsset = PoseSearchColumnScratchArea->ChooserPlayerSettings->PlayingAsset;
						ContinuingProperties.PlayingAssetAccumulatedTime = PoseSearchColumnScratchArea->ChooserPlayerSettings->PlayingAssetAccumulatedTime;
						ContinuingProperties.bIsPlayingAssetMirrored = PoseSearchColumnScratchArea->ChooserPlayerSettings->bIsPlayingAssetMirrored;
						ContinuingProperties.PlayingAssetBlendParameters = PoseSearchColumnScratchArea->ChooserPlayerSettings->PlayingAssetBlendParameters;

						uint8 InterruptModeValue = 0;
						if (InterruptMode.IsValid() && InterruptMode.Get<FChooserParameterEnumBase>().GetValue(Context, InterruptModeValue))
						{
							ContinuingProperties.InterruptMode = (EPoseSearchInterruptMode)InterruptModeValue;
						}
					}

					FSearchResults_Single SearchResults;
					UPoseSearchLibrary::MotionMatch(SearchContext, DatabasesToSearch, ContinuingProperties, SearchResults);
#if DO_CHECK
					SearchResults.IterateOverSearchResults([&AssetIndexesToConsiderPerDatabase](FSearchResult& SearchResult)
						{
							if (const FSearchIndexAsset* SearchIndexAsset = SearchResult.GetSearchIndexAsset())
							{
								const FStackAssetIndexes* AssetIndexesToConsider = AssetIndexesToConsiderPerDatabase.Find(SearchResult.Database.Get());
								ensure(AssetIndexesToConsider && AssetIndexesToConsider->Contains(SearchIndexAsset->GetSourceAssetIdx()));
							}
							return false;
						});
#endif // DO_CHECK
					BestSearchResult = SearchResults.GetBestResult();
					if (const FSearchIndexAsset* SearchIndexAsset = BestSearchResult.GetSearchIndexAsset())
					{
						StartTimeValue = BestSearchResult.GetAssetTime();
						CostValue = BestSearchResult.PoseCost;
						bMirrorValue = SearchIndexAsset->IsMirrored();
						bForceBlendToValue = !BestSearchResult.bIsContinuingPoseSearch;
					}
				}
			}
		}
	}
	else
	{
		for (const FSearchResult& SearchResult : PoseSearchColumnScratchArea->SearchResults)
		{
			if (const FSearchIndexAsset* SearchIndexAsset = SearchResult.GetSearchIndexAsset())
			{
				const int32 SearchResultRowIndex = Mapping.GetRowIndex(SearchIndexAsset->GetSourceAssetIdx(), SearchResult.Database.Get());
				ensure(SearchResultRowIndex >= 0);

				if (SearchResultRowIndex == RowIndex)
				{
					StartTimeValue = SearchResult.GetAssetTime();
					CostValue = SearchResult.PoseCost;
					bMirrorValue = SearchIndexAsset->IsMirrored();
					bForceBlendToValue = !SearchResult.bIsContinuingPoseSearch;

					BestSearchResult = SearchResult;
					break;
				}
			}
		}
	}

	if (PoseSearchColumnScratchArea->PoseHistory)
	{
		if (const FPoseIndicesHistory* PoseIndicesHistory = PoseSearchColumnScratchArea->PoseHistory->GetPoseIndicesHistory())
		{
			// @todo: integrate DeltaTime into SearchContext, and implement it for UAF as well
			float DeltaTime = FiniteDelta;
			if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(GetAnimContext(&Context)))
			{
				DeltaTime = AnimInstance->GetDeltaSeconds();
			}

			// const casting here is safe since we're in the thread owning the pose history, and it's the correct place to update the previously selected poses
			const_cast<FPoseIndicesHistory*>(PoseIndicesHistory)->Update(BestSearchResult, DeltaTime, PoseReselectHistory);
		}
	}

	if (const FChooserParameterFloatBase* StartTime = OutputStartTime.GetPtr<FChooserParameterFloatBase>())
	{
		StartTime->SetValue(Context, StartTimeValue);
	}
		
	if (const FChooserParameterFloatBase* Cost = OutputCost.GetPtr<FChooserParameterFloatBase>())
	{
		Cost->SetValue(Context, CostValue);
	}

	if (const FChooserParameterBoolBase* Mirror = OutputMirror.GetPtr<FChooserParameterBoolBase>())
	{
		Mirror->SetValue(Context, bMirrorValue);
	}

	if (const FChooserParameterBoolBase* ForceBlendTo = OutputForceBlendTo.GetPtr<FChooserParameterBoolBase>())
	{
		ForceBlendTo->SetValue(Context, bForceBlendToValue);
	}
}

int32 FPoseSearchColumn::GetScratchAreaSize() const
{
	return sizeof(UE::PoseSearch::FPoseSearchColumnScratchArea);
}

void FPoseSearchColumn::InitializeScratchArea(TArrayView<uint8> ScratchArea) const
{
	using namespace UE::PoseSearch;

	ensure(ScratchArea.Num() == GetScratchAreaSize());
	FPoseSearchColumnScratchArea* PoseSearchColumnScratchArea = new(ScratchArea.begin()) FPoseSearchColumnScratchArea();
#if DO_CHECK
	PoseSearchColumnScratchArea->DebugOwner = this;
#endif // DO_CHECK
}

void FPoseSearchColumn::DeinitializeScratchArea(TArrayView<uint8> ScratchArea) const
{
	using namespace UE::PoseSearch;

	ensure(ScratchArea.Num() == GetScratchAreaSize());
	FPoseSearchColumnScratchArea* PoseSearchColumnScratchArea = reinterpret_cast<FPoseSearchColumnScratchArea*>(ScratchArea.begin());
#if DO_CHECK
	ensure(PoseSearchColumnScratchArea->DebugOwner == this);
#endif // DO_CHECK
	PoseSearchColumnScratchArea->~FPoseSearchColumnScratchArea();
}

#if WITH_EDITOR
void FPoseSearchColumn::Initialize(UChooserTable* OuterChooser)
{
	using namespace UE::PoseSearch;
	
	ensure(OuterChooser);
	FChooserColumnBase::Initialize(OuterChooser);

	// if Chooser is null we don't reset the rows, since the initialize has been called on a
	// newly copied column via FChooserTableViewModel::CopyColumnInternal
	if (Chooser && Chooser != OuterChooser)
	{
		RowValues.Reset();
		FallbackValue = FPoseSearchColumnRow();
	}

	Chooser = OuterChooser;
	FMapping(Chooser->GetRootChooser()).ApplyMappingToAllColumns();
}

bool FPoseSearchColumn::EditorTestFilter(int32 RowIndex) const
{
	using namespace UE::PoseSearch;

	for (FActiveColumnCost& ActiveColumnCost : ActiveColumnCosts)
	{
		if (ActiveColumnCost.RowIndex == RowIndex)
		{
			return true;
		}
	}
	return false;
}

float FPoseSearchColumn::EditorTestCost(int32 RowIndex) const
{
	using namespace UE::PoseSearch;

	for (FActiveColumnCost& ActiveColumnCost : ActiveColumnCosts)
	{
		if (ActiveColumnCost.RowIndex == RowIndex)
		{
			return ActiveColumnCost.RowCost;
		}
	}
	return 0.f;
}

void FPoseSearchColumn::SetTestValue(TArrayView<const uint8> Value)
{
    FMemoryReaderView Reader(Value);
    Reader << ActiveColumnCosts;
}

void FPoseSearchColumn::AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
{
	using namespace UE::PoseSearch;

	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));

	ensure(Chooser);
	FText DisplayName = NSLOCTEXT("PoseSearchColumn","Pose Search", "Pose Search");
	FName PropertyName("RowData", ColumnIndex);
	FPropertyBagPropertyDesc PropertyDesc(PropertyName,  EPropertyBagPropertyType::Struct, FPoseSearchColumnRowReflection::StaticStruct());
	PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", DisplayName.ToString()));
	PropertyBag.AddProperties({PropertyDesc});

	const FPoseSearchColumnRow& RowValue = GetRowValue(RowIndex);
	if (!ensure(GetReferencedObject(Chooser, RowIndex) == RowValue.Data.AnimAsset))
	{
		UE_LOGF(LogPoseSearch, Error, "FPoseSearchColumn::AddToDetails - Mismatch between chooser table '%ls' with root '%ls' and Database '%ls' for row %d", *Chooser->GetName(), *Chooser->GetRootChooser()->GetName(), *GetNameSafe(RowValue.Database), RowIndex);
	}

	FPoseSearchColumnRowReflection RowReflection;
	static_cast<FPoseSearchColumnRow&>(RowReflection) = RowValue;
	RowReflection.RootChooser = Chooser->GetRootChooser();

	ensure(!RowReflection.Database || RowReflection.Database->GetOuter() == Chooser->GetRootChooser());

	PropertyBag.SetValueStruct(PropertyName, RowReflection);

	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
}

void FPoseSearchColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
{
	using namespace UE::PoseSearch;

	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));

	ensure(Chooser);
	FName PropertyName("RowData", ColumnIndex);
   	TValueOrError<FStructView, EPropertyBagResult> Result = PropertyBag.GetValueStruct(PropertyName, FPoseSearchColumnRowReflection::StaticStruct());
	if (FStructView* StructView = Result.TryGetValue())
	{
		const FPoseSearchColumnRowReflection& RowReflection = StructView->Get<FPoseSearchColumnRowReflection>();
		FPoseSearchColumnRow RowReflectionBase = static_cast<const FPoseSearchColumnRow&>(RowReflection);
		RowReflectionBase.Data.AnimAsset = GetReferencedObject(GetChooser(), RowIndex);

		FPoseSearchColumnRow& RowValue = EditRowValue(RowIndex);

		ensure(!RowReflectionBase.Database || RowReflectionBase.Database->GetOuter() == Chooser->GetRootChooser());

		if (RowValue != RowReflectionBase)
		{
			Chooser->Modify();
			RowValue = RowReflectionBase;
			FMapping(Chooser->GetRootChooser()).ApplyMappingToAllColumns();
			Chooser->PostEditChange();
		}
	}

	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
}

void FPoseSearchColumn::ReplaceReferences(UObject* ReferenceToReplace, UObject* ReplaceWith)
{
	using namespace UE::PoseSearch;
	ensure(Chooser);

	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));

	TArray<UChooserTable*, TInlineAllocator<2>> ModifiedChoosers;

	if (ReferenceToReplace && ReferenceToReplace != ReplaceWith)
	{
		if (UPoseSearchDatabase* DatabaseToReplace = Cast<UPoseSearchDatabase>(ReferenceToReplace))
		{
			UPoseSearchDatabase* DatabaseReplaceWith = Cast<UPoseSearchDatabase>(ReplaceWith);
			if (ReplaceWith && !DatabaseReplaceWith)
			{
				UE_LOGF(LogPoseSearch, Error, "FPoseSearchColumn::ReplaceReferences, mismatch between ReferenceToReplace and ReplaceWith");
			}
			else
			{
				ensure(!DatabaseReplaceWith || DatabaseReplaceWith->GetOuter() == Chooser->GetRootChooser());

				for (FPoseSearchColumnRow& RowValue : RowValues)
				{
					if (RowValue.Database == DatabaseToReplace)
					{
						if (!ModifiedChoosers.Contains(Chooser))
						{
							ModifiedChoosers.Add(Chooser);
							Chooser->Modify();
						}
						RowValue.Database = DatabaseReplaceWith;
					}
				}

				if (FallbackValue.Database == DatabaseToReplace)
				{
					if (!ModifiedChoosers.Contains(Chooser))
					{
						ModifiedChoosers.Add(Chooser);
						Chooser->Modify();
					}
					FallbackValue.Database = DatabaseReplaceWith;
				}
			}
		}
		else if (UChooserTable* ChooserToReplace = Cast<UChooserTable>(ReferenceToReplace))
		{
			UChooserTable* ChooserToReplaceWith = Cast<UChooserTable>(ReplaceWith);
			if (ReplaceWith && !ChooserToReplaceWith)
			{
				UE_LOGF(LogPoseSearch, Error, "FPoseSearchColumn::ReplaceReferences, mismatch between ReferenceToReplace and ReplaceWith");
			}
			else if (ChooserToReplace == Chooser)
			{
				if (!ModifiedChoosers.Contains(Chooser))
				{
					ModifiedChoosers.Add(Chooser);
					Chooser->Modify();
				}
				if (!ModifiedChoosers.Contains(ChooserToReplaceWith))
				{
					ModifiedChoosers.Add(ChooserToReplaceWith);
					ChooserToReplaceWith->Modify();
				}
				Chooser = ChooserToReplaceWith;
			}
		}
	}

	for (UChooserTable* ModifiedChooser : ModifiedChoosers)
	{
		FMapping(ModifiedChooser->GetRootChooser()).ApplyMappingToAllColumns();
		ModifiedChooser->PostEditChange();
	}

	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
}

FName FPoseSearchColumn::RowValuesPropertyName()
{
	return "RowValues";
}

void FPoseSearchColumn::SetNumRows(int32 NumRows)
{
	using namespace UE::PoseSearch;
	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));

	if (NumRows <= RowValues.Num())
	{
		RowValues.SetNum(NumRows);
	}
	else
	{
		while (RowValues.Num() < NumRows)
		{
			RowValues.Add(FPoseSearchColumnRow());
		}
	}
	FMapping(Chooser->GetRootChooser()).ApplyMappingToAllColumns();

	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
}

void FPoseSearchColumn::InsertRows(int Index, int Count)
{
	using namespace UE::PoseSearch;
	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));

	for (int i=0;i<Count;i++)
	{
		RowValues.Insert(FPoseSearchColumnRow(), Index);
	}
	FMapping(Chooser->GetRootChooser()).ApplyMappingToAllColumns();

	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
}

void FPoseSearchColumn::DeleteRows(TArrayView<int> RowIndices)
{
	using namespace UE::PoseSearch;
	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));

	for(int32 Index : RowIndices)
	{
		if (RowValues.IsValidIndex(Index))
		{
			RowValues.RemoveAt(Index);
		}
	}
	FMapping(Chooser->GetRootChooser()).ApplyMappingToAllColumns();

	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
}

void FPoseSearchColumn::MoveRow(int SourceRowIndex, int TargetRowIndex)
{
	using namespace UE::PoseSearch;
	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));

	FPoseSearchColumnRow RowData = RowValues[SourceRowIndex];
    RowValues.RemoveAt(SourceRowIndex);
    if (SourceRowIndex < TargetRowIndex)
	{
		TargetRowIndex--;
	}
    RowValues.Insert(RowData, TargetRowIndex);
	FMapping(Chooser->GetRootChooser()).ApplyMappingToAllColumns();

	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
}

void FPoseSearchColumn::CopyRow(FChooserColumnBase& SourceColumn, int SourceRowIndex, int TargetRowIndex)
{
	using namespace UE::PoseSearch;

	const FPoseSearchColumn& SourcePoseSearchColumn = static_cast<FPoseSearchColumn&>(SourceColumn);
	ensure(Chooser && SourcePoseSearchColumn.Chooser);
	UChooserTable* RootChooser = Chooser->GetRootChooser();
	ensure(RootChooser);

	if (&SourcePoseSearchColumn == this && SourceRowIndex == TargetRowIndex)
	{
		DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
		return;
	}
	
	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
	DO_ENSURE_CONSISTENCY(CheckConsistency(SourcePoseSearchColumn.GetChooser()));

	const bool bSameRootChooser = RootChooser == SourcePoseSearchColumn.Chooser->GetRootChooser();
	const FPoseSearchColumnRow& SourceRow = SourcePoseSearchColumn.GetRowValue(SourceRowIndex);
	if (SourceRow.Database)
	{
		ensure(SourceRow.Database->GetOuter() == SourcePoseSearchColumn.Chooser->GetRootChooser());
		SourceRow.Database->Modify();
	}

	FPoseSearchColumnRow& TargetRow = EditRowValue(TargetRowIndex);
	TargetRow = SourceRow;
	// resynchronizing the TargetRow.RowValue.Data with this column Chooser reference object associated with the RowIndex
	TargetRow.Data.AnimAsset = GetReferencedObject(Chooser, TargetRowIndex);

	bool bPerformDatabasePostLoad = false;
	bool bPerformPostEditChange = false;
	UPoseSearchDatabase* NewTargetDatabase = nullptr;
	if (!bSameRootChooser && SourceRow.Database)
	{
		FString SourceDatabasePlainName = SourceRow.Database->GetFName().GetPlainNameString();
		for (UObject* NestedObject : RootChooser->NestedObjects)
		{
			if (UPoseSearchDatabase* Database = Cast<UPoseSearchDatabase>(NestedObject))
			{
				if (SourceRow.Database->Schema == Database->Schema)
				{
					FString DatabasePlainName = Database->GetFName().GetPlainNameString();
					if (SourceDatabasePlainName == DatabasePlainName)
					{
						// we found a good candidate for NewTargetDatabase. Let's use it instead of creating a new  UPoseSearchDatabase
						NewTargetDatabase = Database;
						bPerformPostEditChange = true;
						NewTargetDatabase->Modify();
						break;
					}
				}
			}
		}

		if (!NewTargetDatabase)
		{
			const FName NewTargetDatabaseUniqueName = MakeUniqueObjectName(RootChooser, UPoseSearchDatabase::StaticClass(), SourceRow.Database->GetFName());
			FObjectDuplicationParameters Parameters = InitStaticDuplicateObjectParams(SourceRow.Database, RootChooser, NewTargetDatabaseUniqueName);
			
			// delaying UPoseSearchDatabase PostLoad (starting database indexing) until we set all the Chooser properties and ApplyMappingToAllColumns
			Parameters.bSkipPostLoad = true;
			bPerformDatabasePostLoad = true;
			NewTargetDatabase = static_cast<UPoseSearchDatabase*>(StaticDuplicateObjectEx(Parameters));

			RootChooser->Modify();
			RootChooser->AddNestedObject(NewTargetDatabase);
			RootChooser->PostEditChange();
		}

		RowValues[TargetRowIndex].Database = NewTargetDatabase;
	}

	ensure(!GetRowValue(TargetRowIndex).Database || GetRowValue(TargetRowIndex).Database->GetOuter() == Chooser->GetRootChooser());

	FMapping(Chooser->GetRootChooser()).ApplyMappingToAllColumns();

	if (bPerformDatabasePostLoad)
	{
		check(NewTargetDatabase);
		NewTargetDatabase->ConditionalPostLoad();
	}

	if (bPerformPostEditChange)
	{
		check(NewTargetDatabase);
		NewTargetDatabase->PostEditChange();
	}

	if (SourceRow.Database)
	{
		SourceRow.Database->PostEditChange();
	}

	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
	DO_ENSURE_CONSISTENCY(CheckConsistency(SourcePoseSearchColumn.GetChooser()));
}

void FPoseSearchColumn::CopyFallback(FChooserColumnBase& SourceColumn)
{
	CopyRow(SourceColumn, ChooserColumn_SpecialIndex_Fallback, ChooserColumn_SpecialIndex_Fallback);
}

void FPoseSearchColumn::AutoPopulate(int32 RowIndex, UObject* OutputObject)
{
	using namespace UE::PoseSearch;
	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
	FPoseSearchColumnRow& RowValue = EditRowValue(RowIndex);
	if (OutputObject != RowValue.Data.AnimAsset)
	{
		RowValue.Data.AnimAsset = OutputObject;
	}
	DO_ENSURE_CONSISTENCY(CheckConsistency(Chooser));
}

UScriptStruct* FPoseSearchColumn::GetInputBaseType() const
{
	return FPoseHistoryContextProperty::StaticStruct();
}

const UScriptStruct* FPoseSearchColumn::GetInputType() const
{
	return InputValue.IsValid() ? InputValue.GetScriptStruct() : nullptr;
}

FInstancedStruct* FPoseSearchColumn::GetInputValuePtr()
{
	return &InputValue;
}

void FPoseSearchColumn::SetInputType(const UScriptStruct* Type)
{
	InputValue.InitializeAs(Type);
}

#endif // WITH_EDITOR

FChooserParameterBase* FPoseSearchColumn::GetInputValue()
{
	return InputValue.GetMutablePtr<FChooserParameterBase>();
}

#undef LOCTEXT_NAMESPACE
