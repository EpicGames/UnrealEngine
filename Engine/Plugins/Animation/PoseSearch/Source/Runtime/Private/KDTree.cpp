// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/KDTree.h"
#include "HAL/IConsoleManager.h"
#include "Stats/Stats.h"

#ifndef UE_POSE_SEARCH_USE_NANOFLANN
	#define UE_POSE_SEARCH_USE_NANOFLANN 1
#endif

// @third party code - BEGIN nanoflann
#if UE_POSE_SEARCH_USE_NANOFLANN
THIRD_PARTY_INCLUDES_START
#include "nanoflann/nanoflann.hpp"
THIRD_PARTY_INCLUDES_END
#endif
// @third party code - END nanoflann

namespace UE::PoseSearch
{

#if UE_POSE_SEARCH_USE_NANOFLANN

#if UE_POSE_SEARCH_ENABLE_ITERATIVE_KDTREE_CONSTRUCTION_VALIDATION
static bool GVarMotionMatchTestIterativeKDTreeConstruction = false;
static FAutoConsoleVariableRef CVarMotionMatchTestIterativeKDTreeConstruction(TEXT("a.MotionMatch.Test.IterativeKDTreeConstruction"), GVarMotionMatchTestIterativeKDTreeConstruction, TEXT("Test new iterative PoseSearchDatabase kdtree construction against the previous recursive method"));
#endif // UE_POSE_SEARCH_ENABLE_ITERATIVE_KDTREE_CONSTRUCTION_VALIDATION

using FKDTreeL2SimpleAdaptor = nanoflann::L2_Simple_Adaptor<float, FKDTree::FDataSource>;
using FKDTreeImplementationBase = nanoflann::KDTreeSingleIndexAdaptor<FKDTreeL2SimpleAdaptor, FKDTree::FDataSource, -1, AccessorType>;
using FKDTreeKDTreeBaseClass = nanoflann::KDTreeBaseClass<FKDTreeImplementationBase, FKDTreeL2SimpleAdaptor, FKDTree::FDataSource, -1, AccessorType>;

struct FKDTreeImplementation : FKDTreeImplementationBase
{
	using FKDTreeImplementationBase::FKDTreeImplementationBase;

	bool operator==(const FKDTreeImplementation& Other) const
	{
		if (m_size != Other.m_size)
		{
			return false;
		}

		if (dim != Other.dim)
		{
			return false;
		}

		const AccessorType RootBBoxSize = root_bbox.size();
		if (RootBBoxSize != Other.root_bbox.size())
		{
			return false;
		}

		for (AccessorType Index = 0; Index < RootBBoxSize; ++Index)
		{
			const Interval& ThisInterval = root_bbox[Index];
			const Interval& OtherInterval = Other.root_bbox[Index];

			if (ThisInterval.high != OtherInterval.high)
			{
				return false;
			}

			if (ThisInterval.low != OtherInterval.low)
			{
				return false;
			}
		}

		if (m_leaf_max_size != Other.m_leaf_max_size)
		{
			return false;
		}

		if (vAcc != Other.vAcc)
		{
			return false;
		}

		if (!CompareNodes(root_node, Other.root_node))
		{
			return false;
		}

		return true;
	}

private:
	static bool CompareNodes(const NodePtr& NodeA, const NodePtr& NodeB)
	{
		const bool bAnyNodeAChild1 = NodeA->child1 != nullptr;
		const bool bAnyNodeBChild1 = NodeB->child1 != nullptr;
		if (bAnyNodeAChild1 != bAnyNodeBChild1)
		{
			return false;
		}

		const bool bAnyNodeAChild2 = NodeA->child2 != nullptr;
		const bool bAnyNodeBChild2 = NodeB->child2 != nullptr;
		if (bAnyNodeAChild2 != bAnyNodeBChild2)
		{
			return false;
		}

		const bool bIsLeafNode = !bAnyNodeAChild1 && !bAnyNodeAChild2;
		if (bIsLeafNode)
		{
			if (NodeA->node_type.lr.left != NodeB->node_type.lr.left)
			{
				return false;
			}

			if (NodeA->node_type.lr.right != NodeB->node_type.lr.right)
			{
				return false;
			}
		}
		else
		{
			if (NodeA->node_type.sub.divfeat != NodeB->node_type.sub.divfeat)
			{
				return false;
			}

			if (NodeA->node_type.sub.divhigh != NodeB->node_type.sub.divhigh)
			{
				return false;
			}

			if (NodeA->node_type.sub.divlow != NodeB->node_type.sub.divlow)
			{
				return false;
			}
		}

		if (bAnyNodeAChild1)
		{
			if (!CompareNodes(NodeA->child1, NodeB->child1))
			{
				return false;
			}
		}

		if (bAnyNodeAChild2)
		{
			if (!CompareNodes(NodeA->child2, NodeB->child2))
			{
				return false;
			}
		}

		return true;
	}
};
#endif

FKDTree::FKDTree(AccessorType Count, AccessorType Dim, const float* Data, AccessorType MaxLeafSize
#if UE_POSE_SEARCH_ENABLE_ITERATIVE_KDTREE_CONSTRUCTION_VALIDATION
		, bool bUseRecursiveDivideTree
#endif // UE_POSE_SEARCH_ENABLE_ITERATIVE_KDTREE_CONSTRUCTION_VALIDATION
	)
	: DataSource(Count, Dim, Data)
	, KDTreeImplementation(nullptr)
{
#if UE_POSE_SEARCH_USE_NANOFLANN
	if (Count > 0 && Dim > 0 && Data)
	{
#if UE_POSE_SEARCH_ENABLE_ITERATIVE_KDTREE_CONSTRUCTION_VALIDATION
		DataSource.bUseRecursiveDivideTree = bUseRecursiveDivideTree;
#endif // UE_POSE_SEARCH_ENABLE_ITERATIVE_KDTREE_CONSTRUCTION_VALIDATION

		KDTreeImplementation = new FKDTreeImplementation(Dim, DataSource, nanoflann::KDTreeSingleIndexAdaptorParams(MaxLeafSize));
	}
#endif // UE_POSE_SEARCH_USE_NANOFLANN
}

FKDTree::FKDTree()
: DataSource(0, 0, nullptr)
, KDTreeImplementation(nullptr)
{
}

FKDTree::~FKDTree()
{
	Reset();
}

#if UE_POSE_SEARCH_USE_NANOFLANN
void CopySubTree(FKDTree& KDTree, FKDTreeImplementation::NodePtr& ThisNode, const FKDTreeImplementation::NodePtr& OtherNode)
{
	check(KDTree.KDTreeImplementation);

	ThisNode = KDTree.KDTreeImplementation->pool.template allocate<FKDTreeImplementation::Node>();
	
	ThisNode->node_type = OtherNode->node_type;

	if (OtherNode->child1 != nullptr)
	{
		CopySubTree(KDTree, ThisNode->child1, OtherNode->child1);
	}
	else
	{
		ThisNode->child1 = nullptr;
	}

	if (OtherNode->child2 != nullptr)
	{
		CopySubTree(KDTree, ThisNode->child2, OtherNode->child2);
	}
	else
	{
		ThisNode->child2 = nullptr;
	}
}

#endif // UE_POSE_SEARCH_USE_NANOFLANN

FKDTree::FKDTree(const FKDTree& Other)
	: DataSource()
	, KDTreeImplementation(nullptr)
{
#if UE_POSE_SEARCH_USE_NANOFLANN
	if (this != &Other && Other.KDTreeImplementation)
	{
		KDTreeImplementation = new FKDTreeImplementation(0, DataSource, nanoflann::KDTreeSingleIndexAdaptorParams(0));

		DataSource = Other.DataSource;

		check(Other.KDTreeImplementation->m_size <= AccessorTypeMax);
		KDTreeImplementation->m_size = Other.KDTreeImplementation->m_size;

		if (KDTreeImplementation->m_size > 0)
		{
			KDTreeImplementation->dim = Other.KDTreeImplementation->dim;

			check(Other.KDTreeImplementation->root_bbox.size() <= AccessorTypeMax);
			const AccessorType root_bbox_size = Other.KDTreeImplementation->root_bbox.size();
			KDTreeImplementation->root_bbox.resize(root_bbox_size);

			for (AccessorType i = 0; i < root_bbox_size; ++i)
			{
				KDTreeImplementation->root_bbox[i] = Other.KDTreeImplementation->root_bbox[i];
			}

			check(Other.KDTreeImplementation->m_leaf_max_size <= AccessorTypeMax);
			const AccessorType KDTreeLeafMaxSize = Other.KDTreeImplementation->m_leaf_max_size;
			KDTreeImplementation->m_leaf_max_size = KDTreeLeafMaxSize;

			check(Other.KDTreeImplementation->vAcc.size() <= AccessorTypeMax);
			const AccessorType VAccSize = Other.KDTreeImplementation->vAcc.size();
			KDTreeImplementation->vAcc.resize(VAccSize);

			for (AccessorType i = 0; i < VAccSize; ++i)
			{
				KDTreeImplementation->vAcc[i] = Other.KDTreeImplementation->vAcc[i];
			}

			CopySubTree(*this, KDTreeImplementation->root_node, Other.KDTreeImplementation->root_node);
		}
	}
#endif // UE_POSE_SEARCH_USE_NANOFLANN
}

FKDTree::FKDTree(FKDTree&& Other)
	: DataSource()
	, KDTreeImplementation(nullptr)
{
#if UE_POSE_SEARCH_USE_NANOFLANN
	if (this != &Other)
	{
		if (Other.KDTreeImplementation)
		{
			KDTreeImplementation = new FKDTreeImplementation(0, DataSource, nanoflann::KDTreeSingleIndexAdaptorParams(0));

			DataSource = MoveTemp(Other.DataSource);

			check(Other.KDTreeImplementation->m_size <= AccessorTypeMax);
			KDTreeImplementation->m_size = Other.KDTreeImplementation->m_size;

			if (KDTreeImplementation->m_size > 0)
			{
				KDTreeImplementation->dim = Other.KDTreeImplementation->dim;

				check(Other.KDTreeImplementation->root_bbox.size() <= AccessorTypeMax);
				const AccessorType root_bbox_size = Other.KDTreeImplementation->root_bbox.size();
				KDTreeImplementation->root_bbox.resize(root_bbox_size);

				for (AccessorType i = 0; i < root_bbox_size; ++i)
				{
					KDTreeImplementation->root_bbox[i] = Other.KDTreeImplementation->root_bbox[i];
				}

				check(Other.KDTreeImplementation->m_leaf_max_size <= AccessorTypeMax);
				const AccessorType KDTreeLeafMaxSize = Other.KDTreeImplementation->m_leaf_max_size;
				KDTreeImplementation->m_leaf_max_size = KDTreeLeafMaxSize;

				check(Other.KDTreeImplementation->vAcc.size() <= AccessorTypeMax);
				const AccessorType VAccSize = Other.KDTreeImplementation->vAcc.size();
				KDTreeImplementation->vAcc.resize(VAccSize);

				for (AccessorType i = 0; i < VAccSize; ++i)
				{
					KDTreeImplementation->vAcc[i] = Other.KDTreeImplementation->vAcc[i];
				}

				CopySubTree(*this, KDTreeImplementation->root_node, Other.KDTreeImplementation->root_node);
			}
		}

		Other.Reset();
	}
#endif // UE_POSE_SEARCH_USE_NANOFLANN
}

FKDTree& FKDTree::operator=(const FKDTree& Other)
{
	if (this != &Other)
	{
		this->~FKDTree();
		new(this) FKDTree(Other);
	}
	return *this;
}

FKDTree& FKDTree::operator=(FKDTree&& Other)
{
	if (this != &Other)
	{
		this->~FKDTree();
		new(this) FKDTree(MoveTemp(Other));
	}
	return *this;
}

bool FKDTree::operator==(const FKDTree& Other) const
{
	const bool bAnyImpl = KDTreeImplementation != nullptr;
	const bool bAnyOtherImpl = Other.KDTreeImplementation != nullptr;
	if (bAnyImpl != bAnyOtherImpl)
	{
		return false;
	}

#if UE_POSE_SEARCH_USE_NANOFLANN
	if (bAnyImpl && bAnyOtherImpl)
	{
		if (*KDTreeImplementation != *Other.KDTreeImplementation)
		{
			return false;
		}
	}
#endif // UE_POSE_SEARCH_USE_NANOFLANN

	if (DataSource != Other.DataSource)
	{
		return false;
	}

	return true;
}

void FKDTree::Reset()
{
#if UE_POSE_SEARCH_USE_NANOFLANN
	delete KDTreeImplementation;
	KDTreeImplementation = nullptr;
#endif // UE_POSE_SEARCH_USE_NANOFLANN
	DataSource = FDataSource();
}

void FKDTree::Construct(AccessorType Count, AccessorType Dim, const float* Data, AccessorType MaxLeafSize)
{
	this->~FKDTree();
	new(this) FKDTree(Count, Dim, Data, MaxLeafSize);

#if UE_POSE_SEARCH_ENABLE_ITERATIVE_KDTREE_CONSTRUCTION_VALIDATION
	if (GVarMotionMatchTestIterativeKDTreeConstruction)
	{
		const FKDTree ConstructedRecursively(Count, Dim, Data, MaxLeafSize, true);
		ensure(*this == ConstructedRecursively);
	}
#endif // UE_POSE_SEARCH_ENABLE_ITERATIVE_KDTREE_CONSTRUCTION_VALIDATION
}

template <typename RESULTSET>
inline int32 FindNeighborsInternal(FKDTreeImplementation* KDTreeImplementation, RESULTSET& Result, TConstArrayView<float> Query)
{
#if UE_POSE_SEARCH_USE_NANOFLANN

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FKDTree_FindNeighbors);

	if (KDTreeImplementation)
	{
		check(Query.GetData() && Query.Num() == KDTreeImplementation->dim && KDTreeImplementation->root_node);

		const nanoflann::SearchParams SearchParams(
			32,			// Ignored parameter (Kept for compatibility with the FLANN interface).
			0.f,		// search for eps-approximate neighbours (default: 0)
			false);		// only for radius search, require neighbours sorted by
		KDTreeImplementation->findNeighbors(Result, Query.GetData(), SearchParams);
		return Result.Num();
	}
	return 0;

#else // UE_POSE_SEARCH_USE_NANOFLANN

	checkNoEntry(); // unimplemented
	return 0;

#endif // UE_POSE_SEARCH_USE_NANOFLANN
}

int32 FKDTree::FindNeighbors(FKNNResultSet& Result, TConstArrayView<float> Query) const
{
	return FindNeighborsInternal(KDTreeImplementation, Result, Query);
}

int32 FKDTree::FindNeighbors(FFilteredKNNResultSet& Result, TConstArrayView<float> Query) const
{
	return FindNeighborsInternal(KDTreeImplementation, Result, Query);
}

int32 FKDTree::FindNeighbors(FRadiusResultSet& Result, TConstArrayView<float> Query) const
{
	return FindNeighborsInternal(KDTreeImplementation, Result, Query);
}

int32 FKDTree::FindNeighbors(FKNNMaxHeapResultSet& Result, TConstArrayView<float> Query) const
{
	return FindNeighborsInternal(KDTreeImplementation, Result, Query);
}

int32 FKDTree::FindNeighbors(FFilteredKNNMaxHeapResultSet& Result, TConstArrayView<float> Query) const
{
	return FindNeighborsInternal(KDTreeImplementation, Result, Query);
}

int32 FKDTree::FindNeighbors(FRadiusMaxHeapResultSet& Result, TConstArrayView<float> Query) const
{
	return FindNeighborsInternal(KDTreeImplementation, Result, Query);
}

SIZE_T FKDTree::GetAllocatedSize() const
{
	SIZE_T AllocatedSize = sizeof(FKDTree);

#if UE_POSE_SEARCH_USE_NANOFLANN
	if (KDTreeImplementation)
	{
		AllocatedSize += sizeof(FKDTreeImplementation);
		AllocatedSize += KDTreeImplementation->usedMemory(*KDTreeImplementation);
	}
#endif // UE_POSE_SEARCH_USE_NANOFLANN

	return AllocatedSize;
}

#if UE_POSE_SEARCH_USE_NANOFLANN
FArchive& SerializeSubTree(FArchive& Ar, FKDTree& KDTree, FKDTreeImplementation::NodePtr& KDTreeNode)
{
	check(KDTree.KDTreeImplementation);

	if (Ar.IsLoading())
	{
		KDTreeNode = KDTree.KDTreeImplementation->pool.template allocate<FKDTreeImplementation::Node>();
		// zeroing FKDTreeImplementation::Node memory since it contains a union and doesn't have a constructor 
		FMemory::Memzero(KDTreeNode, sizeof(FKDTreeImplementation::Node));
	}

	bool bAnyNodeChild1 = KDTreeNode->child1 != nullptr;
	bool bAnyNodeChild2 = KDTreeNode->child2 != nullptr;
	Ar << bAnyNodeChild1;
	Ar << bAnyNodeChild2;

	const bool bIsLeafNode = !bAnyNodeChild1 && !bAnyNodeChild2;
	if (bIsLeafNode)
	{
		check(KDTreeNode->node_type.lr.left <= AccessorTypeMax);
		check(KDTreeNode->node_type.lr.right <= AccessorTypeMax);

		AccessorType OffsetLeft = KDTreeNode->node_type.lr.left;
		AccessorType OffsetRight = KDTreeNode->node_type.lr.right;

		Ar << OffsetLeft;
		Ar << OffsetRight;

		KDTreeNode->node_type.lr.left = OffsetLeft;
		KDTreeNode->node_type.lr.right = OffsetRight;
	}
	else
	{
		Ar << KDTreeNode->node_type.sub.divfeat;
		Ar << KDTreeNode->node_type.sub.divhigh;
		Ar << KDTreeNode->node_type.sub.divlow;
	}

	if (bAnyNodeChild1)
	{
		SerializeSubTree(Ar, KDTree, KDTreeNode->child1);
	}
	else if (Ar.IsLoading())
	{
		KDTreeNode->child1 = nullptr;
	}

	if (bAnyNodeChild2)
	{
		SerializeSubTree(Ar, KDTree, KDTreeNode->child2);
	}
	else if (Ar.IsLoading())
	{
		KDTreeNode->child2 = nullptr;
	}
	return Ar;
}

#endif // UE_POSE_SEARCH_USE_NANOFLANN

FArchive& Serialize(FArchive& Ar, FKDTree& KDTree, const float* KDTreeData)
{
#if UE_POSE_SEARCH_USE_NANOFLANN
	check(!KDTree.KDTreeImplementation || KDTree.KDTreeImplementation->m_size <= AccessorTypeMax);

	AccessorType KDTreeSize = KDTree.KDTreeImplementation ? KDTree.KDTreeImplementation->m_size : 0;

	Ar << KDTreeSize;

	if (KDTreeSize > 0)
	{
		if (Ar.IsLoading() && !KDTree.KDTreeImplementation)
		{
			KDTree.KDTreeImplementation = new FKDTreeImplementation(0, KDTree.DataSource, nanoflann::KDTreeSingleIndexAdaptorParams(0));
		}

		KDTree.KDTreeImplementation->m_size = KDTreeSize;

		Ar << KDTree.KDTreeImplementation->dim;

		AccessorType root_bbox_size = KDTree.KDTreeImplementation->root_bbox.size();
		check(KDTree.KDTreeImplementation->root_bbox.size() <= AccessorTypeMax);
		Ar << root_bbox_size;

		if (Ar.IsLoading())
		{
			KDTree.DataSource.Data = KDTreeData;
			KDTree.DataSource.PointDim = KDTree.KDTreeImplementation->dim;
			KDTree.DataSource.PointCount = KDTree.KDTreeImplementation->m_size;

			KDTree.KDTreeImplementation->root_bbox.resize(root_bbox_size);
		}

		for (FKDTreeImplementation::Interval& el : KDTree.KDTreeImplementation->root_bbox)
		{
			Ar.Serialize(&el, sizeof(FKDTreeImplementation::Interval));
		}

		check(KDTree.KDTreeImplementation->m_leaf_max_size <= AccessorTypeMax);
		AccessorType KDTreeLeafMaxSize = KDTree.KDTreeImplementation->m_leaf_max_size;
		Ar << KDTreeLeafMaxSize;
		KDTree.KDTreeImplementation->m_leaf_max_size = KDTreeLeafMaxSize;

		check(KDTree.KDTreeImplementation->vAcc.size() <= AccessorTypeMax);
		AccessorType VAccSize = KDTree.KDTreeImplementation->vAcc.size();
		Ar << VAccSize;
		if (Ar.IsLoading())
		{
			KDTree.KDTreeImplementation->vAcc.resize(VAccSize);
		}
		for (AccessorType& el : KDTree.KDTreeImplementation->vAcc)
		{
			Ar << el;
		}
		SerializeSubTree(Ar, KDTree, KDTree.KDTreeImplementation->root_node);
	}
	else if (Ar.IsLoading())
	{
		KDTree.Reset();
	}
#endif // UE_POSE_SEARCH_USE_NANOFLANN

	return Ar;
}

} // namespace UE::PoseSearch

#if UE_POSE_SEARCH_USE_NANOFLANN

#include <deque>

namespace nanoflann
{

// Fully resolved explicit specialization of KDTreeBaseClass::divideTree.
// The generic implementation is recursive and can overflow the stack when many
// duplicate data points force a degenerate tree of depth O(N).  This iterative
// replacement simulates the same post-order traversal on an explicit heap stack.
//
// Implementation notes
// --------------------
// The recursion has post-order work: after both children return, the parent reads
// their updated bbox values to compute divlow/divhigh and its own bbox.  We model
// this with two frame kinds pushed onto a std::deque used as a LIFO stack:
//
//   Build  - enter a sub-range; make a leaf or split and enqueue children.
//   Merge  - post-order finalization; runs after both children have written
//            their actual bboxes into this frame's left_bbox / right_bbox fields.
//
// std::deque is chosen over std::vector because deque::push_back does NOT
// invalidate references to existing elements.  Child Build frames store raw
// BoundingBox* pointers into their ancestor Merge frame; those pointers must
// remain valid across all subsequent push_backs that enqueue grandchildren.
template <>
UE::PoseSearch::FKDTreeKDTreeBaseClass::NodePtr
UE::PoseSearch::FKDTreeKDTreeBaseClass::divideTree(
	UE::PoseSearch::FKDTreeImplementationBase& obj,
	const Offset left,
	const Offset right,
	BoundingBox& bbox)
{
#if UE_POSE_SEARCH_ENABLE_ITERATIVE_KDTREE_CONSTRUCTION_VALIDATION
	if (obj.dataset.bUseRecursiveDivideTree)
	{
		NodePtr node = obj.pool.template allocate<Node>();

		// DIM is -1 for this specialization, so obj.dim is always used
		if ((right - left) <= static_cast<Offset>(obj.m_leaf_max_size))
		{
			node->child1 = node->child2 = nullptr;
			node->node_type.lr.left = left;
			node->node_type.lr.right = right;

			for (Dimension i = 0; i < obj.dim; ++i)
			{
				bbox[i].low = dataset_get(obj, obj.vAcc[left], i);
				bbox[i].high = dataset_get(obj, obj.vAcc[left], i);
			}
			for (Offset k = left + 1; k < right; ++k)
			{
				for (Dimension i = 0; i < obj.dim; ++i)
				{
					if (bbox[i].low > dataset_get(obj, obj.vAcc[k], i)) bbox[i].low = dataset_get(obj, obj.vAcc[k], i);
					if (bbox[i].high < dataset_get(obj, obj.vAcc[k], i)) bbox[i].high = dataset_get(obj, obj.vAcc[k], i);
				}
			}
		}
		else
		{
			Offset       idx;
			Dimension    cutfeat;
			DistanceType cutval;
			middleSplit_(obj, left, right - left, idx, cutfeat, cutval, bbox);

			node->node_type.sub.divfeat = cutfeat;

			BoundingBox left_bbox(bbox);
			left_bbox[cutfeat].high = cutval;
			node->child1 = divideTree(obj, left, left + idx, left_bbox);

			BoundingBox right_bbox(bbox);
			right_bbox[cutfeat].low = cutval;
			node->child2 = divideTree(obj, left + idx, right, right_bbox);

			node->node_type.sub.divlow = left_bbox[cutfeat].high;
			node->node_type.sub.divhigh = right_bbox[cutfeat].low;

			for (Dimension i = 0; i < obj.dim; ++i)
			{
				bbox[i].low = std::min(left_bbox[i].low, right_bbox[i].low);
				bbox[i].high = std::max(left_bbox[i].high, right_bbox[i].high);
			}
		}

		return node;
	}
#endif // UE_POSE_SEARCH_ENABLE_ITERATIVE_KDTREE_CONSTRUCTION_VALIDATION

	struct Frame
	{
		enum class Kind : uint8_t { Build, Merge } kind;

		// ---- Build fields ----
		Offset       left    = 0;
		Offset       right   = 0;
		NodePtr*     result  = nullptr;  // write the NodePtr for this subtree here
		BoundingBox  in_bbox;            // working/input bbox consumed by middleSplit_
		BoundingBox* out_bbox = nullptr; // write the actual subtree bbox here when done

		// ---- Merge fields ----
		NodePtr      node      = nullptr;
		Dimension    cutfeat   = 0;
		BoundingBox  left_bbox;   // left  child writes its actual bbox here
		BoundingBox  right_bbox;  // right child writes its actual bbox here
		BoundingBox* merge_out = nullptr; // propagate merged bbox to grandparent
	};

	NodePtr root_result = nullptr;
	std::deque<Frame> stack;

	// Seed the stack with the root Build frame.
	{
		Frame f;
		f.kind     = Frame::Kind::Build;
		f.left     = left;
		f.right    = right;
		f.result   = &root_result;
		f.in_bbox  = bbox;   // copy: middleSplit_ may mutate it
		f.out_bbox = &bbox;  // the function's bbox& parameter receives the root result
		stack.push_back(std::move(f));
	}

	while (!stack.empty())
	{
		// Move the top frame out; pop_back then destroys the now-empty deque slot.
		Frame frame = std::move(stack.back());
		stack.pop_back();

		if (frame.kind == Frame::Kind::Build)
		{
			NodePtr node = obj.pool.template allocate<Node>();

			if ((frame.right - frame.left) <= static_cast<Offset>(obj.m_leaf_max_size))
			{
				// ---- Leaf ----
				node->child1 = node->child2 = nullptr;
				node->node_type.lr.left  = frame.left;
				node->node_type.lr.right = frame.right;

				// DIM == -1 for this specialization, so obj.dim is always used.
				for (Dimension i = 0; i < obj.dim; ++i)
				{
					frame.in_bbox[i].low  = dataset_get(obj, obj.vAcc[frame.left], i);
					frame.in_bbox[i].high = dataset_get(obj, obj.vAcc[frame.left], i);
				}
				for (Offset k = frame.left + 1; k < frame.right; ++k)
				{
					for (Dimension i = 0; i < obj.dim; ++i)
					{
						const ElementType val = dataset_get(obj, obj.vAcc[k], i);
						if (frame.in_bbox[i].low  > val) frame.in_bbox[i].low  = val;
						if (frame.in_bbox[i].high < val) frame.in_bbox[i].high = val;
					}
				}

				*frame.result  = node;
				*frame.out_bbox = frame.in_bbox;
			}
			else
			{
				// ---- Non-leaf: split, then push Merge + two child Build frames ----
				Offset       idx;
				Dimension    cutfeat;
				DistanceType cutval;
				middleSplit_(obj, frame.left, frame.right - frame.left, idx, cutfeat, cutval, frame.in_bbox);

				node->node_type.sub.divfeat = cutfeat;
				*frame.result = node;

				// Push the Merge frame first so it sits BELOW the child Build frames
				// and is therefore processed AFTER both children have completed.
				{
					Frame mf;
					mf.kind       = Frame::Kind::Merge;
					mf.node       = node;
					mf.cutfeat    = cutfeat;
					mf.left_bbox  = frame.in_bbox; mf.left_bbox[cutfeat].high  = cutval;
					mf.right_bbox = frame.in_bbox; mf.right_bbox[cutfeat].low  = cutval;
					mf.merge_out  = frame.out_bbox;
					stack.push_back(std::move(mf));
				}

				// deque::push_back does not invalidate references to existing elements,
				// so mf_ref remains valid through the two push_backs that follow.
				Frame& mf_ref = stack.back();

				// Push right Build frame (processed second).
				{
					Frame rf;
					rf.kind     = Frame::Kind::Build;
					rf.left     = frame.left + idx;
					rf.right    = frame.right;
					rf.result   = &node->child2;
					rf.in_bbox  = mf_ref.right_bbox;   // independent copy for middleSplit_
					rf.out_bbox = &mf_ref.right_bbox;  // child writes actual bbox back here
					stack.push_back(std::move(rf));
				}

				// Push left Build frame last — on top of the stack, processed first.
				{
					Frame lf;
					lf.kind     = Frame::Kind::Build;
					lf.left     = frame.left;
					lf.right    = frame.left + idx;
					lf.result   = &node->child1;
					lf.in_bbox  = mf_ref.left_bbox;    // independent copy for middleSplit_
					lf.out_bbox = &mf_ref.left_bbox;   // child writes actual bbox back here
					stack.push_back(std::move(lf));
				}
			}
		}
		else // Merge: both children have written their actual bboxes back
		{
			frame.node->node_type.sub.divlow  = frame.left_bbox[frame.cutfeat].high;
			frame.node->node_type.sub.divhigh = frame.right_bbox[frame.cutfeat].low;

			for (Dimension i = 0; i < obj.dim; ++i)
			{
				(*frame.merge_out)[i].low  = std::min(frame.left_bbox[i].low,  frame.right_bbox[i].low);
				(*frame.merge_out)[i].high = std::max(frame.left_bbox[i].high, frame.right_bbox[i].high);
			}
		}
	}

	return root_result;
}

} // namespace nanoflann

#endif // UE_POSE_SEARCH_USE_NANOFLANN
