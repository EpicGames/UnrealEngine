// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/MotionMatchingAnimNodeLibrary.h"
#include "AlphaBlend.h"
#include "PoseSearch/AnimNode_MotionMatching.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MotionMatchingAnimNodeLibrary)

FMotionMatchingBlueprintBlendSettings::FMotionMatchingBlueprintBlendSettings()
	: BlendTime(0.2f)
	, BlendProfile(nullptr)
	, BlendOption(UE::Anim::DefaultBlendOption)
	, bUseInertialBlend(false)
{
}

bool FMotionMatchingBlueprintBlendSettings::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading() && Ar.IsSerializingDefaults())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			BlendOption = EAlphaBlendOption::Linear;
		}
	}

	return false;
}

bool FMotionMatchingBlueprintBlendSettings::Serialize(FStructuredArchive::FSlot Slot)
{
	FArchive& Ar = Slot.GetUnderlyingArchive();
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.IsSerializingDefaults())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			BlendOption = EAlphaBlendOption::Linear;
		}
	}
	return false;
}

FMotionMatchingAnimNodeReference UMotionMatchingAnimNodeLibrary::ConvertToMotionMatchingNode(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result)
{
	return FAnimNodeReference::ConvertToType<FMotionMatchingAnimNodeReference>(Node, Result);
}

void UMotionMatchingAnimNodeLibrary::GetMotionMatchingSearchResult(const FMotionMatchingAnimNodeReference& MotionMatchingNode, FPoseSearchBlueprintResult& Result, bool& bIsResultValid)
{
	using namespace UE::PoseSearch;
	
	if (FAnimNode_MotionMatching* MotionMatchingNodePtr = MotionMatchingNode.GetAnimNodePtr<FAnimNode_MotionMatching>())
	{
		const FMotionMatchingState& MotionMatchingState = MotionMatchingNodePtr->GetMotionMatchingState();
		Result = MotionMatchingState.SearchResult;
		bIsResultValid = MotionMatchingState.SearchResult.SelectedAnim != nullptr;
	}
	else
	{
		Result = FPoseSearchBlueprintResult();
		bIsResultValid = false;
		UE_LOGF(LogPoseSearch, Warning, "UMotionMatchingAnimNodeLibrary::GetMotionMatchingSearchResult called on an invalid context or with an invalid type");
	}
}

void UMotionMatchingAnimNodeLibrary::GetMotionMatchingBlendSettings(const FMotionMatchingAnimNodeReference& MotionMatchingNode, FMotionMatchingBlueprintBlendSettings& BlendSettings, bool& bIsResultValid)
{
	if (FAnimNode_MotionMatching* MotionMatchingNodePtr = MotionMatchingNode.GetAnimNodePtr<FAnimNode_MotionMatching>())
	{
		BlendSettings.BlendOption = MotionMatchingNodePtr->BlendOption;
		BlendSettings.BlendProfile = MotionMatchingNodePtr->BlendProfile;
		BlendSettings.BlendTime = MotionMatchingNodePtr->BlendTime;
		BlendSettings.bUseInertialBlend = MotionMatchingNodePtr->bUseInertialBlend;
		bIsResultValid = true;
	}
	else
	{
		BlendSettings = FMotionMatchingBlueprintBlendSettings();
		bIsResultValid = false;
		UE_LOGF(LogPoseSearch, Warning, "UMotionMatchingAnimNodeLibrary::GetMotionMatchingBlendSettings called on an invalid context or with an invalid type");
	}
}

void UMotionMatchingAnimNodeLibrary::OverrideMotionMatchingBlendSettings(const FMotionMatchingAnimNodeReference& MotionMatchingNode, const FMotionMatchingBlueprintBlendSettings& BlendSettings)
{
	if (FAnimNode_MotionMatching* MotionMatchingNodePtr = MotionMatchingNode.GetAnimNodePtr<FAnimNode_MotionMatching>())
	{
		 MotionMatchingNodePtr->BlendOption = BlendSettings.BlendOption;
		 MotionMatchingNodePtr->BlendProfile = BlendSettings.BlendProfile;
		 MotionMatchingNodePtr->BlendTime = BlendSettings.BlendTime;
		 MotionMatchingNodePtr->bUseInertialBlend = BlendSettings.bUseInertialBlend;
	}
	else
	{
		UE_LOGF(LogPoseSearch, Warning, "UMotionMatchingAnimNodeLibrary::OverrideMotionMatchingBlendSettings called on an invalid context or with an invalid type");
	}
}

void UMotionMatchingAnimNodeLibrary::SetDatabaseToSearch(const FMotionMatchingAnimNodeReference& MotionMatchingNode, UPoseSearchDatabase* Database, EPoseSearchInterruptMode InterruptMode)
{
	if (FAnimNode_MotionMatching* MotionMatchingNodePtr = MotionMatchingNode.GetAnimNodePtr<FAnimNode_MotionMatching>())
	{
		MotionMatchingNodePtr->SetDatabaseToSearch(Database, InterruptMode);
	}
	else
	{
		UE_LOGF(LogPoseSearch, Warning, "UMotionMatchingAnimNodeLibrary::SetDatabaseToSearch called on an invalid context or with an invalid type");
	}
}

void UMotionMatchingAnimNodeLibrary::SetDatabasesToSearch(const FMotionMatchingAnimNodeReference& MotionMatchingNode, const TArray<UPoseSearchDatabase*>& Databases, EPoseSearchInterruptMode InterruptMode)
{
	if (FAnimNode_MotionMatching* MotionMatchingNodePtr = MotionMatchingNode.GetAnimNodePtr<FAnimNode_MotionMatching>())
	{
		MotionMatchingNodePtr->SetDatabasesToSearch(Databases, InterruptMode);
	}
	else
	{
		UE_LOGF(LogPoseSearch, Warning, "UMotionMatchingAnimNodeLibrary::SetDatabasesToSearch called on an invalid context or with an invalid type");
	}
}

void UMotionMatchingAnimNodeLibrary::ResetDatabasesToSearch(const FMotionMatchingAnimNodeReference& MotionMatchingNode, EPoseSearchInterruptMode InterruptMode)
{
	if (FAnimNode_MotionMatching* MotionMatchingNodePtr = MotionMatchingNode.GetAnimNodePtr<FAnimNode_MotionMatching>())
	{
		MotionMatchingNodePtr->ResetDatabasesToSearch(InterruptMode);
	}
	else
	{
		UE_LOGF(LogPoseSearch, Warning, "UMotionMatchingAnimNodeLibrary::ResetDatabasesToSearch called on an invalid context or with an invalid type");
	}
}

void UMotionMatchingAnimNodeLibrary::SetInterruptMode(const FMotionMatchingAnimNodeReference& MotionMatchingNode, EPoseSearchInterruptMode InterruptMode)
{
	if (FAnimNode_MotionMatching* MotionMatchingNodePtr = MotionMatchingNode.GetAnimNodePtr<FAnimNode_MotionMatching>())
	{
		MotionMatchingNodePtr->SetInterruptMode(InterruptMode);
	}
	else
	{
		UE_LOGF(LogPoseSearch, Warning, "UMotionMatchingAnimNodeLibrary::SetInterruptMode called on an invalid context or with an invalid type");
	}
}

bool UMotionMatchingAnimNodeLibrary::AnyNewBlendToThisFrame(const FMotionMatchingAnimNodeReference& MotionMatchingNode)
{
	if (const FAnimNode_MotionMatching* MotionMatchingNodePtr = MotionMatchingNode.GetAnimNodePtr<FAnimNode_MotionMatching>())
	{
		return MotionMatchingNodePtr->AnyNewBlendToThisFrame();
	}
	
	UE_LOGF(LogPoseSearch, Warning, "UMotionMatchingAnimNodeLibrary::AnyNewBlendToThisFrame called on an invalid context or with an invalid type");
	return false;
}