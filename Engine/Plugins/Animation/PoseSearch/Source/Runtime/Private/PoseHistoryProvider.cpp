// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseHistoryProvider.h"
#include "PoseSearch/AnimNode_PoseSearchHistoryCollector.h"

namespace UE::PoseSearch
{
	
const IPoseHistory& FPoseHistoryProvider::GetPoseHistory() const
{
	check(HistoryCollector);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return HistoryCollector->GetPoseHistory();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const IPoseHistory* FPoseHistoryProvider::GetPoseHistoryPtr() const
{
	check(HistoryCollector);
	return HistoryCollector->GetPoseHistoryPtr();
}

} // namespace UE::PoseSearch