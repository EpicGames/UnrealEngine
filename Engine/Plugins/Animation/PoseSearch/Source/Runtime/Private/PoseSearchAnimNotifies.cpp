// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchAnimNotifies.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchAnimNotifies)

uint32 UAnimNotifyState_PoseSearchBranchIn::GetBranchInId() const
{
	const uint32 BranchInId = GetTypeHash(GetFullName());
	check(BranchInId != 0);
	return BranchInId;
}
