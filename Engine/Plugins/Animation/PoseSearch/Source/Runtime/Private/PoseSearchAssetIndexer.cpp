// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "PoseSearch/PoseSearchAssetIndexer.h"
#include "AnimationRuntime.h"
#include "Animation/MirrorDataTable.h"
#include "PoseSearch/PoseSearchAnimNotifies.h"
#include "PoseSearch/PoseSearchAssetSampler.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "PoseSearch/PoseSearchFeatureChannel.h"
#include "PoseSearch/PoseSearchIndex.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearch/PoseSearchDatabase.h"

namespace UE::PoseSearch
{

#if ENABLE_ANIM_DEBUG
static bool GVarMotionMatchTestDisableIndexerCaching = false;
static FAutoConsoleVariableRef CVarMotionMatchTestDisableIndexerCaching(TEXT("a.MotionMatch.Test.DisableIndexerCaching"), GVarMotionMatchTestDisableIndexerCaching, TEXT("Disable Motion Matching Indexer Caching"));

static int32 GVarMotionMatchTestExtractPoseDeterminismNumIterations = 0;
static FAutoConsoleVariableRef CVarMotionMatchTestExtractPoseDeterminismNumIterations(TEXT("a.MotionMatch.Test.ExtractPoseDeterminismNumIterations"), GVarMotionMatchTestExtractPoseDeterminismNumIterations, TEXT("Test Motion Matching ExtractPose Determinism via this NumIterations retries"));
#endif // ENABLE_ANIM_DEBUG

//////////////////////////////////////////////////////////////////////////
// FAssetSamplingContext
FAssetSamplingContext::FAssetSamplingContext(const UPoseSearchDatabase& Database)
{
	BaseCostBias = Database.BaseCostBias;
	LoopingCostBias = Database.LoopingCostBias;
}

//////////////////////////////////////////////////////////////////////////
// FAnimationAssetSamplers
void FAnimationAssetSamplers::Reset()
{
	AnimationAssetSamplers.Reset();
	MirrorDataCaches.Reset();
}

int32 FAnimationAssetSamplers::Num() const
{
	check(AnimationAssetSamplers.Num() == MirrorDataCaches.Num());
	return AnimationAssetSamplers.Num();
}

float FAnimationAssetSamplers::GetPlayLength() const
{
	float PlayLength = 0.f;
	for (const FAnimationAssetSampler* Sampler : AnimationAssetSamplers)
	{
		PlayLength = FMath::Max(PlayLength, Sampler->GetPlayLength());
	}
	return PlayLength;
}

bool FAnimationAssetSamplers::IsLoopable() const
{
	float CommonPlayLength = -1.f;
	for (const FAnimationAssetSampler* Sampler : AnimationAssetSamplers)
	{
		if (!Sampler->IsLoopable())
		{
			return false;
		}

		if (CommonPlayLength < 0.f)
		{
			CommonPlayLength = Sampler->GetPlayLength();
		}
		else if (!FMath::IsNearlyEqual(CommonPlayLength, Sampler->GetPlayLength()))
		{
			return false;
		}
	}
	return true;
}

void FAnimationAssetSamplers::ExtractAnimNotifyStates(float Time, FAnimNotifyContext& PreAllocatedNotifyContext, const TFunction<bool(UAnimNotifyState*)>& ProcessAnimNotifyState) const
{
	for (const FAnimationAssetSampler* Sampler : AnimationAssetSamplers)
	{
		Sampler->ExtractAnimNotifyStates(Time, PreAllocatedNotifyContext, ProcessAnimNotifyState);
	}
}

bool FAnimationAssetSamplers::ProcessAllAnimNotifyEvents(const TFunction<bool(TConstArrayView<FAnimNotifyEvent>)>& ProcessAnimNotifyEvents) const
{
	for (const FAnimationAssetSampler* Sampler : AnimationAssetSamplers)
	{
		if (ProcessAnimNotifyEvents(Sampler->GetAllAnimNotifyEvents()))
		{
			return true;
		}
	}
	return false;
}

const FString FAnimationAssetSamplers::GetAssetName() const
{
	FString Name;
	Name.Reserve(256);
	bool bAddComma = false;
	for (const FAnimationAssetSampler* Sampler : AnimationAssetSamplers)
	{
		if (bAddComma)
		{
			Name += ", ";
		}
		else
		{
			bAddComma = true;
		}

		Name += GetNameSafe(Sampler->GetAsset());
	}
	return Name;
}

FTransform FAnimationAssetSamplers::ExtractRootTransform(float Time, int32 RoleIndex) const
{
	return AnimationAssetSamplers[RoleIndex]->ExtractRootTransform(Time);
}

FTransform FAnimationAssetSamplers::GetTotalRootTransform(int32 RoleIndex) const
{
	return AnimationAssetSamplers[RoleIndex]->GetTotalRootTransform();
}

void FAnimationAssetSamplers::ExtractPose(float Time, FCompactPose& OutPose, int32 RoleIndex) const
{
	AnimationAssetSamplers[RoleIndex]->ExtractPose(Time, OutPose);
}

void FAnimationAssetSamplers::ExtractPose(float Time, FCompactPose& OutPose, FBlendedCurve& OutCurve, int32 RoleIndex) const
{
	AnimationAssetSamplers[RoleIndex]->ExtractPose(Time, OutPose, OutCurve);
}

FTransform FAnimationAssetSamplers::MirrorTransform(const FTransform& InTransform, int32 RoleIndex) const
{
	return MirrorDataCaches[RoleIndex]->MirrorTransform(InTransform);
}

void FAnimationAssetSamplers::MirrorPose(FCompactPose& Pose, int32 RoleIndex) const
{
	MirrorDataCaches[RoleIndex]->MirrorPose(Pose);
}

//////////////////////////////////////////////////////////////////////////
// FAssetIndexer
FAssetIndexer::FAssetIndexer(const TConstArrayView<FBoneContainer> InBoneContainers, const FSearchIndexAsset& InSearchIndexAsset, const FAssetSamplingContext& InSamplingContext,
	const UPoseSearchSchema& InSchema, const FAnimationAssetSamplers& InAssetSamplers, const FRoleToIndex& InRoleToIndex, const FFloatInterval& InExtrapolationTimeInterval)
: BoneContainers(InBoneContainers)
, CachedEntries()
, SearchIndexAsset(InSearchIndexAsset)
, SamplingContext(InSamplingContext)
, Schema(InSchema)
, AssetSamplers(InAssetSamplers)
, RoleToIndex(InRoleToIndex)
, ExtrapolationTimeInterval(InExtrapolationTimeInterval)
{
	check(BoneContainers.Num() == AssetSamplers.Num() && BoneContainers.Num() == RoleToIndex.Num());
	check(IsValid(RoleToIndex));

	CachedEntries.Reserve(SearchIndexAsset.GetNumPoses());
}

void FAssetIndexer::AssignWorkingData(int32 InStartPoseIdx, TArrayView<float> InOutFeatureVectorTable, TArrayView<FPoseMetadata> InOutPoseMetadata)
{
	const int32 NumIndexedPoses = GetNumIndexedPoses();

	StartPoseIdx = InStartPoseIdx;
	FeatureVectorTable = InOutFeatureVectorTable.Slice(Schema.SchemaCardinality * StartPoseIdx, Schema.SchemaCardinality * NumIndexedPoses);
	PoseMetadata = InOutPoseMetadata.Slice(StartPoseIdx, NumIndexedPoses);
}

void FAssetIndexer::Process(int32 AssetIdx)
{
	if (AssetSamplers.Num() == 0 || PoseMetadata.IsEmpty())
	{
		bProcessFailed = true;
	}
	else
	{
		bProcessFailed = false;

		// Generate pose metadata
		const float PlayLength = GetPlayLength();
		FAnimNotifyContext PreAllocatedNotifyContext;

		// @todo: optimize this code, by extracting ALL the notify states, and then perform time overlap with the poses
		for (int32 SampleIdx = GetBeginSampleIdx(); SampleIdx != GetEndSampleIdx(); ++SampleIdx)
		{
			const float SampleTime = FMath::Min(CalculateSampleTime(SampleIdx), PlayLength);
			float CostAddend = SamplingContext.BaseCostBias;
			bool bBlockTransition = false;

			AssetSamplers.ExtractAnimNotifyStates(SampleTime, PreAllocatedNotifyContext, [&bBlockTransition, &CostAddend](const UAnimNotifyState* AnimNotifyState)
				{
					if (AnimNotifyState->GetClass()->IsChildOf<UAnimNotifyState_PoseSearchBlockTransition>())
					{
						bBlockTransition = true;
					}
					else if (AnimNotifyState->GetClass()->IsChildOf<UAnimNotifyState_PoseSearchModifyCost>())
					{
						const UAnimNotifyState_PoseSearchModifyCost* ModifyCostNotifyState = Cast<const UAnimNotifyState_PoseSearchModifyCost>(AnimNotifyState);
						check(ModifyCostNotifyState);
						CostAddend = ModifyCostNotifyState->CostAddend;
					}
					return true;
				});

			if (AssetSamplers.IsLoopable())
			{
				CostAddend += SamplingContext.LoopingCostBias;
			}

			const int32 VectorIdx = GetVectorIdx(SampleIdx);
			const int32 PoseIdx = StartPoseIdx + VectorIdx;
			const int32 ValueOffset = PoseIdx * Schema.SchemaCardinality;
			check(ValueOffset >= 0 && AssetIdx >= 0);
			PoseMetadata[VectorIdx] = FPoseMetadata(ValueOffset, AssetIdx, bBlockTransition, CostAddend);
		}

		AssetSamplers.ProcessAllAnimNotifyEvents([this](const TConstArrayView<FAnimNotifyEvent> AnimNotifyEvents)
			{
				for (const FAnimNotifyEvent& AnimNotifyEvent : AnimNotifyEvents)
				{
					if (AnimNotifyEvent.Notify && AnimNotifyEvent.Notify->GetClass()->IsChildOf<UAnimNotify_PoseSearchEvent>())
					{
						const UAnimNotify_PoseSearchEvent* EventNotify = Cast<const UAnimNotify_PoseSearchEvent>(AnimNotifyEvent.Notify);
						check(EventNotify);
						if (EventNotify->EventTag.IsValid())
						{
							const int32 PoseIdx = SearchIndexAsset.GetPoseIndexFromTime(AnimNotifyEvent.GetTime(), Schema.SampleRate);
							if (PoseIdx != INDEX_NONE)
							{
								EventDataCollector.Emplace(EventNotify->EventTag, PoseIdx);
							}
						}
					}
				}
				return true;
			});

		// Generate pose features data
		if (Schema.SchemaCardinality > 0)
		{
			for (const TObjectPtr<UPoseSearchFeatureChannel>& ChannelPtr : Schema.GetChannels())
			{
				if (!ChannelPtr->IndexAsset(*this))
				{
					bProcessFailed = true;
					break;
				}
			}
		}

		// Computing stats
		if (!bProcessFailed)
		{
			ComputeStats();
		}
	}
}

void FAssetIndexer::ComputeStats()
{
	Stats = FStats();

	for (const FRoleToIndexPair& RoleToIndexPair : RoleToIndex)
	{
		const FRole& Role = RoleToIndexPair.Key;
		const int32 RoleIndex = RoleToIndexPair.Value;
		const FBoneReference& RootBoneReference = Schema.GetBoneReferences(Role)[RootSchemaBoneIdx];

		for (int32 SampleIdx = GetBeginSampleIdx(); SampleIdx != GetEndSampleIdx(); ++SampleIdx)
		{
			const float SampleTime = FMath::Min(CalculateSampleTime(SampleIdx), GetPlayLength());

			const FTransform TrajTransformsPast = GetTransform(SampleTime - FiniteDelta, RoleIndex, RootBoneReference);
			const FTransform TrajTransformsPresent = GetTransform(SampleTime, RoleIndex, RootBoneReference);
			const FTransform TrajTransformsFuture = GetTransform(SampleTime + FiniteDelta, RoleIndex, RootBoneReference);
			const FVector LinearVelocityPresent = (TrajTransformsPresent.GetTranslation() - TrajTransformsPast.GetTranslation()) / FiniteDelta;
			const FVector LinearVelocityFuture = (TrajTransformsFuture.GetTranslation() - TrajTransformsPresent.GetTranslation()) / FiniteDelta;
			const FVector LinearAcceleration = (LinearVelocityFuture - LinearVelocityPresent) / FiniteDelta;

			const float Speed = LinearVelocityPresent.Length();
			const float Acceleration = LinearAcceleration.Length();

			Stats.AccumulatedSpeed += Speed;
			Stats.MaxSpeed = FMath::Max(Stats.MaxSpeed, Speed);

			Stats.AccumulatedAcceleration += Acceleration;
			Stats.MaxAcceleration = FMath::Max(Stats.MaxAcceleration, Acceleration);

			++Stats.NumAccumulatedSamples;
		}
	}
}

FTransform FAssetIndexer::MirrorTransform(const FTransform& Transform, int32 RoleIndex) const
{
	return SearchIndexAsset.IsMirrored() ? AssetSamplers.MirrorTransform(Transform, RoleIndex) : Transform;
}

FAssetIndexer::FCachedEntry& FAssetIndexer::GetEntry(float SampleTime)
{
	using namespace UE::Anim;
	
	bool bDisableCaching = false;
#if ENABLE_ANIM_DEBUG
	bDisableCaching = GVarMotionMatchTestDisableIndexerCaching;
#endif // ENABLE_ANIM_DEBUG

	SampleTime = FMath::Clamp(SampleTime, ExtrapolationTimeInterval.Min, ExtrapolationTimeInterval.Max);

	FCachedEntry* Entry = bDisableCaching ? nullptr : CachedEntries.Find(SampleTime);
	if (!Entry)
	{
		Entry = &CachedEntries.Add(SampleTime);
		Entry->SampleTime = SampleTime;

		const bool bLoopable = AssetSamplers.IsLoopable();
		const float PlayLength = GetPlayLength();
		const int32 AssetSamplersNum = AssetSamplers.Num();

		Entry->RootTransform.SetNum(AssetSamplersNum);
		Entry->ComponentSpacePose.SetNum(AssetSamplersNum);
		Entry->Curves.SetNum(AssetSamplersNum);
		for (int32 RoleIndex = 0; RoleIndex < AssetSamplers.AnimationAssetSamplers.Num(); ++RoleIndex)
		{
			if (!BoneContainers[RoleIndex].IsValid())
			{
				UE_LOGF(LogPoseSearch,
					Warning,
					"Invalid BoneContainer encountered in FAssetIndexer::GetEntry. Asset: %ls. Schema: %ls. BoneContainerAsset: %ls. NumBoneIndices: %d",
					*AssetSamplers.GetAssetName(),
					*GetNameSafe(&Schema),
					*GetNameSafe(BoneContainers[RoleIndex].GetAsset()),
					BoneContainers[RoleIndex].GetCompactPoseNumBones());
			}

			const FTransform SampleRootTransform = AssetSamplers.ExtractRootTransform(SampleTime, RoleIndex);

			FMemMark Mark(FMemStack::Get());
			FCompactPose Pose;
			FBlendedCurve Curve;
			Pose.SetBoneContainer(&BoneContainers[RoleIndex]);
			Curve.InitFrom(BoneContainers[RoleIndex]);
			AssetSamplers.ExtractPose(SampleTime, Pose, Curve, RoleIndex);

#if ENABLE_ANIM_DEBUG
			const int32 NumIterations = GVarMotionMatchTestExtractPoseDeterminismNumIterations;
			for (int32 IterationIndex = 0; IterationIndex < NumIterations; ++IterationIndex)
			{
				FCompactPose TestPose;
				TestPose.SetBoneContainer(&BoneContainers[RoleIndex]);
				AssetSamplers.ExtractPose(SampleTime, TestPose, RoleIndex);

				const TConstArrayView<FTransform> Bones = Pose.GetBones();
				const TConstArrayView<FTransform> TestBones = TestPose.GetBones();
				if (Bones.Num() != TestBones.Num())
				{
					UE_LOGF(LogPoseSearch, Warning, "FAssetIndexer::GetEntry - ExtractPose is not deterministic");
				}
				else
				{
					for (int32 BoneIndex = 0; BoneIndex < Bones.Num(); ++BoneIndex)
					{
						if (FMemory::Memcmp(&Bones[BoneIndex], &TestBones[BoneIndex], sizeof(FTransform)) != 0)
						{
							UE_LOGF(LogPoseSearch, Warning, "FAssetIndexer::GetEntry - ExtractPose is not deterministic");
						}
					}
				}
			}
#endif // ENABLE_ANIM_DEBUG

			if (SearchIndexAsset.IsMirrored())
			{
				AssetSamplers.MirrorPose(Pose, RoleIndex);
			}

			Entry->ComponentSpacePose[RoleIndex].InitPose(Pose);
			Entry->Curves[RoleIndex].CopyFrom(Curve);

			Entry->RootTransform[RoleIndex] = MirrorTransform(SampleRootTransform, RoleIndex);
		}
	}

	return *Entry;
}

// returns the transform in component space for the bone indexed by Schema->BoneReferences[SchemaBoneIdx] at SampleTime seconds
FTransform FAssetIndexer::GetComponentSpaceTransform(float SampleTime, const FRole& Role, int8 SchemaBoneIdx)
{
	using namespace UE::PoseSearch;

	if (SchemaBoneIdx == TrajectorySchemaBoneIdx)
	{
		return FTransform::Identity;
	}

	FCachedEntry& Entry = GetEntry(SampleTime);
	const int32 RoleIndex = RoleToIndex[Role];
	const FBoneReference& BoneReference = Schema.GetBoneReferences(Role)[SchemaBoneIdx];
	return CalculateComponentSpaceTransform(Entry, BoneReference, RoleIndex);
}

// returns the transform in animation space for the bone indexed by Schema->BoneReferences[SchemaBoneIdx] at SampleTime seconds
FTransform FAssetIndexer::GetTransform(float SampleTime, const FRole& Role, int8 SchemaBoneIdx)
{
	using namespace UE::PoseSearch;

	FCachedEntry& Entry = GetEntry(SampleTime);
	const int32 RoleIndex = RoleToIndex[Role];
	if (SchemaBoneIdx == TrajectorySchemaBoneIdx)
	{
		return Entry.RootTransform[RoleIndex];
	}
	
	const FBoneReference& BoneReference = Schema.GetBoneReferences(Role)[SchemaBoneIdx];
	return CalculateComponentSpaceTransform(Entry, BoneReference, RoleIndex) * Entry.RootTransform[RoleIndex];
}

// returns the transform in animation space for the BoneReference at SampleTime seconds
FTransform FAssetIndexer::GetTransform(float SampleTime, int32 RoleIndex, const FBoneReference& BoneReference)
{
	FCachedEntry& Entry = GetEntry(SampleTime);
	return CalculateComponentSpaceTransform(Entry, BoneReference, RoleIndex) * Entry.RootTransform[RoleIndex];
}

FTransform FAssetIndexer::CalculateComponentSpaceTransform(FAssetIndexer::FCachedEntry& Entry, const FBoneReference& BoneReference, int32 RoleIndex)
{
	const FCompactPoseBoneIndex CompactBoneIndex = BoneContainers[RoleIndex].MakeCompactPoseIndex(FMeshPoseBoneIndex(BoneReference.BoneIndex));
	return Entry.ComponentSpacePose[RoleIndex].GetComponentSpaceTransform(CompactBoneIndex);
}

float FAssetIndexer::CalculateSampleTime(int32 SampleIdx) const
{
	return SampleIdx / float(Schema.SampleRate);
}

bool FAssetIndexer::GetSampleRotation(FQuat& OutSampleRotation, float SampleTimeOffset, float OriginTimeOffset, int32 SampleIdx, int8 SchemaSampleBoneIdx, int8 SchemaOriginBoneIdx, const FRole& SampleRole, const FRole& OriginRole, EPermutationTimeType PermutationTimeType, int32 SamplingAttributeId)
{
	using namespace UE::PoseSearch;

	float PermutationSampleTimeOffset = 0.f;
	float PermutationOriginTimeOffset = 0.f;
	UPoseSearchFeatureChannel::GetPermutationTimeOffsets(PermutationTimeType, CalculatePermutationTimeOffset(), PermutationSampleTimeOffset, PermutationOriginTimeOffset);

	const float Time = CalculateSampleTime(SampleIdx);
	const float SampleTime = Time + SampleTimeOffset + PermutationSampleTimeOffset;
	const float OriginTime = Time + OriginTimeOffset + PermutationOriginTimeOffset;

	if (SamplingAttributeId >= 0)
	{
		const FPoseSearchTimedNotifies<UAnimNotifyState_PoseSearchSamplingAttribute> TimedNotifies(SamplingAttributeId, *this);
		const FPoseSearchTimedNotifies<UAnimNotifyState_PoseSearchSamplingAttribute>::FItem TimedNotifiesItem = TimedNotifies.GetClosestFutureEvent(SampleTime);
		if (const UAnimNotifyState_PoseSearchSamplingAttribute* SamplingAttribute = TimedNotifiesItem.NotifyState)
		{
			if (SamplingAttribute->Bone.BoneName != NAME_None)
			{
				FBoneReference TempBoneReference = SamplingAttribute->Bone;
				const int32 SampleRoleIndex = RoleToIndex[SampleRole];
				TempBoneReference.Initialize(BoneContainers[SampleRoleIndex].GetSkeletonAsset());
				if (TempBoneReference.HasValidSetup())
				{
					const float SamplingAttributeTime = TimedNotifiesItem.Time;
					const FTransform RootBoneTransform = GetTransform(OriginTime, OriginRole, RootSchemaBoneIdx);
					const FTransform SamplingAttributeBoneTransform = GetTransform(SamplingAttributeTime, SampleRoleIndex, TempBoneReference);
					OutSampleRotation = RootBoneTransform.InverseTransformRotation(SamplingAttributeBoneTransform.GetRotation());
					return true;
				}

				UE_LOGF(LogPoseSearch, Error, "FAssetIndexer::GetSampleRotation: required UAnimNotifyState_PoseSearchSamplingAttribute in '%ls' has an invalid Bone", *AssetSamplers.GetAssetName());
				return false;
			}

			const FTransform RootBoneTransform = GetTransform(OriginTime, OriginRole, RootSchemaBoneIdx);
			OutSampleRotation = RootBoneTransform.InverseTransformRotation(SamplingAttribute->Rotation);
			return true;
		}

		UE_LOGF(LogPoseSearch, Error, "FAssetIndexer::GetSampleRotation: required UAnimNotifyState_PoseSearchSamplingAttribute not found in '%ls'", *AssetSamplers.GetAssetName());
		OutSampleRotation = FQuat::Identity;
		return false;
	}

	const FTransform OriginBoneTransform = GetTransform(OriginTime, OriginRole, SchemaOriginBoneIdx);
	const FTransform SampleBoneTransform = GetTransform(SampleTime, SampleRole, SchemaSampleBoneIdx);
	OutSampleRotation = OriginBoneTransform.InverseTransformRotation(SampleBoneTransform.GetRotation());

	return true;
}

bool FAssetIndexer::GetSampleCurveValue(float& OutCurveValue, float SampleTimeOffset, int32 SampleIdx, const FName& CurveName, const FRole& SampleRole)
{
	const float Time = CalculateSampleTime(SampleIdx);
	const float SampleTime = Time + SampleTimeOffset;

	OutCurveValue = GetSampleCurveValueInternal(SampleTime, CurveName, SampleRole);
	return true;
}


float FAssetIndexer::GetSampleCurveValueInternal(float SampleTime, const FName& CurveName, const FRole& Role)
{
	const int32 SampleRoleIndex = RoleToIndex[Role];
	FCachedEntry& Entry = GetEntry(SampleTime);

	return Entry.Curves[SampleRoleIndex].Get(CurveName);
}

bool FAssetIndexer::GetSamplePosition(FVector& OutSamplePosition, float SampleTimeOffset, float OriginTimeOffset, int32 SampleIdx, int8 SchemaSampleBoneIdx, int8 SchemaOriginBoneIdx, const FRole& SampleRole, const FRole& OriginRole, EPermutationTimeType PermutationTimeType, int32 SamplingAttributeId)
{
	float PermutationSampleTimeOffset = 0.f;
	float PermutationOriginTimeOffset = 0.f;
	UPoseSearchFeatureChannel::GetPermutationTimeOffsets(PermutationTimeType, CalculatePermutationTimeOffset(), PermutationSampleTimeOffset, PermutationOriginTimeOffset);

	const float Time = CalculateSampleTime(SampleIdx);
	const float SampleTime = Time + SampleTimeOffset + PermutationSampleTimeOffset;
	const float OriginTime = Time + OriginTimeOffset + PermutationOriginTimeOffset;
	
	return GetSamplePositionInternal(OutSamplePosition, SampleTime, OriginTime, SchemaSampleBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole, SamplingAttributeId);
}

bool FAssetIndexer::GetSamplePositionInternal(FVector& OutSamplePosition, float SampleTime, float OriginTime, int8 SchemaSampleBoneIdx, int8 SchemaOriginBoneIdx, const FRole& SampleRole, const FRole& OriginRole, int32 SamplingAttributeId)
{
	using namespace UE::PoseSearch;

	if (SamplingAttributeId >= 0)
	{
		const FPoseSearchTimedNotifies<UAnimNotifyState_PoseSearchSamplingAttribute> TimedNotifies(SamplingAttributeId, *this);
		const FPoseSearchTimedNotifies<UAnimNotifyState_PoseSearchSamplingAttribute>::FItem TimedNotifiesItem = TimedNotifies.GetClosestFutureEvent(SampleTime);
		if (const UAnimNotifyState_PoseSearchSamplingAttribute* SamplingAttribute = TimedNotifiesItem.NotifyState)
		{
			if (SamplingAttribute->Bone.BoneName != NAME_None)
			{
				FBoneReference TempBoneReference = SamplingAttribute->Bone;
				const int32 SampleRoleIndex = RoleToIndex[SampleRole];
				TempBoneReference.Initialize(BoneContainers[SampleRoleIndex].GetSkeletonAsset());
				if (TempBoneReference.HasValidSetup())
				{
					const float SamplingAttributeTime = TimedNotifiesItem.Time;
					const FTransform RootBoneTransform = GetTransform(OriginTime, OriginRole, RootSchemaBoneIdx);
					const FTransform SamplingAttributeBoneTransform = GetTransform(SamplingAttributeTime, SampleRoleIndex, TempBoneReference);
					if (SchemaOriginBoneIdx == RootSchemaBoneIdx)
					{
						OutSamplePosition = RootBoneTransform.InverseTransformPosition(SamplingAttributeBoneTransform.GetTranslation());
					}
					else
					{
						const FTransform OriginBoneTransform = GetTransform(OriginTime, OriginRole, SchemaOriginBoneIdx);
						const FVector DeltaBoneTranslation = SamplingAttributeBoneTransform.GetTranslation() - OriginBoneTransform.GetTranslation();
						OutSamplePosition = RootBoneTransform.InverseTransformVector(DeltaBoneTranslation);
					}
					return true;
				}

				UE_LOGF(LogPoseSearch, Error, "FAssetIndexer::GetSamplePositionInternal: required UAnimNotifyState_PoseSearchSamplingAttribute in '%ls' has an invalid Bone", *AssetSamplers.GetAssetName());
				return false;
			}

			const FTransform RootBoneTransform = GetTransform(OriginTime, OriginRole, RootSchemaBoneIdx);
			OutSamplePosition = RootBoneTransform.InverseTransformPosition(SamplingAttribute->Position);
			return true;
		}

		UE_LOGF(LogPoseSearch, Error, "FAssetIndexer::GetSamplePositionInternal: required UAnimNotifyState_PoseSearchSamplingAttribute not found in '%ls'", *AssetSamplers.GetAssetName());
		OutSamplePosition = FVector::ZeroVector;
		return false;
	}

	const FTransform RootBoneTransform = GetTransform(OriginTime, OriginRole, RootSchemaBoneIdx);
	const FTransform SampleBoneTransform = GetTransform(SampleTime, SampleRole, SchemaSampleBoneIdx);
	if (SchemaOriginBoneIdx == RootSchemaBoneIdx)
	{
		OutSamplePosition = RootBoneTransform.InverseTransformPosition(SampleBoneTransform.GetTranslation());
		return true;
	}

	const FTransform OriginBoneTransform = GetTransform(OriginTime, OriginRole, SchemaOriginBoneIdx);
	const FVector DeltaBoneTranslation = SampleBoneTransform.GetTranslation() - OriginBoneTransform.GetTranslation();
	OutSamplePosition = RootBoneTransform.InverseTransformVector(DeltaBoneTranslation);
	return true;
}

bool FAssetIndexer::GetSampleVelocity(FVector& OutSampleVelocity, float SampleTimeOffset, float OriginTimeOffset, int32 SampleIdx, int8 SchemaSampleBoneIdx, int8 SchemaOriginBoneIdx, const FRole& SampleRole, const FRole& OriginRole, bool bUseCharacterSpaceVelocities, EPermutationTimeType PermutationTimeType, int32 SamplingAttributeId)
{
	float PermutationSampleTimeOffset = 0.f;
	float PermutationOriginTimeOffset = 0.f;
	UPoseSearchFeatureChannel::GetPermutationTimeOffsets(PermutationTimeType, CalculatePermutationTimeOffset(), PermutationSampleTimeOffset, PermutationOriginTimeOffset);

	const float Time = CalculateSampleTime(SampleIdx);
	const float SampleTime = Time + SampleTimeOffset + PermutationSampleTimeOffset;
	const float OriginTime = Time + OriginTimeOffset + PermutationOriginTimeOffset;
	
	if (SamplingAttributeId >= 0)
	{
		const FPoseSearchTimedNotifies<UAnimNotifyState_PoseSearchSamplingAttribute> TimedNotifies(SamplingAttributeId, *this);
		const FPoseSearchTimedNotifies<UAnimNotifyState_PoseSearchSamplingAttribute>::FItem TimedNotifiesItem = TimedNotifies.GetClosestFutureEvent(SampleTime);
		if (const UAnimNotifyState_PoseSearchSamplingAttribute* SamplingAttribute = TimedNotifiesItem.NotifyState)
		{
			FVector BonePositionPast, BonePositionPresent;
			if (SamplingAttribute->Bone.BoneName != NAME_None)
			{
				FBoneReference TempBoneReference = SamplingAttribute->Bone;
				const int32 SampleRoleIndex = RoleToIndex[SampleRole];
				TempBoneReference.Initialize(BoneContainers[SampleRoleIndex].GetSkeletonAsset());
				if (TempBoneReference.HasValidSetup())
				{
					const float SamplingAttributeTime = TimedNotifiesItem.Time;

					if (GetSamplePositionInternal(BonePositionPast, SamplingAttributeTime - FiniteDelta, bUseCharacterSpaceVelocities ? OriginTime - FiniteDelta : OriginTime, SchemaSampleBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole, SamplingAttributeId) &&
						GetSamplePositionInternal(BonePositionPresent, SamplingAttributeTime, OriginTime, SchemaSampleBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole, SamplingAttributeId))
					{
						OutSampleVelocity = (BonePositionPresent - BonePositionPast) / FiniteDelta;
						return true;
					}

					return false;
				}

				UE_LOGF(LogPoseSearch, Error, "FAssetIndexer::GetSampleVelocity: required UAnimNotifyState_PoseSearchSamplingAttribute in '%ls' has an invalid Bone", *AssetSamplers.GetAssetName());
				return false;
			}

			const FTransform RootBoneTransform = GetTransform(OriginTime, OriginRole, RootSchemaBoneIdx);
			OutSampleVelocity = RootBoneTransform.InverseTransformVector(SamplingAttribute->LinearVelocity);
			return true;
		}

		UE_LOGF(LogPoseSearch, Error, "FAssetIndexer::GetSampleVelocity: required UAnimNotifyState_PoseSearchSamplingAttribute not found in '%ls'", *AssetSamplers.GetAssetName());
		OutSampleVelocity = FVector::ZeroVector;
		return false;
	}

	FVector BonePositionPast, BonePositionPresent;
	if (GetSamplePositionInternal(BonePositionPast, SampleTime - FiniteDelta, bUseCharacterSpaceVelocities ? OriginTime - FiniteDelta : OriginTime, SchemaSampleBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole, INDEX_NONE) &&
		GetSamplePositionInternal(BonePositionPresent, SampleTime, OriginTime, SchemaSampleBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole, INDEX_NONE))
	{
		OutSampleVelocity = (BonePositionPresent - BonePositionPast) / FiniteDelta;
		return true;
	}

	OutSampleVelocity = FVector::ZeroVector;
	return false;
}

bool FAssetIndexer::ProcessAllAnimNotifyEvents(const TFunction<bool(TConstArrayView<FAnimNotifyEvent>)>& ProcessAnimNotifyEvents) const
{
	return AssetSamplers.ProcessAllAnimNotifyEvents(ProcessAnimNotifyEvents);
}

const FString FAssetIndexer::GetAssetName() const
{
	return AssetSamplers.GetAssetName();
}

float FAssetIndexer::GetPlayLength() const
{
	return AssetSamplers.GetPlayLength();
}

int32 FAssetIndexer::GetBeginSampleIdx() const
{
	return SearchIndexAsset.GetBeginSampleIdx();
}

int32 FAssetIndexer::GetEndSampleIdx() const
{
	return SearchIndexAsset.GetEndSampleIdx();
}

int32 FAssetIndexer::GetNumIndexedPoses() const
{
	return SearchIndexAsset.GetNumPoses();
}

int32 FAssetIndexer::GetVectorIdx(int32 SampleIdx) const
{
	check(SampleIdx >= 0);
	const int32 BeginSampleIdx = GetBeginSampleIdx();
	check(BeginSampleIdx >= 0);
	check(SampleIdx >= BeginSampleIdx);
	return SampleIdx - BeginSampleIdx;
}

TArrayView<float> FAssetIndexer::GetPoseVector(int32 SampleIdx) const
{
	check(Schema.SchemaCardinality > 0);
	return MakeArrayView(&FeatureVectorTable[GetVectorIdx(SampleIdx) * Schema.SchemaCardinality], Schema.SchemaCardinality);
}

const UPoseSearchSchema* FAssetIndexer::GetSchema() const
{
	return &Schema;
}

#if WITH_EDITOR
float FAssetIndexer::CalculatePermutationTimeOffset() const
{
#if DO_CHECK
	check(Schema.PermutationsSampleRate > 0 && SearchIndexAsset.IsInitialized());
#endif
	const float PermutationTimeOffset = Schema.PermutationsTimeOffset + SearchIndexAsset.GetPermutationIdx() / float(Schema.PermutationsSampleRate);
	return PermutationTimeOffset;
}
#endif // WITH_EDITOR

#if ENABLE_ANIM_DEBUG
void FAssetIndexer::CompareCachedEntries(const FAssetIndexer& Other) const
{
	if (CachedEntries.Num() != Other.CachedEntries.Num())
	{
		UE_LOGF(LogPoseSearch, Warning, "CompareCachedEntries - FAssetIndexer::CachedEntries::Num is not deterministic");
	}
	else
	{
		for (const TPair<float, FCachedEntry>& Pair : CachedEntries)
		{
			if (const FCachedEntry* OtherEntry = Other.CachedEntries.Find(Pair.Key))
			{
				const FCachedEntry* Entry = &Pair.Value;
				if (Entry->SampleTime != OtherEntry->SampleTime)
				{
					UE_LOGF(LogPoseSearch, Warning, "CompareCachedEntries - FAssetIndexer::CachedEntries::SampleTime is not deterministic (%f, %f)", Entry->SampleTime, OtherEntry->SampleTime);
				}

				if (Entry->RootTransform.Num() != OtherEntry->RootTransform.Num())
				{
					UE_LOGF(LogPoseSearch, Warning, "CompareCachedEntries - FAssetIndexer::CachedEntries::RootTransform::Num is not deterministic");
				}
				else
				{
					for (int32 RoleIndex = 0; RoleIndex < Entry->RootTransform.Num(); ++RoleIndex)
					{
						if (FMemory::Memcmp(&Entry->RootTransform[RoleIndex], &OtherEntry->RootTransform[RoleIndex], sizeof(FTransform)) != 0)
						{
							UE_LOGF(LogPoseSearch, Warning, "CompareCachedEntries - FAssetIndexer::CachedEntries::RootTransform[%d] is not deterministic", RoleIndex);
						}
					}
				}

				if (Entry->ComponentSpacePose.Num() != OtherEntry->ComponentSpacePose.Num())
				{
					UE_LOGF(LogPoseSearch, Warning, "CompareCachedEntries - FAssetIndexer::CachedEntries::ComponentSpacePose::Num is not deterministic");
				}
				else
				{
					for (int32 RoleIndex = 0; RoleIndex < Entry->ComponentSpacePose.Num(); ++RoleIndex)
					{
						if (Entry->ComponentSpacePose[RoleIndex].GetComponentSpaceFlags() != OtherEntry->ComponentSpacePose[RoleIndex].GetComponentSpaceFlags())
						{
							UE_LOGF(LogPoseSearch, Warning, "CompareCachedEntries - FAssetIndexer::CachedEntries::ComponentSpacePose[%d]::ComponentSpaceFlags is not deterministic", RoleIndex);
						}

						const TConstArrayView<FTransform> Bones = Entry->ComponentSpacePose[RoleIndex].GetPose().GetBones();
						const TConstArrayView<FTransform> OtherBones = OtherEntry->ComponentSpacePose[RoleIndex].GetPose().GetBones();
						if (Bones.Num() != OtherBones.Num())
						{
							UE_LOGF(LogPoseSearch, Warning, "CompareCachedEntries - FAssetIndexer::CachedEntries::ComponentSpacePose[%d]::Bones is not deterministic", RoleIndex);
						}
						else
						{
							for (int32 BoneIndex = 0; BoneIndex < Bones.Num(); ++BoneIndex)
							{
								if (FMemory::Memcmp(&Bones[BoneIndex], &OtherBones[BoneIndex], sizeof(FTransform)) != 0)
								{
									UE_LOGF(LogPoseSearch, Warning, "CompareCachedEntries - FAssetIndexer::CachedEntries::ComponentSpacePose[%d]::Bones[%d] is not deterministic", RoleIndex, BoneIndex);
								}
							}
						}
					}
				}
			}
			else
			{
				UE_LOGF(LogPoseSearch, Warning, "CompareCachedEntries - FAssetIndexer::CachedEntries is not deterministic. Missing CachedEntry at time %f", Pair.Key);
			}
		}
	}
}
#endif // ENABLE_ANIM_DEBUG

} // namespace UE::PoseSearch
#endif // WITH_EDITOR
