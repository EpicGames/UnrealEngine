// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchAssetSampler.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "PoseSearch/PoseSearchAnimNotifies.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "UObject/Package.h"

namespace UE::PoseSearch
{

//////////////////////////////////////////////////////////////////////////
// Root motion extrapolation

static FTransform ExtrapolateRootMotion(FTransform SampleToExtrapolate, float SampleStart, float SampleEnd, float ExtrapolationTime)
{
	const float SampleDelta = SampleEnd - SampleStart;
	check(!FMath::IsNearlyZero(SampleDelta));

	// converting ExtrapolationTime to a positive number to avoid dealing with the negative extrapolation and inverting
	// transforms later on.
	const float AbsExtrapolationTime = FMath::Abs(ExtrapolationTime);
	const float AbsSampleDelta = FMath::Abs(SampleDelta);
	const FTransform AbsTimeSampleToExtrapolate = ExtrapolationTime >= 0.0f ? SampleToExtrapolate : SampleToExtrapolate.Inverse();

	// because we're extrapolating rotation, the extrapolation must be integrated over time
	const float SampleMultiplier = AbsExtrapolationTime / AbsSampleDelta;
	float IntegralNumSamples;
	float RemainingSampleFraction = FMath::Modf(SampleMultiplier, &IntegralNumSamples);
	int32 NumSamples = (int32)IntegralNumSamples;

	// adding full samples to the extrapolated root motion
	FTransform ExtrapolatedRootMotion = FTransform::Identity;
	for (int32 i = 0; i < NumSamples; ++i)
	{
		ExtrapolatedRootMotion = AbsTimeSampleToExtrapolate * ExtrapolatedRootMotion;
	}

	// and a blend with identity for whatever is left
	FTransform RemainingExtrapolatedRootMotion;
	RemainingExtrapolatedRootMotion.Blend(
		FTransform::Identity,
		AbsTimeSampleToExtrapolate,
		RemainingSampleFraction);

	ExtrapolatedRootMotion = RemainingExtrapolatedRootMotion * ExtrapolatedRootMotion;
	return ExtrapolatedRootMotion;
}

static FTransform ExtractRootTransformInternal(const UAnimMontage* AnimMontage, float StartTime, float EndTime, bool bEnforceCompressedDataSampling)
{
	// @todo: add support for SlotName / multiple SlotAnimTracks
	if (AnimMontage->SlotAnimTracks.Num() != 1)
	{
		UE_LOGF(LogPoseSearch, Error, "ExtractRootTransformInternal: so far we support only montages with one SlotAnimTracks. %ls has %d", *AnimMontage->GetName(), AnimMontage->SlotAnimTracks.Num());
		return FTransform::Identity;
	}

	FAnimExtractContext ExtractionCtx(0.0, true, FDeltaTimeRecord(StartTime), true);
#if WITH_EDITOR
	ExtractionCtx.bEnforceCompressedDataSampling = bEnforceCompressedDataSampling;
	ExtractionCtx.TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
#endif // WITH_EDITOR

	const FAnimTrack& RootMotionAnimTrack = AnimMontage->SlotAnimTracks[0].AnimTrack;
	TArray<FRootMotionExtractionStep> RootMotionExtractionSteps;
	RootMotionAnimTrack.GetRootMotionExtractionStepsForTrackRange(RootMotionExtractionSteps, StartTime, EndTime);
	FRootMotionMovementParams AccumulatedRootMotionParams;
	for (const FRootMotionExtractionStep& CurStep : RootMotionExtractionSteps)
	{
		if (CurStep.AnimSequence)
		{
			AccumulatedRootMotionParams.Accumulate(CurStep.AnimSequence->ExtractRootMotionFromRange(CurStep.StartPosition, CurStep.EndPosition, ExtractionCtx));
		}
	}
	return AccumulatedRootMotionParams.GetRootMotionTransform();
}

static FTransform ExtractBlendSpaceRootTrackTransform(float Time, const TArray<FTransform>& AccumulatedRootTransform, int32 RootTransformSamplingRate)
{
	checkf(AccumulatedRootTransform.Num() > 0, TEXT("ProcessRootTransform must be run first"));

	const int32 Index = Time * RootTransformSamplingRate;
	const int32 FirstIndexClamped = FMath::Clamp(Index + 0, 0, AccumulatedRootTransform.Num() - 1);
	const int32 SecondIndexClamped = FMath::Clamp(Index + 1, 0, AccumulatedRootTransform.Num() - 1);
	const float Alpha = FMath::Fmod(Time * RootTransformSamplingRate, 1.0f);
	FTransform OutputTransform;
	OutputTransform.Blend(AccumulatedRootTransform[FirstIndexClamped], AccumulatedRootTransform[SecondIndexClamped], Alpha);
	return OutputTransform;
}

static FTransform ExtractBlendSpaceRootMotionFromRange(float StartTrackPosition, float EndTrackPosition, const TArray<FTransform>& AccumulatedRootTransform, int32 RootTransformSamplingRate)
{
	checkf(AccumulatedRootTransform.Num() > 0, TEXT("ProcessRootTransform must be run first"));

	FTransform RootTransformRefPose = ExtractBlendSpaceRootTrackTransform(0.f, AccumulatedRootTransform, RootTransformSamplingRate);

	FTransform StartTransform = ExtractBlendSpaceRootTrackTransform(StartTrackPosition, AccumulatedRootTransform, RootTransformSamplingRate);
	FTransform EndTransform = ExtractBlendSpaceRootTrackTransform(EndTrackPosition, AccumulatedRootTransform, RootTransformSamplingRate);

	// Transform to Component Space
	const FTransform RootToComponent = RootTransformRefPose.Inverse();
	StartTransform = RootToComponent * StartTransform;
	EndTransform = RootToComponent * EndTransform;

	return EndTransform.GetRelativeTransform(StartTransform);
}

static FTransform ExtractBlendSpaceRootMotion(float StartTime, float DeltaTime, bool bAllowLooping, float CachedPlayLength, const TArray<FTransform>& AccumulatedRootTransform, int32 RootTransformSamplingRate)
{
	FRootMotionMovementParams RootMotionParams;

	// looking for conditions that will calculate an FTransform::Identity to early out
	const bool bIsTrivial = FMath::IsNearlyZero(DeltaTime) || CachedPlayLength < UE_SMALL_NUMBER || AccumulatedRootTransform.Num() <= 1;
	if (!bIsTrivial)
	{
		bool const bPlayingBackwards = (DeltaTime < 0.f);

		float PreviousPosition = StartTime;
		float CurrentPosition = StartTime;
		float DesiredDeltaMove = DeltaTime;

		do
		{
			// Disable looping here. Advance to desired position, or beginning / end of animation 
			const ETypeAdvanceAnim AdvanceType = FAnimationRuntime::AdvanceTime(false, DesiredDeltaMove, CurrentPosition, CachedPlayLength);

			// Verify position assumptions
			//ensureMsgf(bPlayingBackwards ? (CurrentPosition <= PreviousPosition) : (CurrentPosition >= PreviousPosition), TEXT("in Animation %s(Skeleton %s) : bPlayingBackwards(%d), PreviousPosition(%0.2f), Current Position(%0.2f)"),
			//	*GetName(), *GetNameSafe(GetSkeleton()), bPlayingBackwards, PreviousPosition, CurrentPosition);

			RootMotionParams.Accumulate(ExtractBlendSpaceRootMotionFromRange(PreviousPosition, CurrentPosition, AccumulatedRootTransform, RootTransformSamplingRate));

			// If we've hit the end of the animation, and we're allowed to loop, keep going.
			if ((AdvanceType == ETAA_Finished) && bAllowLooping)
			{
				const float ActualDeltaMove = (CurrentPosition - PreviousPosition);
				DesiredDeltaMove -= ActualDeltaMove;

				PreviousPosition = bPlayingBackwards ? CachedPlayLength : 0.f;
				CurrentPosition = PreviousPosition;
			}
			else
			{
				break;
			}
		} while (true);
	}

	return RootMotionParams.GetRootMotionTransform();
}

static void ProcessRootTransform(const UBlendSpace* BlendSpace, const FVector& BlendParameters, float CachedPlayLength,
	int32 RootTransformSamplingRate, bool bIsLoopable, TArray<FTransform>& AccumulatedRootTransform, bool bEnforceCompressedDataSampling)
{
	// Pre-compute root motion
	const int32 NumRootSamples = FMath::Max(FMath::CeilToInt(CachedPlayLength * RootTransformSamplingRate) + 1, 1);
	AccumulatedRootTransform.Init(FTransform::Identity, NumRootSamples);
	
	TArray<FBlendSampleData> BlendSamplesData;

	int32 TriangulationIndex = 0;
	if (BlendSpace->GetSamplesFromBlendInput(BlendParameters, BlendSamplesData, TriangulationIndex, true))
	{
		TArray<float, TInlineAllocator<16, TMemStackAllocator<>>> PrevSampleTimes;
		PrevSampleTimes.AddDefaulted(BlendSamplesData.Num());
		
		// Get starting time for all samples.
		BlendSpace->ResetBlendSamples(BlendSamplesData, 0.0f, bIsLoopable, true);
		
		for (int32 SampleIdx = 1; SampleIdx < NumRootSamples; ++SampleIdx)
		{
			// Keep track of previous samples
			for (int32 BlendSampleIndex = 0; BlendSampleIndex < BlendSamplesData.Num(); BlendSampleIndex++)
			{
				PrevSampleTimes[BlendSampleIndex] = BlendSamplesData[BlendSampleIndex].Time;
			}

			// Compute samples with new data.
			const float SampleTime = static_cast<float>(SampleIdx) / (NumRootSamples - 1);
			BlendSpace->ResetBlendSamples(BlendSamplesData, SampleTime, bIsLoopable, true);

			FAnimExtractContext ExtractionCtx(0.0, true, {}, bIsLoopable);
#if WITH_EDITOR
			ExtractionCtx.bEnforceCompressedDataSampling = bEnforceCompressedDataSampling;
			ExtractionCtx.TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
#endif // WITH_EDITOR
			
			// Accumulate root motion after samples have been updated.
			FRootMotionMovementParams RootMotionMovementParams;
			for (int32 BlendSampleIndex = 0; BlendSampleIndex < BlendSamplesData.Num(); BlendSampleIndex++)
			{
				FBlendSampleData& BlendSample = BlendSamplesData[BlendSampleIndex];

				if (BlendSample.TotalWeight > ZERO_ANIMWEIGHT_THRESH)
				{
					float DeltaTime = BlendSample.Time - PrevSampleTimes[BlendSampleIndex];

					// Account for looping.
					if (DeltaTime < 0.0f)
					{
						DeltaTime += BlendSample.Animation->GetPlayLength();
					}
					
					ExtractionCtx.CurrentTime = PrevSampleTimes[BlendSampleIndex];
					ExtractionCtx.DeltaTimeRecord.Delta = DeltaTime;
					const FTransform BlendSampleRootMotion = BlendSample.Animation->ExtractRootMotion(ExtractionCtx);
					RootMotionMovementParams.AccumulateWithBlend(BlendSampleRootMotion, BlendSample.GetClampedWeight());
				}
			}

			AccumulatedRootTransform[SampleIdx] = RootMotionMovementParams.GetRootMotionTransform() * AccumulatedRootTransform[SampleIdx - 1];
			
			// keep numerical errors in check
			AccumulatedRootTransform[SampleIdx].NormalizeRotation();
		}
	}
}

static int32 GetHighestWeightSample(const TArray<struct FBlendSampleData>& SampleDataList)
{
	check(!SampleDataList.IsEmpty());
	int32 HighestWeightIndex = 0;
	float HighestWeight = SampleDataList[HighestWeightIndex].GetClampedWeight();
	for (int32 I = 1; I < SampleDataList.Num(); I++)
	{
		if (SampleDataList[I].GetClampedWeight() > HighestWeight)
		{
			HighestWeightIndex = I;
			HighestWeight = SampleDataList[I].GetClampedWeight();
		}
	}
	return HighestWeightIndex;
}

//////////////////////////////////////////////////////////////////////////
// FAssetSamplerBase

FAnimationAssetSampler::FAnimationAssetSampler()
{
	Init(nullptr, FTransform::Identity, FVector::ZeroVector, DefaultRootTransformSamplingRate, true, true);
}

FAnimationAssetSampler::FAnimationAssetSampler(const UAnimationAsset* InAnimationAsset, const FTransform& InRootTransformOrigin, const FVector& InBlendParameters, int32 InRootTransformSamplingRate, bool bPreProcessRootTransform, bool bInEnforceCompressedDataSampling)
{
	Init(InAnimationAsset, InRootTransformOrigin, InBlendParameters, InRootTransformSamplingRate, bPreProcessRootTransform, bInEnforceCompressedDataSampling);
}

FAnimationAssetSampler::~FAnimationAssetSampler() = default;

void FAnimationAssetSampler::Init(const UAnimationAsset* InAnimationAsset, const FTransform& InRootTransformOrigin, const FVector& InBlendParameters, int32 InRootTransformSamplingRate, bool bPreProcessRootTransform, bool bInEnforceCompressedDataSampling)
{
	check(!InAnimationAsset || (InAnimationAsset->GetPackage() && !InAnimationAsset->HasAnyFlags(RF_MirroredGarbage)));
	
	AnimationAssetPtr = InAnimationAsset;
	RootTransformOrigin = InRootTransformOrigin;
	BlendParameters = InBlendParameters;
	RootTransformSamplingRate = InRootTransformSamplingRate;
	CachedPlayLength = GetPlayLength(AnimationAssetPtr.Get(), BlendParameters);

#if WITH_EDITOR
	bEnforceCompressedDataSampling = bInEnforceCompressedDataSampling;
	ITargetPlatform* TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();		
	PlatformHash = FIoHash::Zero;
	if (const UAnimSequence* AnimSequence = Cast<UAnimSequence>(InAnimationAsset))
	{		
		if (!AnimationAssetPtr->GetPackage()->HasAnyPackageFlags(PKG_Cooked))
		{
			PlatformHash = AnimSequence->GetDerivedDataKeyHash(TargetPlatform);
			
			if (bEnforceCompressedDataSampling)
			{
				ensureAlwaysMsgf(AnimSequence->HasCompressedDataForPlatform(TargetPlatform), TEXT("AnimationAssetSampler initialized to forcefully sample compressed animation data, but none is currently available for the target-platform"));
			}
		}
	}
#endif

	if (bPreProcessRootTransform)
	{
		Process();
	}
}

bool FAnimationAssetSampler::IsInitialized() const
{
	return AnimationAssetPtr != nullptr;
}

float FAnimationAssetSampler::GetPlayLength(const UAnimationAsset* AnimAsset, const FVector& BlendParameters)
{
	float PlayLength = 0.f;
	if (AnimAsset)
	{
		if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(AnimAsset))
		{
			TArray<FBlendSampleData> BlendSamples;
			int32 TriangulationIndex = 0;
			BlendSpace->GetSamplesFromBlendInput(BlendParameters, BlendSamples, TriangulationIndex, true);
			PlayLength = BlendSpace->GetAnimationLengthFromSampleData(BlendSamples);
		}
		else
		{
			PlayLength = AnimAsset->GetPlayLength();
		}
	}

	return PlayLength;
}

float FAnimationAssetSampler::WrapOrClampTime(float Time, float PlayLength, bool bLoopable)
{
	if (PlayLength <= UE_SMALL_NUMBER)
	{
		return 0.f;
	}
	
	if (bLoopable)
	{
		const float Result = FMath::Fmod(Time, PlayLength);
		return (Result < 0.f) ? (Result + PlayLength) : Result;
	}
		
	return FMath::Clamp(Time, 0.f, PlayLength);
}

const UAnimationAsset* FAnimationAssetSampler::GetAsset() const
{
	return AnimationAssetPtr.Get();
}

float FAnimationAssetSampler::ToRealTime(float NormalizedTime) const
{
	// Asset player time for blend spaces is normalized [0, 1] so we convert the sampling / animation time to asset time by multiplying it by CachedPlayLength
	if (CachedPlayLength > UE_KINDA_SMALL_NUMBER && Cast<UBlendSpace>(AnimationAssetPtr.Get()))
	{
		check(NormalizedTime >= 0.f && NormalizedTime <= 1.f);
		const float RealTime = NormalizedTime * CachedPlayLength;
		return RealTime;
	}

	return NormalizedTime;
}

float FAnimationAssetSampler::ToNormalizedTime(float RealTime) const
{
	// Asset player time for blend spaces is normalized [0, 1] so we convert the sampling / animation time to asset time by dividing it by CachedPlayLength
	if (CachedPlayLength > UE_KINDA_SMALL_NUMBER && Cast<UBlendSpace>(AnimationAssetPtr.Get()))
	{
		const float NormalizedTime = RealTime / CachedPlayLength;

		if (NormalizedTime >= 0.f && NormalizedTime <= 1.f)
		{
			return NormalizedTime;
		}

		UE_LOGF(LogPoseSearch, Error, "FAnimationAssetSampler::ToNormalizedTime: requested RealTime %f is greater than CachedPlayLength %f for UBlendSpace %ls!", RealTime, CachedPlayLength, *AnimationAssetPtr->GetName());
		return FMath::Clamp(NormalizedTime, 0.f, 1.f);
	}

	return RealTime;
}

float FAnimationAssetSampler::GetPlayLength() const
{
	return CachedPlayLength;
}

bool FAnimationAssetSampler::IsLoopable() const
{
	if (const UAnimSequenceBase* SequenceBase = Cast<UAnimSequenceBase>(AnimationAssetPtr.Get()))
	{
		return SequenceBase->bLoop;
	}

	if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(AnimationAssetPtr.Get()))
	{
		return BlendSpace->bLoop;
	}

	return false;
}

FTransform FAnimationAssetSampler::GetTotalRootTransform() const
{
	if (Cast<UBlendSpace>(AnimationAssetPtr.Get()))
	{
		const FTransform InitialRootTransform = ExtractBlendSpaceRootTrackTransform(0.f, AccumulatedRootTransform, RootTransformSamplingRate);
		const FTransform LastRootTransform = ExtractBlendSpaceRootTrackTransform(CachedPlayLength, AccumulatedRootTransform, RootTransformSamplingRate);
		const FTransform TotalRootTransform = LastRootTransform.GetRelativeTransform(InitialRootTransform);
		return TotalRootTransform;
	}

	if (const UAnimMontage* AnimMontage = Cast<UAnimMontage>(AnimationAssetPtr.Get()))
	{
		// @todo: add support for SlotName / multiple SlotAnimTracks
		if (AnimMontage->SlotAnimTracks.Num() != 1)
		{
			UE_LOGF(LogPoseSearch, Error, "FAssetSamplerBase::GetTotalRootTransform: so far we support only montages with one SlotAnimTracks. %ls has %d", *AnimMontage->GetName(), AnimMontage->SlotAnimTracks.Num());
			return FTransform::Identity;
		}

		// @todo: optimize me
		const FTransform InitialRootTransform = ExtractRootTransform(0.f);
		const FTransform LastRootTransform = ExtractRootTransform(GetPlayLength());
		const FTransform TotalRootTransform = LastRootTransform.GetRelativeTransform(InitialRootTransform);
		return TotalRootTransform;
	}

	const FTransform InitialRootTransform = ExtractRootTransform(0.f);
	const FTransform LastRootTransform = ExtractRootTransform(GetPlayLength());
	const FTransform TotalRootTransform = LastRootTransform.GetRelativeTransform(InitialRootTransform);
	return TotalRootTransform;
}

void FAnimationAssetSampler::ExtractPose(const FAnimExtractContext& ExtractionCtx, FAnimationPoseData& OutAnimPoseData) const
{
#if WITH_EDITOR
	check(ExtractionCtx.TargetPlatformHash == FIoHash::Zero);
#endif // WITH_EDITOR
	if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(AnimationAssetPtr.Get()))
	{
		TArray<FBlendSampleData> BlendSamples;
		int32 TriangulationIndex = 0;
		if (BlendSpace->GetSamplesFromBlendInput(BlendParameters, BlendSamples, TriangulationIndex, true))
		{
			BlendSpace->ResetBlendSamples(BlendSamples, ToNormalizedTime(ExtractionCtx.CurrentTime), ExtractionCtx.bLooping, true);
			BlendSpace->GetAnimationPose(BlendSamples, ExtractionCtx, OutAnimPoseData);
		}
		else
		{
			UE_LOGF(LogPoseSearch, Error, "FAnimMontageSampler::ExtractPose: UBlendSpace %ls couldn't return a valid pose for BlendParameters (%.2f, %.2f, %.2f)", *BlendSpace->GetName(), BlendParameters.X, BlendParameters.Y, BlendParameters.Z);
			OutAnimPoseData.GetPose().ResetToRefPose();
		}
	}
	else if (const UAnimMontage* AnimMontage = Cast<UAnimMontage>(AnimationAssetPtr.Get()))
	{
		// @todo: add support for SlotName / multiple SlotAnimTracks
		if (AnimMontage->SlotAnimTracks.Num() != 1)
		{
			UE_LOGF(LogPoseSearch, Error, "FAnimMontageSampler::ExtractPose: so far we support only montages with one SlotAnimTracks. %ls has %d", *AnimMontage->GetName(), AnimMontage->SlotAnimTracks.Num());
			OutAnimPoseData.GetPose().ResetToRefPose();
		}
		else
		{
			AnimMontage->SlotAnimTracks[0].AnimTrack.GetAnimationPose(OutAnimPoseData, ExtractionCtx);
		}
	}
	else if (const UAnimSequenceBase* SequenceBase = Cast<UAnimSequenceBase>(AnimationAssetPtr.Get()))
	{
		FAnimExtractContext SequenceContext = ExtractionCtx;
#if WITH_EDITOR
		if (Cast<UAnimSequence>(SequenceBase))
		{
			check(SequenceBase->GetPackage()->HasAnyPackageFlags(PKG_Cooked) || PlatformHash != FIoHash::Zero);
			SequenceContext.TargetPlatformHash = PlatformHash;
		}
#endif // WITH_EDITOR
		SequenceBase->GetAnimationPose(OutAnimPoseData, SequenceContext);
	}
	else
	{
		OutAnimPoseData.GetPose().ResetToRefPose();
	}
}

void FAnimationAssetSampler::ExtractPose(float Time, FCompactPose& OutPose) const
{
	UE::Anim::FStackAttributeContainer UnusedAtrribute;
	FBlendedCurve UnusedCurve;
	UnusedCurve.InitFrom(OutPose.GetBoneContainer());
	FAnimationPoseData AnimPoseData = { OutPose, UnusedCurve, UnusedAtrribute };

	const bool bLoopable = IsLoopable();
	const float WrappedOrClampedTime = WrapOrClampTime(Time, GetPlayLength(), bLoopable);

	FDeltaTimeRecord DeltaTimeRecord;
	DeltaTimeRecord.Set(WrappedOrClampedTime, 0.f);
	FAnimExtractContext ExtractionCtx(double(WrappedOrClampedTime), false, DeltaTimeRecord, bLoopable);

#if WITH_EDITOR
	ExtractionCtx.bExtractWithRootMotionProvider = false;
	ExtractionCtx.bEnforceCompressedDataSampling = bEnforceCompressedDataSampling;
	ExtractionCtx.TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
#endif // WITH_EDITOR

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ExtractPose(ExtractionCtx, AnimPoseData);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FAnimationAssetSampler::ExtractPose(float Time, FCompactPose& OutPose, FBlendedCurve& OutCurve) const
{
	UE::Anim::FStackAttributeContainer UnusedAtrribute;
	OutCurve.InitFrom(OutPose.GetBoneContainer());
	FAnimationPoseData AnimPoseData = { OutPose, OutCurve, UnusedAtrribute };

	const bool bLoopable = IsLoopable();
	const float WrappedOrClampedTime = WrapOrClampTime(Time, GetPlayLength(), bLoopable);

	FDeltaTimeRecord DeltaTimeRecord;
	DeltaTimeRecord.Set(WrappedOrClampedTime, 0.f);
	FAnimExtractContext ExtractionCtx(double(WrappedOrClampedTime), false, DeltaTimeRecord, bLoopable);

#if WITH_EDITOR
	ExtractionCtx.bExtractWithRootMotionProvider = false;
	ExtractionCtx.bEnforceCompressedDataSampling = bEnforceCompressedDataSampling;
	ExtractionCtx.TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
#endif // WITH_EDITOR

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ExtractPose(ExtractionCtx, AnimPoseData);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FTransform FAnimationAssetSampler::ExtractRootTransform(float Time) const
{
	FTransform RootTransform = FTransform::Identity;
	
	if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(AnimationAssetPtr.Get()))
	{
		if (IsLoopable())
		{
			RootTransform = ExtractBlendSpaceRootMotion(0.0f, Time, true, CachedPlayLength, AccumulatedRootTransform, RootTransformSamplingRate);
		}
		else
		{
			const float ClampedTime = FMath::Clamp(Time, 0.0f, CachedPlayLength);
			const float ExtrapolationTime = Time - ClampedTime;

			// If Time is less than zero, ExtrapolationTime will be negative. In this case, we extrapolate the beginning of the 
			// animation to estimate where the root would be at Time
			if (ExtrapolationTime < -SMALL_NUMBER)
			{
				FTransform SampleToExtrapolate = ExtractBlendSpaceRootMotionFromRange(0.0f, ExtrapolationSampleTime, AccumulatedRootTransform, RootTransformSamplingRate);

				const FTransform ExtrapolatedRootMotion = UE::PoseSearch::ExtrapolateRootMotion(
					SampleToExtrapolate,
					0.0f, ExtrapolationSampleTime,
					ExtrapolationTime);
				RootTransform = ExtrapolatedRootMotion;
			}
			else
			{
				RootTransform = ExtractBlendSpaceRootMotionFromRange(0.0f, ClampedTime, AccumulatedRootTransform, RootTransformSamplingRate);

				// If Time is greater than PlayLength, ExtrapolationTime will be a positive number. In this case, we extrapolate
				// the end of the animation to estimate where the root would be at Time
				if (ExtrapolationTime > SMALL_NUMBER)
				{
					FTransform SampleToExtrapolate = ExtractBlendSpaceRootMotionFromRange(CachedPlayLength - ExtrapolationSampleTime, CachedPlayLength, AccumulatedRootTransform, RootTransformSamplingRate);

					const FTransform ExtrapolatedRootMotion = UE::PoseSearch::ExtrapolateRootMotion(
						SampleToExtrapolate,
						CachedPlayLength - ExtrapolationSampleTime, CachedPlayLength,
						ExtrapolationTime);
					RootTransform = ExtrapolatedRootMotion * RootTransform;
				}
			}
		}
	}
	else if (const UAnimMontage* AnimMontage = Cast<UAnimMontage>(AnimationAssetPtr.Get()))
	{
		if (IsLoopable())
		{
			RootTransform = ExtractRootTransformInternal(AnimMontage, 0.f, Time, bEnforceCompressedDataSampling);
		}
		else
		{
			const float PlayLength = GetPlayLength();
			const float ClampedTime = FMath::Clamp(Time, 0.f, PlayLength);
			const float ExtrapolationTime = Time - ClampedTime;

			// If Time is less than zero, ExtrapolationTime will be negative. In this case, we extrapolate the beginning of the 
			// animation to estimate where the root would be at Time
			if (ExtrapolationTime < -SMALL_NUMBER)
			{
				FTransform SampleToExtrapolate = ExtractRootTransformInternal(AnimMontage, 0.f, ExtrapolationSampleTime, bEnforceCompressedDataSampling);

				const FTransform ExtrapolatedRootMotion = UE::PoseSearch::ExtrapolateRootMotion(
					SampleToExtrapolate,
					0.0f, ExtrapolationSampleTime,
					ExtrapolationTime);
				RootTransform = ExtrapolatedRootMotion;
			}
			else
			{
				RootTransform = ExtractRootTransformInternal(AnimMontage, 0.f, ClampedTime, bEnforceCompressedDataSampling);

				// If Time is greater than PlayLength, ExtrapolationTime will be a positive number. In this case, we extrapolate
				// the end of the animation to estimate where the root would be at Time
				if (ExtrapolationTime > SMALL_NUMBER)
				{
					FTransform SampleToExtrapolate = ExtractRootTransformInternal(AnimMontage, PlayLength - ExtrapolationSampleTime, PlayLength, bEnforceCompressedDataSampling);

					const FTransform ExtrapolatedRootMotion = UE::PoseSearch::ExtrapolateRootMotion(
						SampleToExtrapolate,
						PlayLength - ExtrapolationSampleTime, PlayLength,
						ExtrapolationTime);
					RootTransform = ExtrapolatedRootMotion * RootTransform;
				}
			}
		}
	}
	else if (const UAnimSequenceBase* SequenceBase = Cast<UAnimSequenceBase>(AnimationAssetPtr.Get()))
	{
		FAnimExtractContext ExtractionCtx(0.0, true, FDeltaTimeRecord(Time), true);
#if WITH_EDITOR
		ExtractionCtx.bEnforceCompressedDataSampling = bEnforceCompressedDataSampling;
		ExtractionCtx.TargetPlatformHash = PlatformHash;
#endif // WITH_EDITOR
		
		if (IsLoopable())
		{
			RootTransform = SequenceBase->ExtractRootMotion(ExtractionCtx);
		}
		else
		{
			const float PlayLength = GetPlayLength();
			const float ClampedTime = FMath::Clamp(Time, 0.0f, PlayLength);
			const float ExtrapolationTime = Time - ClampedTime;

			// If Time is less than zero, ExtrapolationTime will be negative. In this case, we extrapolate the beginning of the 
			// animation to estimate where the root would be at Time
			if (ExtrapolationTime < -SMALL_NUMBER)
			{
				FTransform SampleToExtrapolate = SequenceBase->ExtractRootMotionFromRange(0.0f, ExtrapolationSampleTime, ExtractionCtx);

				const FTransform ExtrapolatedRootMotion = UE::PoseSearch::ExtrapolateRootMotion(
					SampleToExtrapolate,
					0.0f, ExtrapolationSampleTime,
					ExtrapolationTime);
				RootTransform = ExtrapolatedRootMotion;
			}
			else
			{
				RootTransform = SequenceBase->ExtractRootMotionFromRange(0.0f, ClampedTime, ExtractionCtx);

				// If Time is greater than PlayLength, ExtrapolationTime will be a positive number. In this case, we extrapolate
				// the end of the animation to estimate where the root would be at Time
				if (ExtrapolationTime > SMALL_NUMBER)
				{
					FTransform SampleToExtrapolate = SequenceBase->ExtractRootMotionFromRange(PlayLength - ExtrapolationSampleTime, PlayLength, ExtractionCtx);

					const FTransform ExtrapolatedRootMotion = UE::PoseSearch::ExtrapolateRootMotion(
						SampleToExtrapolate,
						PlayLength - ExtrapolationSampleTime, PlayLength,
						ExtrapolationTime);
					RootTransform = ExtrapolatedRootMotion * RootTransform;
				}
			}
		}
	}

	return RootTransform * RootTransformOrigin;
}

void FAnimationAssetSampler::Process()
{
	if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(AnimationAssetPtr.Get()))
	{
		ProcessRootTransform(BlendSpace, BlendParameters, CachedPlayLength, RootTransformSamplingRate, IsLoopable(), AccumulatedRootTransform, bEnforceCompressedDataSampling);
	}
}

void FAnimationAssetSampler::ExtractAnimNotifyContext(float Time, FAnimNotifyContext& PreAllocatedNotifyContext) const
{
	float SampleTime = Time;

	// resetting PreAllocatedNotifyContext without losing ActiveNotifies allocation 
	PreAllocatedNotifyContext.TickRecord = nullptr;
	PreAllocatedNotifyContext.ActiveNotifies.Reset();

	if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(AnimationAssetPtr.Get()))
	{
		if (BlendSpace->NotifyTriggerMode == ENotifyTriggerMode::HighestWeightedAnimation)
		{
			// Set up blend samples
			TArray<FBlendSampleData> BlendSamples;
			int32 TriangulationIndex = 0;
			if (BlendSpace->GetSamplesFromBlendInput(BlendParameters, BlendSamples, TriangulationIndex, true))
			{
				// Find highest weighted
				const int32 HighestWeightIndex = GetHighestWeightSample(BlendSamples);
				const FBlendSampleData& BlendSample = BlendSamples[HighestWeightIndex];
				if (BlendSample.Animation)
				{
					// getting pose search notifies in an interval of size ExtractionInterval, centered on Time
					if (CachedPlayLength > UE_KINDA_SMALL_NUMBER)
					{
						SampleTime = Time * (BlendSample.Animation->GetPlayLength() / CachedPlayLength);
					}

					// Get notifies for highest weighted
					const float ExtractionStartTime = FMath::Min(SampleTime, BlendSample.Animation->GetPlayLength()) - (ExtractionInterval * 0.5f);
					BlendSample.Animation->GetAnimNotifies(ExtractionStartTime, ExtractionInterval, PreAllocatedNotifyContext);
				}
			}
		}
		else
		{
			UE_LOGF(LogPoseSearch, Error, "FAnimationAssetSampler::ExtractAnimNotifyContext: Unsupported BlendSpace NotifyTriggerMode for '%ls'", *BlendSpace->GetName());
		}
	}
	else if (const UAnimSequenceBase* SequenceBase = Cast<UAnimSequenceBase>(AnimationAssetPtr.Get()))
	{
		// getting pose search notifies in an interval of size ExtractionInterval, centered on Time
		const float ExtractionStartTime = FMath::Min(Time, SequenceBase->GetPlayLength()) - (ExtractionInterval * 0.5f);
		SequenceBase->GetAnimNotifies(ExtractionStartTime, ExtractionInterval, PreAllocatedNotifyContext);
	}
	else
	{
		checkNoEntry();
	}
}

void FAnimationAssetSampler::ExtractAnimNotifyStates(float Time, FAnimNotifyContext& PreAllocatedNotifyContext, const TFunction<bool(UAnimNotifyState*)>& ProcessAnimNotifyState) const
{
	ExtractAnimNotifyContext(Time, PreAllocatedNotifyContext);

	// check which notifies actually overlap Time and are of the right base type
	for (const FAnimNotifyEventReference& EventReference : PreAllocatedNotifyContext.ActiveNotifies)
	{
		if (const FAnimNotifyEvent* NotifyEvent = EventReference.GetNotify())
		{
			if (NotifyEvent->NotifyStateClass && !ProcessAnimNotifyState(NotifyEvent->NotifyStateClass))
			{
				break;
			}
		}
	}
}

void FAnimationAssetSampler::ExtractAnimNotifyEvents(float Time, FAnimNotifyContext& PreAllocatedNotifyContext, const TFunction<bool(const FAnimNotifyEvent*)>& ProcessAnimNotifyEvent) const
{
	ExtractAnimNotifyContext(Time, PreAllocatedNotifyContext);

	// check which notifies actually overlap Time and are of the right base type
	for (const FAnimNotifyEventReference& EventReference : PreAllocatedNotifyContext.ActiveNotifies)
	{
		if (const FAnimNotifyEvent* NotifyEvent = EventReference.GetNotify())
		{
			if (!ProcessAnimNotifyEvent(NotifyEvent))
			{
				break;
			}
		}
	}
}

TConstArrayView<FAnimNotifyEvent> FAnimationAssetSampler::GetAllAnimNotifyEvents() const
{
	if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(AnimationAssetPtr.Get()))
	{
		if (BlendSpace->NotifyTriggerMode == ENotifyTriggerMode::HighestWeightedAnimation)
		{
			TArray<FBlendSampleData> BlendSamples;
			int32 TriangulationIndex = 0;
			if (BlendSpace->GetSamplesFromBlendInput(BlendParameters, BlendSamples, TriangulationIndex, true))
			{
				// Find highest weighted
				const int32 HighestWeightIndex = GetHighestWeightSample(BlendSamples);
				const FBlendSampleData& BlendSample = BlendSamples[HighestWeightIndex];
				if (BlendSample.Animation)
				{
					return BlendSample.Animation->Notifies;
				}
			}
		}
		else
		{
			UE_LOGF(LogPoseSearch, Error, "FAnimationAssetSampler::GetAllAnimNotifyEvents: Unsupported BlendSpace NotifyTriggerMode for '%ls'", *BlendSpace->GetName());
		}
	}
	else if (const UAnimSequenceBase* SequenceBase = Cast<UAnimSequenceBase>(AnimationAssetPtr.Get()))
	{
		return SequenceBase->Notifies;
	}

	return TConstArrayView<FAnimNotifyEvent>(); 
}

} // namespace UE::PoseSearch
