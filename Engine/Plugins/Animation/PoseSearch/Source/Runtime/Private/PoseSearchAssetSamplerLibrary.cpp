// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchAssetSamplerLibrary.h"
#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "DrawDebugHelpers.h"
#include "Engine/SkinnedAsset.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "PoseSearch/PoseSearchMirrorDataCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchAssetSamplerLibrary)

FPoseSearchAssetSamplerPose UPoseSearchAssetSamplerLibrary::SamplePose(const FPoseSearchAssetSamplerInput Input, const UAnimInstance* AnimInstance)
{
	using namespace UE::PoseSearch;
	if (!Input.Animation)
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchAssetSamplerLibrary::SamplePose invalid Input.Animation");
		return FPoseSearchAssetSamplerPose();
	}
	
	if (Input.bMirrored && !Input.MirrorDataTable)
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchAssetSamplerLibrary::SamplePose unable to mirror the pose from %ls at time %f because of invalid MirrorDataTable", *Input.Animation->GetName(), Input.AnimationTime);
		return FPoseSearchAssetSamplerPose();
	}

	if (AnimInstance)
	{
		return SamplePose(AnimInstance->GetRequiredBonesOnAnyThread(), Input);
	}
	
	const USkeleton* Skeleton = Input.Animation->GetSkeleton();
	TArray<uint16> BoneIndices;

	const int32 NumBones = Skeleton->GetReferenceSkeleton().GetNum();
	BoneIndices.SetNum(NumBones);
	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		BoneIndices[BoneIndex] = BoneIndex;
	}

	FBoneContainer BoneContainer;
	BoneContainer.InitializeTo(BoneIndices, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *Skeleton);

	return SamplePose(BoneContainer, Input);
}

FPoseSearchAssetSamplerPose UPoseSearchAssetSamplerLibrary::SamplePose(const FBoneContainer& BoneContainer, const FPoseSearchAssetSamplerInput Input)
{
	using namespace UE::PoseSearch;

	FMemMark Mark(FMemStack::Get());

	// @todo: should we expose those parameters?
	constexpr bool bPreProcessRootTransform = true;
	constexpr bool bEnforceCompressedDataSampling = false;
	constexpr bool bUseRAWData = false;

	const FAnimationAssetSampler Sampler(Input.Animation, Input.RootTransformOrigin, Input.BlendParameters, FAnimationAssetSampler::DefaultRootTransformSamplingRate, bPreProcessRootTransform, bEnforceCompressedDataSampling);

	FPoseSearchAssetSamplerPose AssetSamplerPose;
	AssetSamplerPose.BoneContainer = BoneContainer;
	AssetSamplerPose.BoneContainer.SetUseRAWData(bUseRAWData);

	FBlendedCurve Curve;
	FCompactPose Pose;
	Pose.SetBoneContainer(&AssetSamplerPose.BoneContainer);
	Sampler.ExtractPose(Input.AnimationTime, Pose, Curve);
	AssetSamplerPose.RootTransform = Sampler.ExtractRootTransform(Input.AnimationTime);

	if (Input.bMirrored)
	{
		const FMirrorDataCache MirrorDataCache(Input.MirrorDataTable, AssetSamplerPose.BoneContainer);
		MirrorDataCache.MirrorPose(Pose);
		AssetSamplerPose.RootTransform = MirrorDataCache.MirrorTransform(AssetSamplerPose.RootTransform);
	}

	AssetSamplerPose.Pose.CopyBonesFrom(Pose);
	AssetSamplerPose.ComponentSpacePose.InitPose(AssetSamplerPose.Pose);
	check(AssetSamplerPose.ComponentSpacePose.GetPose().IsValid());

	return AssetSamplerPose;
}

FTransform UPoseSearchAssetSamplerLibrary::GetTransform(FPoseSearchAssetSamplerPose& AssetSamplerPose, FCompactPoseBoneIndex CompactPoseBoneIndex, EPoseSearchAssetSamplerSpace Space)
{
	return GetTransform(AssetSamplerPose.ComponentSpacePose, AssetSamplerPose.RootTransform, CompactPoseBoneIndex, Space);
}

FTransform UPoseSearchAssetSamplerLibrary::GetTransformByName(UPARAM(ref) FPoseSearchAssetSamplerPose& AssetSamplerPose, FName BoneName, EPoseSearchAssetSamplerSpace Space)
{
	if (!AssetSamplerPose.Pose.IsValid())
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchAssetSamplerLibrary::GetTransformByName invalid AssetSamplerPose.Pose");
		return FTransform::Identity;
	}

	const FBoneContainer& BoneContainer = AssetSamplerPose.Pose.GetBoneContainer();
	const USkeleton* Skeleton = BoneContainer.GetSkeletonAsset();

	FBoneReference BoneReference;
	BoneReference.BoneName = BoneName;
	BoneReference.Initialize(Skeleton);
	if (!BoneReference.HasValidSetup())
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchAssetSamplerLibrary::GetTransformByName invalid BoneName %ls for Skeleton %ls", *BoneName.ToString(), *GetNameSafe(Skeleton));
		return FTransform::Identity;
	}

	const FCompactPoseBoneIndex CompactPoseBoneIndex = BoneContainer.GetCompactPoseIndexFromSkeletonPoseIndex(FSkeletonPoseBoneIndex(BoneReference.BoneIndex));
	if (!CompactPoseBoneIndex.IsValid())
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchAssetSamplerLibrary::GetTransformByName invalid FCompactPoseBoneIndex for BoneName %ls for Skeleton %ls", *BoneName.ToString(), *GetNameSafe(Skeleton));
		return FTransform::Identity;
	}

	return GetTransform(AssetSamplerPose, CompactPoseBoneIndex, Space);
}

void UPoseSearchAssetSamplerLibrary::Draw(const UAnimInstance* AnimInstance, UPARAM(ref) FPoseSearchAssetSamplerPose& AssetSamplerPose)
{
	check(IsInGameThread());

#if ENABLE_DRAW_DEBUG
	static float DebugDrawSamplerRootAxisLength = 20.f;
	static float DebugDrawSamplerSize = 6.f;

	if (AnimInstance)
	{
		Draw(AnimInstance->GetWorld(), AssetSamplerPose.ComponentSpacePose, AssetSamplerPose.RootTransform);
	}
#endif // ENABLE_DRAW_DEBUG
}

#if ENABLE_VISUAL_LOG
void UPoseSearchAssetSamplerLibrary::VLogDraw(const UObject* VLogContext, const USkeletalMeshComponent* Mesh, const TCHAR* VLogName, const FColor Color, float DebugDrawSamplerRootAxisLength)
{
	check(IsInGameThread());

	if (!Mesh)
	{
		return;
	}
		
	const USkinnedAsset* SkinnedAsset = Mesh->GetSkinnedAsset();
	if (!SkinnedAsset)
	{
		return;
	}
	
	if (DebugDrawSamplerRootAxisLength > 0.f)
	{
		const FTransform& AxisWorldTransform = Mesh->GetComponentTransform();
		UE_VLOG_SEGMENT(VLogContext, VLogName, Display, AxisWorldTransform.GetTranslation(), AxisWorldTransform.GetTranslation() + AxisWorldTransform.GetScaledAxis(EAxis::X) * DebugDrawSamplerRootAxisLength, FColor::Red, TEXT(""));
		UE_VLOG_SEGMENT(VLogContext, VLogName, Display, AxisWorldTransform.GetTranslation(), AxisWorldTransform.GetTranslation() + AxisWorldTransform.GetScaledAxis(EAxis::Y) * DebugDrawSamplerRootAxisLength, FColor::Green, TEXT(""));
		UE_VLOG_SEGMENT(VLogContext, VLogName, Display, AxisWorldTransform.GetTranslation(), AxisWorldTransform.GetTranslation() + AxisWorldTransform.GetScaledAxis(EAxis::Z) * DebugDrawSamplerRootAxisLength, FColor::Blue, TEXT(""));
	}

	const int NumBones = Mesh->GetNumBones();
	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		const int32 ParentBoneIndex = SkinnedAsset->GetRefSkeleton().GetParentIndex(BoneIndex);
		if (ParentBoneIndex != INDEX_NONE)
		{
			const FTransform BoneWorldTransform = Mesh->GetBoneTransform(BoneIndex);
			const FTransform ParentBoneWorldTransform = Mesh->GetBoneTransform(ParentBoneIndex);
			UE_VLOG_SEGMENT(VLogContext, VLogName, Display, BoneWorldTransform.GetTranslation(), ParentBoneWorldTransform.GetTranslation(), Color, TEXT(""));
		}
	}
}
#endif // ENABLE_VISUAL_LOG


