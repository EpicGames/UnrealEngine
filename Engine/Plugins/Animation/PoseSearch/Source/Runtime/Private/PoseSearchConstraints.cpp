// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchConstraints.h"
#include "PoseSearch/PoseSearchAnimNotifies.h"

void FPoseSearchConstraint::Initialize(const UAnimNotifyState_PoseSearchConstraint* ConstraintNotifyState)
{
	FromSocketName = ConstraintNotifyState->FromSocketName;
	FromSocketRole = ConstraintNotifyState->FromSocketRole;
	ToSocketName = ConstraintNotifyState->ToSocketName;
	ToSocketRole = ConstraintNotifyState->ToSocketRole;
	TranslationWeight = ConstraintNotifyState->TranslationWeight;
	RotationWeight = ConstraintNotifyState->RotationWeight;
	RampUpTime = ConstraintNotifyState->RampUpTime;
	CoolDownTime = ConstraintNotifyState->CoolDownTime;
}

const FTransform* FPoseSearchConstraint::GetSocketTransform(UE::PoseSearch::FRole Role, FName SocketName) const
{
	if (FromSocketRole == Role && FromSocketName == SocketName)
	{
		return &FromSocketTransform;
	}

	if (ToSocketRole == Role && ToSocketName == SocketName)
	{
		return &ToSocketTransform;
	}

	return nullptr;
}