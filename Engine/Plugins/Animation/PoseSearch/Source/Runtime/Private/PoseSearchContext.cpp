// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchContext.h"
#include "Animation/AnimInstanceProxy.h"
#include "AnimationRuntime.h"
#include "BoneContainer.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchFeatureChannel_Heading.h"
#include "PoseSearch/PoseSearchFeatureChannel_PermutationTime.h"
#include "PoseSearch/PoseSearchFeatureChannel_Position.h"
#include "PoseSearch/PoseSearchHistory.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearchIndex.inl"

namespace UE::PoseSearch
{
	
const UObject* GetAnimContext(const FChooserEvaluationContext* Context)
{
	check(Context);
	return Context->GetFirstObjectParam();
}

const FTransform& GetContextTransform(const UObject* AnimContext, bool bCheckIsInGameThread)
{
	check(!bCheckIsInGameThread || IsInGameThread());

	if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(AnimContext))
	{
		return AnimInstance->GetSkelMeshComponent()->GetComponentTransform();
	}
	
	if (const UActorComponent* ActorComponent = Cast<UActorComponent>(AnimContext))
	{
		const AActor* Actor = ActorComponent->GetOwner();
		check(Actor);
		// @todo: this code depends on how AnimNext gather its context object, and will likely change in the future
		if (const USkeletalMeshComponent* SkeletalMeshComponent = Actor->GetComponentByClass<USkeletalMeshComponent>())
		{
			return SkeletalMeshComponent->GetComponentTransform();
		}

		return ActorComponent->GetOwner()->GetTransform();
	}

	unimplemented();
	return FTransform::Identity;
}

const USkeleton* GetContextSkeleton(const UObject* AnimContext, bool bCheckIsInGameThread)
{
	check(!bCheckIsInGameThread || IsInGameThread());

	const USkeleton* Skeleton = nullptr;
	if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(AnimContext))
	{
		Skeleton = AnimInstance->GetRequiredBonesOnAnyThread().GetSkeletonAsset();
	}
	else if (const UActorComponent* ActorComponent = Cast<UActorComponent>(AnimContext))
	{
		const AActor* Actor = ActorComponent->GetOwner();
		check(Actor);
		// @todo: this code depends on how AnimNext gather its context object, and will likely change in the future
		if (const USkeletalMeshComponent* SkeletalMeshComponent = Actor->GetComponentByClass<USkeletalMeshComponent>())
		{
			if (const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset())
			{
				Skeleton = SkeletalMesh->GetSkeleton();
			}
		}
	}

	check(Skeleton)
	return Skeleton;
}
	
const USkeletalMeshComponent* GetContextSkeletalMeshComponent(const UObject* AnimContext, bool bCheckIsInGameThread)
{
	check(!bCheckIsInGameThread || IsInGameThread());

	const USkeletalMeshComponent* SkeletalMeshComponent = nullptr;
	if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(AnimContext))
	{
		SkeletalMeshComponent = AnimInstance->GetSkelMeshComponent();
	}
	else if (const UActorComponent* ActorComponent = Cast<UActorComponent>(AnimContext))
	{
		const AActor* Actor = ActorComponent->GetOwner();
		check(Actor);
		// @todo: this code depends on how AnimNext gather its context object, and will likely change in the future
		SkeletalMeshComponent = Actor->GetComponentByClass<USkeletalMeshComponent>();
	}

	check(SkeletalMeshComponent);
	return SkeletalMeshComponent;
}

const FBoneContainer GetBoneContainer(const UObject* AnimContext, bool bCheckIsInGameThread)
{
	check(!bCheckIsInGameThread || IsInGameThread());

	if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(AnimContext))
	{
		return AnimInstance->GetRequiredBonesOnAnyThread();
	}

	if (const UActorComponent* ActorComponent = Cast<UActorComponent>(AnimContext))
	{
		const AActor* Actor = ActorComponent->GetOwner();
		check(Actor);
		// @todo: this code depends on how AnimNext gather its context object, and will likely change in the future
		if (const USkeletalMeshComponent* SkeletalMeshComponent = Actor->GetComponentByClass<USkeletalMeshComponent>())
		{
			if (USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset())
			{
				return FBoneContainer(SkeletalMeshComponent->RequiredBones, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *SkeletalMesh->GetSkeleton());
			}
		}
		return FBoneContainer();
	}

	unimplemented();
	return FBoneContainer();
}

const AActor* GetContextOwningActor(const UObject* AnimContext, bool bCheckIsInGameThread)
{
	check(!bCheckIsInGameThread || IsInGameThread());

	if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(AnimContext))
	{
		return AnimInstance->GetOwningActor();
	}
	
	if (const UActorComponent* AnimNextComponent = Cast<UActorComponent>(AnimContext))
	{
		return AnimNextComponent->GetOwner();
	}
	
	return nullptr;
}

FVector GetContextLocation(const UObject* AnimContext, bool bCheckIsInGameThread)
{
	return GetContextTransform(AnimContext, bCheckIsInGameThread).GetLocation();
}

#if ENABLE_DRAW_DEBUG

class UAnimInstanceProxyProvider : public UAnimInstance
{
public:
	static FAnimInstanceProxy* GetAnimInstanceProxy(const UAnimInstance* AnimInstance)
	{
		if (AnimInstance)
		{
			// const cast is ok, since UAnimInstance::AnimInstanceProxy is mutable
			return &static_cast<UAnimInstanceProxyProvider*>(const_cast<UAnimInstance*>(AnimInstance))->GetProxyOnAnyThread<FAnimInstanceProxy>();
		}
		return nullptr;
	}
};

static FAnimInstanceProxy* GetAnimInstanceProxy(TConstArrayView<FChooserEvaluationContext*> Contexts)
{
	if (!Contexts.IsEmpty())
	{
		return UAnimInstanceProxyProvider::GetAnimInstanceProxy(Cast<UAnimInstance>(GetAnimContext(Contexts[0])));
	}
	return nullptr;
}

static const USkinnedMeshComponent* GetMesh(TConstArrayView<FChooserEvaluationContext*> Contexts, int32 RoleIndex = 0)
{
	return Cast<USkinnedMeshComponent>(GetAnimContext(Contexts[RoleIndex]));
}

static const UWorld* GetWorld(TConstArrayView<FChooserEvaluationContext*> AnimContexts)
{
	const USkinnedMeshComponent* SkinnedMeshComponent = GetMesh(AnimContexts);
	check(SkinnedMeshComponent);
	return SkinnedMeshComponent->GetWorld();
}

//////////////////////////////////////////////////////////////////////////
// FDebugDrawParams
FDebugDrawParams::FDebugDrawParams(TConstArrayView<FChooserEvaluationContext*> InAnimContexts, TConstArrayView<const IPoseHistory*> InPoseHistories, const FRoleToIndex& InRoleToIndex, const UPoseSearchDatabase* InDatabase, EDrawContext InDrawContext)
: FDebugDrawParams()
{
	Init(InAnimContexts, InPoseHistories, InRoleToIndex, InDatabase, InDrawContext);
}

FDebugDrawParams::FDebugDrawParams(const FDebugDrawParams& Other)
: AnimContexts(Other.AnimContexts)
, PoseHistories(Other.PoseHistories)
, RoleToIndex(Other.RoleToIndex)
, Database(Other.Database)
, DrawContext(Other.DrawContext)
, DynamicWeightsSqrtBuffer(Other.DynamicWeightsSqrtBuffer)
{
	// pointing DynamicWeightsSqrt this object internal DynamicWeightsSqrtBuffer memory buffer!
	DynamicWeightsSqrt = DynamicWeightsSqrtBuffer;
}

FDebugDrawParams::FDebugDrawParams(FDebugDrawParams&& Other)
: AnimContexts(MoveTemp(Other.AnimContexts))
, PoseHistories(MoveTemp(Other.PoseHistories))
, RoleToIndex(MoveTemp(Other.RoleToIndex))
, Database(Other.Database)
, DrawContext(Other.DrawContext)
, DynamicWeightsSqrtBuffer(MoveTemp(Other.DynamicWeightsSqrtBuffer))
{
	// pointing DynamicWeightsSqrt this object internal DynamicWeightsSqrtBuffer memory buffer!
	DynamicWeightsSqrt = DynamicWeightsSqrtBuffer;

	Other.Database = nullptr;
}

FDebugDrawParams& FDebugDrawParams::operator=(const FDebugDrawParams& Other)
{
	if (this != &Other)
	{
		this->~FDebugDrawParams();
		new(this) FDebugDrawParams(Other);
	}
	return *this;
}

FDebugDrawParams& FDebugDrawParams::operator=(FDebugDrawParams&& Other)
{
	if (this != &Other)
	{
		this->~FDebugDrawParams();
		new(this) FDebugDrawParams(MoveTemp(Other));
	}
	return *this;
}

void FDebugDrawParams::Init(TConstArrayView<FChooserEvaluationContext*> InAnimContexts, TConstArrayView<const IPoseHistory*> InPoseHistories, const FRoleToIndex& InRoleToIndex, const UPoseSearchDatabase* InDatabase, EDrawContext InDrawContext)
{
	AnimContexts = InAnimContexts;
	PoseHistories = InPoseHistories;
	RoleToIndex = InRoleToIndex;
	Database = InDatabase;
	DrawContext = InDrawContext;

	check(RoleToIndex.Num() == PoseHistories.Num());
	check(IsValid(RoleToIndex));

	if (const UPoseSearchSchema* Schema = GetSchema())
	{
		DynamicWeightsSqrtBuffer.SetNum(Schema->SchemaCardinality);
		DynamicWeightsSqrt = Database->CalculateDynamicWeightsSqrt(DynamicWeightsSqrtBuffer);
	}
}

bool FDebugDrawParams::CanDraw() const
{
	return !AnimContexts.IsEmpty() && !RoleToIndex.IsEmpty() && Database && Database->Schema;
}

const FSearchIndex* FDebugDrawParams::GetSearchIndex() const
{
	return Database ? &Database->GetSearchIndex() : nullptr;
}

const UPoseSearchSchema* FDebugDrawParams::GetSchema() const
{
	return Database ? Database->Schema : nullptr;
}

float FDebugDrawParams::ExtractPermutationTime(TConstArrayView<float> PoseVector) const
{
	if (const UPoseSearchSchema* Schema = GetSchema())
	{
		if (const UPoseSearchFeatureChannel_PermutationTime* FoundPermutationTime = static_cast<const UPoseSearchFeatureChannel_PermutationTime*>(
			Schema->FindChannel([](const UPoseSearchFeatureChannel* Channel) -> const UPoseSearchFeatureChannel_PermutationTime*
				{
					if (const UPoseSearchFeatureChannel_PermutationTime* PermutationTime = Cast<UPoseSearchFeatureChannel_PermutationTime>(Channel))
					{
						return PermutationTime;
					}
					return nullptr;
				})))
		{
			check(FoundPermutationTime->GetChannelCardinality() == 1);
			return FFeatureVectorHelper::DecodeFloat(PoseVector, FoundPermutationTime->GetChannelDataOffset());
		}
	}
	return 0.f;
}

FVector FDebugDrawParams::ExtractPosition(TConstArrayView<float> PoseVector, float SampleTimeOffset, int8 SchemaBoneIdx, const FRole& Role, EPermutationTimeType PermutationTimeType, int32 SamplingAttributeId, float PermutationSampleTimeOffset) const
{
	// we don't wanna ask for a SchemaOriginBoneIdx in the future or past
	check(PermutationTimeType != EPermutationTimeType::UsePermutationTime);
	if (const UPoseSearchSchema* Schema = GetSchema())
	{
		// looking for a UPoseSearchFeatureChannel_Position that matches the TimeOffset and SchemaBoneIdx,
		// with SchemaOriginBoneIdx to be the root bone and the appropriate PermutationTimeType 
		if (const UPoseSearchFeatureChannel_Position* FoundPosition = static_cast<const UPoseSearchFeatureChannel_Position*>(
			Schema->FindChannel([SampleTimeOffset, SchemaBoneIdx, &Role, PermutationTimeType, SamplingAttributeId](const UPoseSearchFeatureChannel* Channel) -> const UPoseSearchFeatureChannel_Position*
				{
					if (const UPoseSearchFeatureChannel_Position* Position = Cast<UPoseSearchFeatureChannel_Position>(Channel))
					{
						if (Position->SchemaBoneIdx == SchemaBoneIdx &&
							Position->SampleTimeOffset == SampleTimeOffset &&
							Position->OriginTimeOffset == 0.f &&
							Position->PermutationTimeType == PermutationTimeType &&
							Position->SamplingAttributeId == SamplingAttributeId &&
							Position->SchemaOriginBoneIdx == RootSchemaBoneIdx &&
							Position->SampleRole == Role &&
							Position->OriginRole == Role)
						{
							return Position;
						}
					}
					return nullptr;
				})))
		{
			const FVector BonePosition = FFeatureVectorHelper::DecodeVector(PoseVector, FoundPosition->GetChannelDataOffset(), FoundPosition->ComponentStripping);
			const FVector WorldBonePosition = GetRootBoneTransform(FoundPosition->SampleRole).TransformPosition(BonePosition);
			return WorldBonePosition;
		}

		if (const int32* RoleIndex = RoleToIndex.Find(Role))
		{
			const IPoseHistory* PoseHistory = PoseHistories[*RoleIndex];
			check(PoseHistory);

			if (const USkeleton* Skeleton = Schema->GetSkeleton(Role))
			{
				const FBoneIndexType BoneIndexType = SchemaBoneIdx == TrajectorySchemaBoneIdx ? ComponentSpaceIndexType : Schema->GetBoneReferences(Role)[SchemaBoneIdx].BoneIndex;

				FTransform WorldBoneTransform;
				if (PoseHistory->GetTransformAtTime(SampleTimeOffset + PermutationSampleTimeOffset, WorldBoneTransform, Skeleton, BoneIndexType, WorldSpaceIndexType))
				{
					return WorldBoneTransform.GetTranslation();
				}
			}

			if (SchemaBoneIdx > RootSchemaBoneIdx)
			{
				if (const USkinnedMeshComponent* Mesh = GetMesh(AnimContexts, *RoleIndex))
				{
					return Mesh->GetSocketTransform(Schema->GetBoneReferences(Role)[SchemaBoneIdx].BoneName).GetTranslation();
				}
			}
		}
	}
	return GetRootBoneTransform(Role, SampleTimeOffset + PermutationSampleTimeOffset).GetTranslation();
}

FQuat FDebugDrawParams::ExtractRotation(TConstArrayView<float> PoseVector, float SampleTimeOffset, int8 SchemaBoneIdx, const FRole& Role, EPermutationTimeType PermutationTimeType, int32 SamplingAttributeId, float PermutationSampleTimeOffset) const
{
	// we don't wanna ask for a SchemaOriginBoneIdx in the future or past
	check(PermutationTimeType != EPermutationTimeType::UsePermutationTime);
	if (const UPoseSearchSchema* Schema = GetSchema())
	{
		int32 HeadingAxisFoundNum = 0;
		const UPoseSearchFeatureChannel_Heading* FoundHeading[int32(EHeadingAxis::Num)] = { nullptr };
		FVector DecodedHeading[int32(EHeadingAxis::Num)] = { FVector::ZeroVector };
		for (int32 HeadingAxis = 0; HeadingAxis < int32(EHeadingAxis::Num); ++HeadingAxis)
		{
			// looking for a UPoseSearchFeatureChannel_Heading that matches the SampleTimeOffset, SchemaBoneIdx, and with OriginTimeOffset as zero.
			// the features data associated to this channel would be a heading vector in GetRootTransform space (since OriginTimeOffset is zero)), 
			// so by finding at least two with differnt axis we'll be able to compose a delta rotation from OriginTimeOffset (zero) to SampleTimeOffset
			Schema->FindChannel([SampleTimeOffset, SchemaBoneIdx, &Role, PermutationTimeType, SamplingAttributeId, HeadingAxis, PoseVector, &FoundHeading, &DecodedHeading](const UPoseSearchFeatureChannel* Channel) -> const UPoseSearchFeatureChannel_Heading*
				{
					const UPoseSearchFeatureChannel_Heading* Heading = Cast<UPoseSearchFeatureChannel_Heading>(Channel);
					if (Heading &&
						Heading->SchemaBoneIdx == SchemaBoneIdx &&
						Heading->SampleTimeOffset == SampleTimeOffset &&
						Heading->OriginTimeOffset == 0.f &&
						Heading->PermutationTimeType == PermutationTimeType &&
						Heading->SamplingAttributeId == SamplingAttributeId &&
						Heading->SchemaOriginBoneIdx == RootSchemaBoneIdx &&
						Heading->SampleRole == Role &&
						Heading->OriginRole == Role &&
						int32(Heading->HeadingAxis) == HeadingAxis)
					{
						FVector DecodedHeadingValue = FFeatureVectorHelper::DecodeVector(PoseVector, Heading->GetChannelDataOffset(), Heading->ComponentStripping);
						if (DecodedHeadingValue.Normalize())
						{
							FoundHeading[HeadingAxis] = Heading;
							DecodedHeading[HeadingAxis] = DecodedHeadingValue;
							return Heading;
						}
					}
					return nullptr;
				});

			if (FoundHeading[HeadingAxis])
			{
				++HeadingAxisFoundNum;
				if (HeadingAxisFoundNum == 2)
				{
					// we've found enough heading axis to compose a rotation
					break;
				}
			}
		}

		if (HeadingAxisFoundNum > 0)
		{
			bool bAbleToReconstructMissingAxis = true;
			if (HeadingAxisFoundNum == 2)
			{
				// reconstructing the missing axis
				if (!FoundHeading[int32(EHeadingAxis::X)])
				{
					DecodedHeading[int32(EHeadingAxis::X)] = FVector::CrossProduct(DecodedHeading[int32(EHeadingAxis::Y)], DecodedHeading[int32(EHeadingAxis::Z)]);
					bAbleToReconstructMissingAxis &= DecodedHeading[int32(EHeadingAxis::X)].Normalize();
				}
				else if (!FoundHeading[int32(EHeadingAxis::Y)])
				{
					DecodedHeading[int32(EHeadingAxis::Y)] = FVector::CrossProduct(DecodedHeading[int32(EHeadingAxis::Z)], DecodedHeading[int32(EHeadingAxis::X)]);
					bAbleToReconstructMissingAxis &= DecodedHeading[int32(EHeadingAxis::Y)].Normalize();
				}
				else // if (!FoundHeading[int32(EHeadingAxis::Z)])
				{
					DecodedHeading[int32(EHeadingAxis::Z)] = FVector::CrossProduct(DecodedHeading[int32(EHeadingAxis::X)], DecodedHeading[int32(EHeadingAxis::Y)]);
					bAbleToReconstructMissingAxis &= DecodedHeading[int32(EHeadingAxis::Z)].Normalize();
				}
			}
			else 
			{
				check(HeadingAxisFoundNum == 1);
			
				// reconstructing the two missing axis
				if (FoundHeading[int32(EHeadingAxis::X)])
				{
					DecodedHeading[int32(EHeadingAxis::Y)] = FVector::CrossProduct(FVector::ZAxisVector, DecodedHeading[int32(EHeadingAxis::X)]);
					bAbleToReconstructMissingAxis &= DecodedHeading[int32(EHeadingAxis::Y)].Normalize();
					DecodedHeading[int32(EHeadingAxis::Z)] = FVector::CrossProduct(DecodedHeading[int32(EHeadingAxis::X)], DecodedHeading[int32(EHeadingAxis::Y)]);
					bAbleToReconstructMissingAxis &= DecodedHeading[int32(EHeadingAxis::Z)].Normalize();
				}
				else if (FoundHeading[int32(EHeadingAxis::Y)])
				{
					DecodedHeading[int32(EHeadingAxis::X)] = FVector::CrossProduct(DecodedHeading[int32(EHeadingAxis::Y)], FVector::ZAxisVector);
					bAbleToReconstructMissingAxis &= DecodedHeading[int32(EHeadingAxis::X)].Normalize();
					DecodedHeading[int32(EHeadingAxis::Z)] = FVector::CrossProduct(DecodedHeading[int32(EHeadingAxis::X)], DecodedHeading[int32(EHeadingAxis::Y)]);
					bAbleToReconstructMissingAxis &= DecodedHeading[int32(EHeadingAxis::Z)].Normalize();
				}
				else // if (FoundHeading[int32(EHeadingAxis::Z)])
				{
					DecodedHeading[int32(EHeadingAxis::X)] = FVector::CrossProduct(FVector::YAxisVector, DecodedHeading[int32(EHeadingAxis::Z)]);
					bAbleToReconstructMissingAxis &= DecodedHeading[int32(EHeadingAxis::X)].Normalize();
					DecodedHeading[int32(EHeadingAxis::Y)] = FVector::CrossProduct(DecodedHeading[int32(EHeadingAxis::Z)], DecodedHeading[int32(EHeadingAxis::X)]);
					bAbleToReconstructMissingAxis &= DecodedHeading[int32(EHeadingAxis::Y)].Normalize();
				}
			}

			if (bAbleToReconstructMissingAxis)
			{
				// RotMatrix is the rotation matrix from time zero (OriginTimeOffset) to time SampleTimeOffset, so by composing it with GetRootTransform().GetRotation(),
				// world rotation associated to the time zero, we can calcualte the world rotation at time SampleTimeOffset
				const FMatrix RotMatrix(DecodedHeading[int32(EHeadingAxis::X)], DecodedHeading[int32(EHeadingAxis::Y)], DecodedHeading[int32(EHeadingAxis::Z)], FVector::ZeroVector);
				const FQuat RotQuat(RotMatrix);
				const FQuat RotQuatWorld = RotQuat * GetRootBoneTransform(Role).GetRotation();
				return RotQuatWorld;
			}
		}

		if (const int32* RoleIndex = RoleToIndex.Find(Role))
		{
			const IPoseHistory* PoseHistory = PoseHistories[*RoleIndex];
			check(PoseHistory);

			if (const USkeleton* Skeleton = Schema->GetSkeleton(Role))
			{
				const FBoneIndexType BoneIndexType = SchemaBoneIdx == TrajectorySchemaBoneIdx ? ComponentSpaceIndexType : Schema->GetBoneReferences(Role)[SchemaBoneIdx].BoneIndex;

				FTransform WorldBoneTransform;
				if (PoseHistory->GetTransformAtTime(SampleTimeOffset + PermutationSampleTimeOffset, WorldBoneTransform, Skeleton, BoneIndexType, WorldSpaceIndexType))
				{
					return WorldBoneTransform.GetRotation();
				}
			}

			if (SchemaBoneIdx > RootSchemaBoneIdx)
			{
				if (const USkinnedMeshComponent* Mesh = GetMesh(AnimContexts, *RoleIndex))
				{
					return Mesh->GetSocketTransform(Schema->GetBoneReferences(Role)[SchemaBoneIdx].BoneName).GetRotation();
				}
			}
		}
	}

	return GetRootBoneTransform(Role, SampleTimeOffset + PermutationSampleTimeOffset).GetRotation();
}

FTransform FDebugDrawParams::GetRootBoneTransform(const FRole& Role, float SampleTimeOffset) const
{
	FTransform RootBoneTransform = FTransform::Identity;

	if (const int32* RoleIndex = RoleToIndex.Find(Role))
	{
		PoseHistories[*RoleIndex]->GetTransformAtTime(SampleTimeOffset, RootBoneTransform, nullptr, RootBoneIndexType, WorldSpaceIndexType);
	}
	return RootBoneTransform;
}

void FDebugDrawParams::DrawLine(const FVector& LineStart, const FVector& LineEnd, const FColor& Color, float Thickness) const
{
	if (Color.A > 0)
	{
		if (FAnimInstanceProxy* AnimInstanceProxy = GetAnimInstanceProxy(AnimContexts))
		{
			AnimInstanceProxy->AnimDrawDebugLine(LineStart, LineEnd, Color, false, 0.f, Thickness, SDPG_Foreground);
		}
		else if (const UWorld* World = GetWorld(AnimContexts))
		{
			// any Mesh is fine to draw
			DrawDebugLine(World, LineStart, LineEnd, Color, false, 0.f, SDPG_Foreground, Thickness);
		}
	}
}

void FDebugDrawParams::DrawPoint(const FVector& Position, const FColor& Color, float Thickness) const
{
	if (Color.A > 0)
	{
		if (FAnimInstanceProxy* AnimInstanceProxy = GetAnimInstanceProxy(AnimContexts))
		{
			AnimInstanceProxy->AnimDrawDebugPoint(Position, Thickness, Color, false, 0.f, SDPG_Foreground);
		}
		else if (const UWorld* World = GetWorld(AnimContexts))
		{
			// any Mesh is fine to draw
			DrawDebugPoint(World, Position, Thickness, Color, false, 0.f, SDPG_Foreground);
		}
	}
}

void FDebugDrawParams::DrawCircle(const FVector& Center, const FVector& UpVector, float Radius, int32 Segments, const FColor& Color, float Thickness) const
{
	FVector A = UpVector;
	if (A.Normalize())
	{
		FMatrix TransformMatrix;
		FVector B = A.Cross(FVector::ZAxisVector);
		if (B.Normalize())
		{
			const FVector C = A.Cross(B);
			TransformMatrix = FMatrix(A, B, C, Center);
		}
		else
		{
			TransformMatrix = FMatrix(FVector::ZAxisVector, FVector::XAxisVector, FVector::YAxisVector, Center);
		}

		DrawCircle(TransformMatrix, Radius, Segments, Color, Thickness);
	}
}

void FDebugDrawParams::DrawCircle(const FMatrix& TransformMatrix, float Radius, int32 Segments, const FColor& Color, float Thickness) const
{
	if (Color.A > 0)
	{
		if (FAnimInstanceProxy* AnimInstanceProxy = GetAnimInstanceProxy(AnimContexts))
		{
			AnimInstanceProxy->AnimDrawDebugCircle(TransformMatrix.GetOrigin(), Radius, Segments, Color, TransformMatrix.GetScaledAxis(EAxis::X), false, 0.f, SDPG_Foreground, Thickness);
		}
		else if (const UWorld* World = GetWorld(AnimContexts))
		{
			// any Mesh is fine to draw
			DrawDebugCircle(World, TransformMatrix, Radius, Segments, Color, false, 0.f, SDPG_Foreground, Thickness, false);
		}
	}
}

void FDebugDrawParams::DrawWedge(const FVector& Origin, const FVector& Direction, float InnerRadius, float OuterRadius, float WidthInDegrees, int32 Segments, const FColor& Color, float Thickness) const
{
	FVector NormalizedDirection = Direction;
	if (Color.A > 0 && Segments > 1 && NormalizedDirection.Normalize())
	{
		float AngleDeg = -WidthInDegrees * 0.5f;
		const float AngleDegIncrement = WidthInDegrees / (Segments - 1);

		FVector PrevDirection = NormalizedDirection.RotateAngleAxis(AngleDeg, FVector::ZAxisVector);
		DrawLine(PrevDirection * OuterRadius + Origin, PrevDirection * InnerRadius + Origin, Color);

		for (int32 Segment = 1; Segment < Segments; ++Segment)
		{
			AngleDeg += AngleDegIncrement;

			const FVector CurrDirection = NormalizedDirection.RotateAngleAxis(AngleDeg, FVector::ZAxisVector);
			
			DrawLine(PrevDirection * InnerRadius + Origin, CurrDirection * InnerRadius + Origin, Color);
			DrawLine(PrevDirection * OuterRadius + Origin, CurrDirection * OuterRadius + Origin, Color);

			PrevDirection = CurrDirection;
		}

		DrawLine(PrevDirection * OuterRadius + Origin, PrevDirection * InnerRadius + Origin, Color);
	}
}

void FDebugDrawParams::DrawSphere(const FVector& Center, float Radius, int32 Segments, const FColor& Color, float Thickness) const
{
	if (Color.A > 0)
	{
		if (FAnimInstanceProxy* AnimInstanceProxy = GetAnimInstanceProxy(AnimContexts))
		{
			AnimInstanceProxy->AnimDrawDebugSphere(Center, Radius, Segments, Color, false, 0.f, Thickness, SDPG_Foreground);
		}
		else if (const UWorld* World = GetWorld(AnimContexts))
		{
			// any Mesh is fine to draw
			DrawDebugSphere(World, Center, Radius, Segments, Color, false, 0.f, SDPG_Foreground, Thickness);
		}
	}
}

void FDebugDrawParams::DrawCentripetalCatmullRomSpline(TConstArrayView<FVector> Points, TConstArrayView<FColor> Colors, float Alpha, int32 NumSamplesPerSegment, float Thickness) const
{
	const int32 NumPoints = Points.Num();
	const int32 NumColors = Colors.Num();
	if (NumPoints > 1 && NumColors > 0)
	{
		auto GetT = [](float T, float Alpha, const FVector& P0, const FVector& P1)
		{
			const FVector P1P0 = P1 - P0;
			const float Dot = P1P0 | P1P0;
			const float Pow = FMath::Pow(Dot, Alpha * .5f);
			return Pow + T;
		};

		auto LerpColor = [](FColor A, FColor B, float T) -> FColor
		{
			return FColor(
				FMath::RoundToInt(float(A.R) * (1.f - T) + float(B.R) * T),
				FMath::RoundToInt(float(A.G) * (1.f - T) + float(B.G) * T),
				FMath::RoundToInt(float(A.B) * (1.f - T) + float(B.B) * T),
				FMath::RoundToInt(float(A.A) * (1.f - T) + float(B.A) * T));
		};

		FVector PrevPoint = Points[0];
		for (int32 i = 0; i < NumPoints - 1; ++i)
		{
			const FVector& P0 = Points[FMath::Max(i - 1, 0)];
			const FVector& P1 = Points[i];
			const FVector& P2 = Points[i + 1];
			const FVector& P3 = Points[FMath::Min(i + 2, NumPoints - 1)];

			const float T0 = 0.0f;
			const float T1 = GetT(T0, Alpha, P0, P1);
			const float T2 = GetT(T1, Alpha, P1, P2);
			const float T3 = GetT(T2, Alpha, P2, P3);

			const float T1T0 = T1 - T0;
			const float T2T1 = T2 - T1;
			const float T3T2 = T3 - T2;
			const float T2T0 = T2 - T0;
			const float T3T1 = T3 - T1;

			const bool bIsNearlyZeroT1T0 = FMath::IsNearlyZero(T1T0, UE_KINDA_SMALL_NUMBER);
			const bool bIsNearlyZeroT2T1 = FMath::IsNearlyZero(T2T1, UE_KINDA_SMALL_NUMBER);
			const bool bIsNearlyZeroT3T2 = FMath::IsNearlyZero(T3T2, UE_KINDA_SMALL_NUMBER);
			const bool bIsNearlyZeroT2T0 = FMath::IsNearlyZero(T2T0, UE_KINDA_SMALL_NUMBER);
			const bool bIsNearlyZeroT3T1 = FMath::IsNearlyZero(T3T1, UE_KINDA_SMALL_NUMBER);

			const FColor Color1 = Colors[FMath::Min(i, NumColors - 1)];
			const FColor Color2 = Colors[FMath::Min(i + 1, NumColors - 1)];

			for (int32 SampleIndex = 1; SampleIndex < NumSamplesPerSegment; ++SampleIndex)
			{
				const float ParametricDistance = float(SampleIndex) / float(NumSamplesPerSegment - 1);

				const float T = FMath::Lerp(T1, T2, ParametricDistance);

				const FVector A1 = bIsNearlyZeroT1T0 ? P0 : (T1 - T) / T1T0 * P0 + (T - T0) / T1T0 * P1;
				const FVector A2 = bIsNearlyZeroT2T1 ? P1 : (T2 - T) / T2T1 * P1 + (T - T1) / T2T1 * P2;
				const FVector A3 = bIsNearlyZeroT3T2 ? P2 : (T3 - T) / T3T2 * P2 + (T - T2) / T3T2 * P3;
				const FVector B1 = bIsNearlyZeroT2T0 ? A1 : (T2 - T) / T2T0 * A1 + (T - T0) / T2T0 * A2;
				const FVector B2 = bIsNearlyZeroT3T1 ? A2 : (T3 - T) / T3T1 * A2 + (T - T1) / T3T1 * A3;
				const FVector Point = bIsNearlyZeroT2T1 ? B1 : (T2 - T) / T2T1 * B1 + (T - T1) / T2T1 * B2;

				DrawLine(PrevPoint, Point, LerpColor(Color1, Color2, ParametricDistance));

				PrevPoint = Point;
			}
		}
	}
}

void FDebugDrawParams::DrawFeatureVector(TConstArrayView<float> PoseVector)
{
	if (CanDraw())
	{
		const UPoseSearchSchema* Schema = GetSchema();
		check(Schema);

		if (PoseVector.Num() == Schema->SchemaCardinality)
		{
			for (const TObjectPtr<UPoseSearchFeatureChannel>& ChannelPtr : Schema->GetChannels())
			{
				ChannelPtr->DebugDraw(*this, PoseVector);
			}
		}
	}
}

void FDebugDrawParams::DrawFeatureVector(int32 PoseIdx)
{
	if (CanDraw())
	{
		TArray<float> BufferUsedForReconstruction;
		DrawFeatureVector(GetSearchIndex()->GetPoseValuesSafe(PoseIdx, BufferUsedForReconstruction));
	}
}

bool FDebugDrawParams::IsAnyWeightRelevant(const UPoseSearchFeatureChannel* Channel) const
{
	const int32 StartIndex = Channel->GetChannelDataOffset();
	const int32 EndIndex = StartIndex + Channel->GetChannelCardinality();
	for (int32 Index = StartIndex; Index < EndIndex; ++Index)
	{
		if (DynamicWeightsSqrt[Index] > UE_SMALL_NUMBER)
		{
			return true;
		}
	}
	return false;
}

#endif // ENABLE_DRAW_DEBUG

//////////////////////////////////////////////////////////////////////////
// FCachedQuery
FCachedQuery::FCachedQuery(const UPoseSearchSchema* InSchema)
{
	check(InSchema);
	Schema = InSchema;
	Values.SetNumZeroed(Schema->SchemaCardinality);
}

//////////////////////////////////////////////////////////////////////////
// FCachedPCAQuery
FCachedPCAQuery::FCachedPCAQuery(const UPoseSearchDatabase* InDatabase)
{
	check(InDatabase);
	Database = InDatabase;
	Values.SetNumZeroed(Database->GetSearchIndex().GetNumberOfPrincipalComponents());
}

#if UE_POSE_SEARCH_TRACE_ENABLED
//////////////////////////////////////////////////////////////////////////
// FSearchContext::FBestPoseCandidates
FSearchContext::FBestPoseCandidates::FBestPoseCandidates()
{
	// preallocating memory to avoid multiple reallocations / rehashing
	TransientPoseCandidates.Reserve(MaxNumberOfCollectedPoseCandidatesPerDatabase);
	PermanentPoseCandidates.Reserve(16);
}

void FSearchContext::FBestPoseCandidates::Add(int32 PoseIdx, EPoseCandidateFlags PoseCandidateFlags, const FPoseSearchCost& Cost)
{
	if (EnumHasAnyFlags(PoseCandidateFlags, EPoseCandidateFlags::Valid_ContinuingPose | EPoseCandidateFlags::Valid_CurrentPose))
	{
		const FPoseCandidate PoseCandidate(PoseIdx, Cost, PoseCandidateFlags);
		bool bIsAlreadyInSet = false;
		FPoseCandidate& FoundPoseCandidate = PermanentPoseCandidates.FindOrAdd(PoseCandidate, &bIsAlreadyInSet);
		if (bIsAlreadyInSet)
		{
			if (Cost < FoundPoseCandidate.Cost)
			{
				FoundPoseCandidate.Cost = Cost;
			}
			FoundPoseCandidate.PoseCandidateFlags |= PoseCandidateFlags;
		}
	}
	else if (TransientPoseCandidates.Num() < MaxNumberOfCollectedPoseCandidatesPerDatabase)
	{
		const FPoseCandidate PoseCandidate(PoseIdx, Cost, PoseCandidateFlags);
		TransientPoseCandidates.HeapPush(PoseCandidate, FPoseCandidateHeapCompare());
	}
	else if (Cost < TransientPoseCandidates.HeapTop().Cost)
	{
		// popping the max heap (TransientPoseCandidates) head since it costs more than the candidate we're about to insert
		FPoseCandidate PoppedPoseCandidate;
		TransientPoseCandidates.HeapPop(PoppedPoseCandidate, FPoseCandidateHeapCompare(), EAllowShrinking::No);

		const FPoseCandidate PoseCandidate(PoseIdx, Cost, PoseCandidateFlags);
		TransientPoseCandidates.HeapPush(PoseCandidate, FPoseCandidateHeapCompare());
	}
}

bool FSearchContext::FBestPoseCandidates::IterateOverBestPoseCandidates(const TFunctionRef<bool(const FPoseCandidate& PoseCandidate)> IterateOverBestPoseCandidatesFunction) const
{
	TSet<uint32> DiscoveredPoseIndexes;
	DiscoveredPoseIndexes.Reserve(PermanentPoseCandidates.Num() + TransientPoseCandidates.Num());

	for (const FPoseCandidate& PoseCandidate : PermanentPoseCandidates)
	{
		if (IterateOverBestPoseCandidatesFunction(PoseCandidate))
		{
			return true;
		}
			
		DiscoveredPoseIndexes.Add(PoseCandidate.PoseIdx);
	}

	for (const FPoseCandidate& PoseCandidate : TransientPoseCandidates)
	{
		bool bIsAlreadyInSet = false;
		DiscoveredPoseIndexes.FindOrAdd(PoseCandidate.PoseIdx, &bIsAlreadyInSet);
		if (!bIsAlreadyInSet && IterateOverBestPoseCandidatesFunction(PoseCandidate))
		{
			return true;
		}
	}
		
	return false;
}

int32 FSearchContext::FBestPoseCandidates::Num() const
{
	return PermanentPoseCandidates.Num() + TransientPoseCandidates.Num();
}

FSearchContext::FPoseCandidate FSearchContext::FBestPoseCandidates::GetUnsortedCandidate(int32 Index) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	check(Index >= 0 && Index < Num());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (Index < PermanentPoseCandidates.Num())
	{
		TSet<FPoseCandidate, FPoseCandidatePoseIdxKeyFunc>::TConstIterator ConstIterator = PermanentPoseCandidates.CreateConstIterator();
		for (int32 Iteration = 0; Iteration < Index; ++Iteration)
		{
			++ConstIterator;
			check(ConstIterator);
		}
		return *ConstIterator;
	}

	return TransientPoseCandidates[Index - PermanentPoseCandidates.Num()];
}

#endif // UE_POSE_SEARCH_TRACE_ENABLED

//////////////////////////////////////////////////////////////////////////
// FSearchContext
FSearchContext::FSearchContext(float InDesiredPermutationTimeOffset, const FPoseIndicesHistory* InPoseIndicesHistory,
	const FSearchResult& InCurrentResult, const FFloatInterval& InPoseJumpThresholdTime, const FPoseSearchEvent& InEventToSearch)
: FSearchContext(InDesiredPermutationTimeOffset, InPoseJumpThresholdTime, InEventToSearch)
{
	UpdateContinuingPoseSearchResult(InCurrentResult, InCurrentResult);
}

FSearchContext::FSearchContext(float InDesiredPermutationTimeOffset, const FFloatInterval& InPoseJumpThresholdTime, const FPoseSearchEvent& InEventToSearch)
: AnimContexts()
, PoseHistories()
, RoleToIndex()
, AssetsToConsider()
, EventToSearch(InEventToSearch)
, DesiredPermutationTimeOffset(InDesiredPermutationTimeOffset)
, ContinuingPoseSearchResult(FSearchResult())
, ContinuingPoseValuesDatabasePoseIdx(FDatabasePoseIdx())
, PoseJumpThresholdTime(InPoseJumpThresholdTime)
, bUseCachedChannelData(false)
{
#if UE_POSE_SEARCH_TRACE_ENABLED
	BestPoseCandidatesMap.Reserve(16);
#endif // UE_POSE_SEARCH_TRACE_ENABLED
}

void FSearchContext::AddRole(const FRole& Role, FChooserEvaluationContext* AnimContext, const IPoseHistory* PoseHistory)
{
	check(RoleToIndex.Num() == AnimContexts.Num() && RoleToIndex.Num() == PoseHistories.Num());
	check(AnimContext && PoseHistory);

	AnimContexts.Add(AnimContext);
	PoseHistories.Add(PoseHistory);
	RoleToIndex.Add(Role) = RoleToIndex.Num();

	check(IsValid(RoleToIndex));
}

void FSearchContext::UpdateCurrentResultPoseVector()
{
	UpdateContinuingPoseValues();
}

void FSearchContext::UpdateContinuingPoseValues()
{
	if (ContinuingPoseValuesDatabasePoseIdx.IsValid())
	{
		const FSearchIndex& SearchIndex = ContinuingPoseValuesDatabasePoseIdx.Database->GetSearchIndex();
		if (SearchIndex.IsValuesEmpty())
		{
			const int32 NumDimensions = ContinuingPoseValuesDatabasePoseIdx.Database->Schema->SchemaCardinality;
			ContinuingPoseValuesData.SetNumUninitialized(NumDimensions, EAllowShrinking::No);
			ContinuingPoseValues = SearchIndex.GetReconstructedPoseValues(ContinuingPoseValuesDatabasePoseIdx.PoseIdx, ContinuingPoseValuesData);
		}
		else
		{
			ContinuingPoseValues = SearchIndex.GetPoseValues(ContinuingPoseValuesDatabasePoseIdx.PoseIdx);
		}
	}
	else
	{
		ContinuingPoseValues = {};
	}
}

void FSearchContext::UpdateContinuingPoseSearchResult(const FSearchResult& InContinuingPoseSearchResult, const FDatabasePoseIdx& InContinuingPoseValuesDatabasePoseIdx)
{
#if DO_CHECK
	// checking the validity of the continuing pose search values result, that has to be valid
	// if the result used to gather continuing pose search properties is valid
	if (InContinuingPoseSearchResult.IsValid())
	{
		check(InContinuingPoseValuesDatabasePoseIdx.IsValid());
		check(InContinuingPoseSearchResult.Database->Schema == InContinuingPoseValuesDatabasePoseIdx.Database->Schema);
	}
#endif //DO_CHECK

	const bool bNeedToUpdateContinuingPoseValues =
		ContinuingPoseValuesDatabasePoseIdx.PoseIdx != InContinuingPoseValuesDatabasePoseIdx.PoseIdx ||
		ContinuingPoseValuesDatabasePoseIdx.Database != InContinuingPoseValuesDatabasePoseIdx.Database;

	ContinuingPoseSearchResult = InContinuingPoseSearchResult;
	ContinuingPoseValuesDatabasePoseIdx = InContinuingPoseValuesDatabasePoseIdx;
	
	if (bNeedToUpdateContinuingPoseValues)
	{
		UpdateContinuingPoseValues();
	}

	check(!ContinuingPoseValuesDatabasePoseIdx.IsValid() || ContinuingPoseValues.Num() == ContinuingPoseValuesDatabasePoseIdx.Database->Schema->SchemaCardinality);
}

float FSearchContext::GetSampleCurveValue(float SampleTimeOffset, const FName& CurveName, const FRole& SampleRole)
{
	const float SampleTime = SampleTimeOffset;
	return GetSampleCurveValueInternal(SampleTime, CurveName, SampleRole);
}

float FSearchContext::GetSampleCurveValueInternal(float SampleTime, const FName& CurveName, const FRole& SampleRole)
{
	check(!CachedQueries.IsEmpty());
	const UPoseSearchSchema* Schema = CachedQueries.Last().GetSchema();
	check(Schema);

	float OutCurveValue = 0.0f;
	const IPoseHistory* PoseHistory = GetPoseHistory(SampleRole);
	check(PoseHistory);
	
#if !NO_LOGGING
	const bool bSuccess = 
#endif // !NO_LOGGING
		PoseHistory->GetCurveValueAtTime(SampleTime, CurveName, OutCurveValue);
#if !NO_LOGGING
	if (!bSuccess && !PoseHistory->IsEmpty())
	{
		UE_LOGF(LogPoseSearch, Warning, "FSearchContext::GetSampleCurveValueInternal - Couldn't find Curve %ls in the input IPoseHistory requested by %ls. Consider adding it to the Pose History!", *CurveName.ToString(), *Schema->GetName());
	}
#endif // !NO_LOGGING

	return OutCurveValue;
}

FQuat FSearchContext::GetSampleRotation(float SampleTimeOffset, float OriginTimeOffset, int8 SchemaSampleBoneIdx, int8 SchemaOriginBoneIdx, const FRole& SampleRole, const FRole& OriginRole, EPermutationTimeType PermutationTimeType, const FQuat* SampleBoneRotationWorldOverride)
{
	float PermutationSampleTimeOffset = 0.f;
	float PermutationOriginTimeOffset = 0.f;
	UPoseSearchFeatureChannel::GetPermutationTimeOffsets(PermutationTimeType, DesiredPermutationTimeOffset, PermutationSampleTimeOffset, PermutationOriginTimeOffset);

	const float SampleTime = SampleTimeOffset + PermutationSampleTimeOffset;
	const float OriginTime = OriginTimeOffset + PermutationOriginTimeOffset;

	return GetSampleRotationInternal(SampleTime, OriginTime, SchemaSampleBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole, SampleBoneRotationWorldOverride);
}

FVector FSearchContext::GetSamplePosition(float SampleTimeOffset, float OriginTimeOffset, int8 SchemaSampleBoneIdx, int8 SchemaOriginBoneIdx, const FRole& SampleRole, const FRole& OriginRole, EPermutationTimeType PermutationTimeType, const FVector* SampleBonePositionWorldOverride)
{
	float PermutationSampleTimeOffset = 0.f;
	float PermutationOriginTimeOffset = 0.f;
	UPoseSearchFeatureChannel::GetPermutationTimeOffsets(PermutationTimeType, DesiredPermutationTimeOffset, PermutationSampleTimeOffset, PermutationOriginTimeOffset);

	const float SampleTime = SampleTimeOffset + PermutationSampleTimeOffset;
	const float OriginTime = OriginTimeOffset + PermutationOriginTimeOffset;
	return GetSamplePositionInternal(SampleTime, OriginTime, SchemaSampleBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole, SampleBonePositionWorldOverride);
}

FVector FSearchContext::GetSampleVelocity(float SampleTimeOffset, float OriginTimeOffset, int8 SchemaSampleBoneIdx, int8 SchemaOriginBoneIdx, const FRole& SampleRole, const FRole& OriginRole, bool bUseCharacterSpaceVelocities, EPermutationTimeType PermutationTimeType, const FVector* SampleBoneVelocityWorldOverride)
{
	using namespace UE::PoseSearch;

	float PermutationSampleTimeOffset = 0.f;
	float PermutationOriginTimeOffset = 0.f;
	UPoseSearchFeatureChannel::GetPermutationTimeOffsets(PermutationTimeType, DesiredPermutationTimeOffset, PermutationSampleTimeOffset, PermutationOriginTimeOffset);

	const float SampleTime = SampleTimeOffset + PermutationSampleTimeOffset;
	const float OriginTime = OriginTimeOffset + PermutationOriginTimeOffset;

	if (SampleBoneVelocityWorldOverride)
	{
		const FTransform RootBoneTransform = GetWorldBoneTransformAtTime(OriginTime, OriginRole, RootSchemaBoneIdx);
		return RootBoneTransform.InverseTransformVector(*SampleBoneVelocityWorldOverride);
	}

	// calculating the local Position for the bone indexed by SchemaSampleBoneIdx
	const FVector PreviousTranslation = GetSamplePositionInternal(SampleTime - FiniteDelta, bUseCharacterSpaceVelocities ? OriginTime - FiniteDelta : OriginTime, SchemaSampleBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole);
	const FVector CurrentTranslation = GetSamplePositionInternal(SampleTime, OriginTime, SchemaSampleBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole);

	const FVector LinearVelocity = (CurrentTranslation - PreviousTranslation) / FiniteDelta;
	return LinearVelocity;
}

FTransform FSearchContext::GetWorldRootBoneTransformAtTime(float SampleTime, const FRole& SampleRole) const
{
	check(!CachedQueries.IsEmpty());
	const UPoseSearchSchema* Schema = CachedQueries.Last().GetSchema();
	check(Schema);

	FTransform PoseHistoryTransformAtTime;
	const IPoseHistory* PoseHistory = GetPoseHistory(SampleRole);
	check(PoseHistory)
	if (PoseHistory->GetTransformAtTime(SampleTime, PoseHistoryTransformAtTime, Schema->GetSkeleton(SampleRole), RootBoneIndexType, WorldSpaceIndexType))
	{
		// PoseHistoryTransformAtTime contains the WorldRootBoneTransform (RootBoneTransform from the blended entry in the pose history times ComponentToWorldTransform from the trajectory)
		return PoseHistoryTransformAtTime;
	}

	// @ todo: should we use the component transform here in case we lack the root bone transform?
	// NoTe: when GetTransformAtTime returns false PoseHistoryTransformAtTime doesn't contain a valid root bone transform in world space, BUT a valid ComponentToWorldTransform!
	// So we use PoseHistoryTransformAtTime as such below (in the form of "const FTransform& ComponentToWorldTransform = PoseHistory ? PoseHistoryTransformAtTime : ...)

	// accessing AnimInstance or SkeletalMesh properties is not thread safe from a worker thread! Logging an error and returning identity transform
	
	//const UObject* AnimContext = GetAnimContext(SampleRole);
	//if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(AnimContext))
	//{
	//	if (AnimInstance && AnimInstance->CurrentSkeleton)
	//	{
	//		const FTransform& RootBoneTransform = AnimInstance->CurrentSkeleton->GetReferenceSkeleton().GetRefBonePose()[RootSchemaBoneIdx];
	//		const FTransform& ComponentToWorldTransform = PoseHistory ? PoseHistoryTransformAtTime : AnimInstance->GetSkelMeshComponent()->GetComponentTransform();
	//		return RootBoneTransform * ComponentToWorldTransform;
	//	}
	//}
	//else if (const UActorComponent* ActorComponent = Cast<UActorComponent>(AnimContext))
	//{
	//	const AActor* Actor = ActorComponent->GetOwner();
	//	check(Actor);
	//	// @todo: this code depends on how AnimNext gather its context object, and will likely change in the future
	//	if (const USkeletalMeshComponent* SkeletalMeshComponent = Actor->GetComponentByClass<USkeletalMeshComponent>())
	//	{
	//		if (const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset())
	//		{
	//			const FTransform& RootBoneTransform = SkeletalMesh->GetSkeleton()->GetReferenceSkeleton().GetRefBonePose()[RootSchemaBoneIdx];
	//			const FTransform& ComponentToWorldTransform = PoseHistory ? PoseHistoryTransformAtTime : SkeletalMeshComponent->GetComponentTransform();
	//			return RootBoneTransform * ComponentToWorldTransform;
	//		}
	//	}
	//}

	return FTransform::Identity;
}

const FPoseIndicesHistory* FSearchContext::GetPoseIndicesHistory() const
{
	if (PoseHistories.Num() > 0)
	{
		const IPoseHistory* PoseHistory = PoseHistories[0]; 
		check(PoseHistory);
		return PoseHistory->GetPoseIndicesHistory();
	}
	return nullptr;
}

bool FSearchContext::ArePoseHistoriesValid() const
{
	for (const IPoseHistory* PoseHistory : PoseHistories)
	{
		if (!PoseHistory)
		{
			return false;
		}
	}
	return true;
}

const IPoseHistory* FSearchContext::GetPoseHistory(const FRole& Role) const
{
	return PoseHistories[RoleToIndex[Role]];
}

const FChooserEvaluationContext* FSearchContext::GetContext(const FRole& Role) const
{
	return AnimContexts[RoleToIndex[Role]];
}
	
FChooserEvaluationContext* FSearchContext::GetContext(const FRole& Role)
{
	return AnimContexts[RoleToIndex[Role]];
}

FTransform FSearchContext::GetWorldBoneTransformAtTime(float SampleTime, const FRole& SampleRole, int8 SchemaBoneIdx)
{
	// CachedQueries.Last is the query we're building 
	check(!CachedQueries.IsEmpty());
	const UPoseSearchSchema* Schema = CachedQueries.Last().GetSchema();
	check(Schema);

	const FBoneIndexType BoneIndexType = SchemaBoneIdx == TrajectorySchemaBoneIdx ? ComponentSpaceIndexType : Schema->GetBoneReferences(SampleRole)[SchemaBoneIdx].BoneIndex;

	const uint32 SampleTimeHash = GetTypeHash(SampleTime);
	const uint32 SampleRoleHash = GetTypeHash(SampleRole);
	const uint32 SampleTimeAndRoleHash = HashCombineFast(SampleTimeHash, SampleRoleHash);
	const uint32 BoneIndexTypeHash = GetTypeHash(BoneIndexType);
	const uint32 BoneCachedTransformKey = HashCombineFast(SampleTimeAndRoleHash, BoneIndexTypeHash);

	if (const FTransform* CachedTransform = CachedTransforms.Find(BoneCachedTransformKey))
	{
		return *CachedTransform;
	}

	FTransform WorldBoneTransform;
	if (BoneIndexType == RootBoneIndexType)
	{
		// we already tried querying the CachedTransforms so, let's search in Trajectory
		WorldBoneTransform = GetWorldRootBoneTransformAtTime(SampleTime, SampleRole);
	}
	else // if (BoneIndexType != RootBoneIndexType)
	{
		// searching for RootBoneIndexType in CachedTransforms
		static const uint32 RootBoneIndexTypeHash = GetTypeHash(RootBoneIndexType); // Note: static const, since RootBoneIndexType is a constant
		const uint32 RootBoneCachedTransformKey = HashCombineFast(SampleTimeAndRoleHash, RootBoneIndexTypeHash);
		if (const FTransform* CachedTransform = CachedTransforms.Find(RootBoneCachedTransformKey))
		{
			WorldBoneTransform = *CachedTransform;
		}
		else
		{
			WorldBoneTransform = GetWorldRootBoneTransformAtTime(SampleTime, SampleRole);
		}

		// collecting the local bone transforms from the IPoseHistory
		const IPoseHistory* PoseHistory = GetPoseHistory(SampleRole);
		check(PoseHistory);
		
		const USkeleton* Skeleton = Schema->GetSkeleton(SampleRole);
		FTransform LocalBoneTransform;
		if (!PoseHistory->GetTransformAtTime(SampleTime, LocalBoneTransform, Skeleton, BoneIndexType, RootBoneIndexType))
		{
			if (Skeleton)
			{
				if (!PoseHistory->IsEmpty())
				{
					UE_LOGF(LogPoseSearch, Warning, "FSearchContext::GetWorldBoneTransformAtTime - Couldn't find BoneIndexType %d (%ls) for Skeleton %ls in the input IPoseHistory requested by %ls. Consider adding it to the Pose History!", BoneIndexType, *Skeleton->GetReferenceSkeleton().GetBoneName(BoneIndexType).ToString(), *Skeleton->GetName(), *Schema->GetName());
				}
			}
			else
			{
				UE_LOGF(LogPoseSearch, Warning, "FSearchContext::GetWorldBoneTransformAtTime - Schema '%ls' Skeleton is not properly set", *Schema->GetName());
			}
		}

		WorldBoneTransform = LocalBoneTransform * WorldBoneTransform;
	}

	CachedTransforms.Add(BoneCachedTransformKey) = WorldBoneTransform;
	return WorldBoneTransform;
}

FVector FSearchContext::GetSamplePositionInternal(float SampleTime, float OriginTime, int8 SchemaSampleBoneIdx, int8 SchemaOriginBoneIdx, const FRole& SampleRole, const FRole& OriginRole, const FVector* SampleBonePositionWorldOverride)
{
	if (SampleBonePositionWorldOverride)
	{
		const FTransform RootBoneTransform = GetWorldBoneTransformAtTime(OriginTime, OriginRole, RootSchemaBoneIdx);
		if (SchemaOriginBoneIdx == RootSchemaBoneIdx)
		{
			return RootBoneTransform.InverseTransformPosition(*SampleBonePositionWorldOverride);
		}

		// @todo: validate this still works for when root bone is not Identity
		const FTransform OriginBoneTransform = GetWorldBoneTransformAtTime(OriginTime, OriginRole, SchemaOriginBoneIdx);
		const FVector DeltaBoneTranslation = *SampleBonePositionWorldOverride - OriginBoneTransform.GetTranslation();
		return RootBoneTransform.InverseTransformVector(DeltaBoneTranslation);
	}

	const FTransform RootBoneTransform = GetWorldBoneTransformAtTime(OriginTime, OriginRole, RootSchemaBoneIdx);
	const FTransform SampleBoneTransform = GetWorldBoneTransformAtTime(SampleTime, SampleRole, SchemaSampleBoneIdx);
	if (SchemaOriginBoneIdx == RootSchemaBoneIdx)
	{
		return RootBoneTransform.InverseTransformPosition(SampleBoneTransform.GetTranslation());
	}

	const FTransform OriginBoneTransform = GetWorldBoneTransformAtTime(OriginTime, OriginRole, SchemaOriginBoneIdx);
	const FVector DeltaBoneTranslation = SampleBoneTransform.GetTranslation() - OriginBoneTransform.GetTranslation();
	return RootBoneTransform.InverseTransformVector(DeltaBoneTranslation);
}

FQuat FSearchContext::GetSampleRotationInternal(float SampleTime, float OriginTime, int8 SchemaSampleBoneIdx, int8 SchemaOriginBoneIdx, const FRole& SampleRole, const FRole& OriginRole, const FQuat* SampleBoneRotationWorldOverride)
{
	if (SampleBoneRotationWorldOverride)
	{
		const FTransform RootBoneTransform = GetWorldBoneTransformAtTime(OriginTime, OriginRole, RootSchemaBoneIdx);
		if (SchemaOriginBoneIdx == RootSchemaBoneIdx)
		{
			return RootBoneTransform.InverseTransformRotation(*SampleBoneRotationWorldOverride);
		}

		const FTransform OriginBoneTransform = GetWorldBoneTransformAtTime(OriginTime, OriginRole, SchemaOriginBoneIdx);
		const FQuat DeltaBoneRotation = OriginBoneTransform.InverseTransformRotation(*SampleBoneRotationWorldOverride);
		return RootBoneTransform.InverseTransformRotation(DeltaBoneRotation);
	}

	const FTransform OriginBoneTransform = GetWorldBoneTransformAtTime(OriginTime, OriginRole, SchemaOriginBoneIdx);
	const FTransform SampleBoneTransform = GetWorldBoneTransformAtTime(SampleTime, SampleRole, SchemaSampleBoneIdx);
	return OriginBoneTransform.InverseTransformRotation(SampleBoneTransform.GetRotation());
}

TArrayView<float> FSearchContext::EditFeatureVector()
{
	// CachedQueries.Last is the query we're building 
	check(!CachedQueries.IsEmpty());
	FCachedQuery& CachedQuery = CachedQueries.Last();
	TArrayView<float> FeatureVector = CachedQuery.EditValues();
	check(CachedQuery.GetSchema());
	check(FeatureVector.GetData());
	check(CachedQuery.GetSchema()->SchemaCardinality == FeatureVector.Num());
	return FeatureVector;
}

const UPoseSearchFeatureChannel* FSearchContext::GetCachedChannelData(uint32 ChannelUniqueIdentifier, const UPoseSearchFeatureChannel* Channel, TConstArrayView<float>& CachedChannelData)
{
	// searching CachedChannels for the ChannelUniqueIdentifier as representation of Channel
	FCachedChannel& CachedChannel = CachedChannels.FindOrAdd(ChannelUniqueIdentifier);
	if (CachedChannel.Channel)
	{
		// we found CachedChannel.Channel, a channel from a different schema (CachedQueries[CachedChannel.CachedQueryIndex].GetSchema()) compatible with Channel.
		// let's collect the associated data to CachedChannel.Channel 
		CachedChannelData = CachedQueries[CachedChannel.CachedQueryIndex].GetValues().Slice(CachedChannel.Channel->GetChannelDataOffset(), CachedChannel.Channel->GetChannelCardinality());
		return CachedChannel.Channel;
	}
	
	// we couldn't find the cached channel, so let's add the pair ChannelUniqueIdentifier / Channel to CachedChannels.
	// the associated CachedQueries[CachedQueries.Num() - 1].GetValues() data will be filled up by the end of Channel BuildQuery
	CachedChannel.CachedQueryIndex = CachedQueries.Num() - 1;
	CachedChannel.Channel = Channel;
	
	CachedChannelData = TConstArrayView<float>();
	return nullptr;
}

void FSearchContext::ResetCurrentBestCost()
{
}

void FSearchContext::UpdateCurrentBestCost(const FPoseSearchCost& PoseSearchCost)
{
}

TConstArrayView<float> FSearchContext::GetCachedQuery(const UPoseSearchSchema* Schema) const
{
	if (const FCachedQuery* FoundCachedQuery = CachedQueries.FindByPredicate(
		[Schema](const FCachedQuery& CachedQuery)
		{
			return CachedQuery.GetSchema() == Schema;
		}))
	{
		return FoundCachedQuery->GetValues();
	}
	return TConstArrayView<float>();
}

TConstArrayView<float> FSearchContext::GetOrBuildQuery(const UPoseSearchSchema* Schema)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PoseSearch_GetOrBuildQuery);

	check(Schema);
	if (const FCachedQuery* FoundCachedQuery = CachedQueries.FindByPredicate(
		[Schema](const FCachedQuery& CachedQuery)
		{
			return CachedQuery.GetSchema() == Schema;
		}))
	{
		return FoundCachedQuery->GetValues();
	}
	
	return Schema->BuildQuery(*this);
}

TConstArrayView<float> FSearchContext::GetOrBuildPCAQuery(const UPoseSearchDatabase* Database)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PoseSearch_GetOrBuildPCAQuery);

	check(Database);
	if (const FCachedPCAQuery* FoundCachedPCAQuery = CachedPCAQueries.FindByPredicate(
		[Database](const FCachedPCAQuery& CachedPCAQuery)
		{
			return CachedPCAQuery.GetDatabase() == Database;
		}))
	{
		return FoundCachedPCAQuery->GetValues();
	}
	
	// building the query for the database schema
	TConstArrayView<float> QueryValues = GetOrBuildQuery(Database->Schema);

	// converting it into PCA space
	FCachedPCAQuery& NewCachedPCAQuery = CachedPCAQueries.Emplace_GetRef(Database);
	return Database->GetSearchIndex().PCAProject(QueryValues, NewCachedPCAQuery.EditValues());
}

bool FSearchContext::IsCurrentResultFromDatabase(const UPoseSearchDatabase* Database) const
{
	return IsContinuingPoseDatabase(Database);
}

bool FSearchContext::IsContinuingPoseDatabase(const UPoseSearchDatabase* Database) const
{
	return ContinuingPoseSearchResult.IsValid() && ContinuingPoseSearchResult.Database == Database;
}

bool FSearchContext::CanUseCurrentResult() const
{
	return CanUseContinuingPoseValues();
}

bool FSearchContext::CanUseContinuingPoseValues() const
{
	// CachedQueries.Last is the query we're building 
	check(!CachedQueries.IsEmpty());
	const bool bCanUseContinuingPoseValues = ContinuingPoseSearchResult.IsValid() && ContinuingPoseSearchResult.Database->Schema == CachedQueries.Last().GetSchema();
	check(!bCanUseContinuingPoseValues || ContinuingPoseValues.Num() == ContinuingPoseSearchResult.Database->Schema->SchemaCardinality);
	return bCanUseContinuingPoseValues;
}

#if UE_POSE_SEARCH_TRACE_ENABLED
void FSearchContext::Track(const UPoseSearchDatabase* Database, int32 PoseIdx, EPoseCandidateFlags PoseCandidateFlags, const FPoseSearchCost& Cost)
{
	if (Database)
	{
		FBestPoseCandidates& BestPoseCandidates = BestPoseCandidatesMap.FindOrAdd(Database);
		if (PoseIdx != INDEX_NONE)
		{
			BestPoseCandidates.Add(PoseIdx, PoseCandidateFlags, Cost);
		}
	}
}

void FSearchContext::Track(const UPoseSearchDatabase* Database, int32 PoseIdx, EPoseCandidateFlags PoseCandidateFlags, TConstArrayView<float> DynamicWeightsSqrt, float ContinuingPoseCostAddend, float ContinuingInteractionCostAddend, float ContinuingContextInteractionCostAddend)
{
	check(PoseIdx >= 0 && Database && Database->Schema);

	// if we're editing the database and removing assets it's possible that the PoseIndicesHistory contains invalid pose indexes
	const FSearchIndex& SearchIndex = Database->GetSearchIndex();
	if (PoseIdx < SearchIndex.GetNumPoses())
	{
		float DissimilarityCost = 0.f;
		if (SearchIndex.IsValuesEmpty())
		{
			const int32 PCAValueIdx = SearchIndex.GetPCAValueIndex(PoseIdx);
			if (PCAValueIdx != INDEX_NONE)
			{
				TConstArrayView<float> PCAQueryValues = GetOrBuildPCAQuery(Database);
				// estimating the cost with the PCAVAlues rather than the Values
				DissimilarityCost = CompareFeatureVectors(SearchIndex.GetPCAPoseValues(PCAValueIdx), PCAQueryValues);
			}
			else
			{
				DissimilarityCost = 100.f;
			}
		}
		else
		{
			TConstArrayView<float> QueryValues = GetOrBuildQuery(Database->Schema);
			DissimilarityCost = CompareFeatureVectors<false>(SearchIndex.GetPoseValues(PoseIdx), QueryValues, DynamicWeightsSqrt);
		}
		const FPoseSearchCost PoseCost(DissimilarityCost, SearchIndex.PoseMetadata[PoseIdx].GetCostAddend(), ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend);
		Track(Database, PoseIdx, PoseCandidateFlags, PoseCost);
	}
}
#endif // UE_POSE_SEARCH_TRACE_ENABLED

} // namespace UE::PoseSearch