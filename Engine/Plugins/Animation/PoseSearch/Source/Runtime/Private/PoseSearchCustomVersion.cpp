// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FPoseSearchCustomVersion::GUID("99c277d4-1347-497b-ab23-47514df0f30b");

// Register the custom version with core
FCustomVersionRegistration GRegisterPoseSearchCustomVersion(FPoseSearchCustomVersion::GUID, FPoseSearchCustomVersion::LatestVersion, TEXT("Dev-PoseSearch-Version"));
