// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchDatabase.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "Animation/BlendSpace1D.h"
#include "Chooser.h"
#include "PoseSearch/Chooser/PoseSearchChooserColumn.h"
#include "PoseSearch/MultiAnimAsset.h"
#include "PoseSearch/PoseSearchAnimNotifies.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchCustomVersion.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "PoseSearch/PoseSearchDerivedData.h"
#include "PoseSearch/PoseSearchFeatureChannel_Group.h"
#include "PoseSearch/PoseSearchHistory.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearchIndex.inl"
#include "Serialization/ArchiveCountMem.h"
#include "UObject/ObjectSaveContext.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#include "AssetRegistry/AssetRegistryModule.h"
#endif //WITH_EDITOR

#if WITH_EDITOR && WITH_ENGINE
#include "Editor/EditorEngine.h"
#endif //WITH_EDITOR && WITH_ENGINE

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchDatabase)

struct FPoseSearchDatabaseAnimationAssetBase;
DECLARE_STATS_GROUP(TEXT("PoseSearch"), STATGROUP_PoseSearch, STATCAT_Advanced);
DECLARE_CYCLE_STAT_EXTERN(TEXT("PoseSearch Brute Force"), STAT_PoseSearch_BruteForce, STATGROUP_PoseSearch, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("PoseSearch PCA/KNN"), STAT_PoseSearch_PCAKNN, STATGROUP_PoseSearch, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("PoseSearch VPTree"), STAT_PoseSearch_VPTree, STATGROUP_PoseSearch, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("PoseSearch Event"), STAT_PoseSearch_Event, STATGROUP_PoseSearch, );
DEFINE_STAT(STAT_PoseSearch_BruteForce);
DEFINE_STAT(STAT_PoseSearch_PCAKNN);
DEFINE_STAT(STAT_PoseSearch_VPTree);
DEFINE_STAT(STAT_PoseSearch_Event);

#define LOCTEXT_NAMESPACE "PoseSearchDatabase"

namespace UE::PoseSearch
{
#if WITH_EDITOR && ENABLE_ANIM_DEBUG

static bool GVarMotionMatchCompareAgainstBruteForce = false;
static FAutoConsoleVariableRef CVarMotionMatchCompareAgainstBruteForce(TEXT("a.MotionMatch.CompareAgainstBruteForce"), GVarMotionMatchCompareAgainstBruteForce, TEXT("Compare optimized search against brute force search"));

static bool GVarMotionMatchValidateKNNSearch = false;
static FAutoConsoleVariableRef CVarMotionMatchValidateKNNSearch(TEXT("a.MotionMatch.ValidateKNNSearch"), GVarMotionMatchValidateKNNSearch, TEXT("Validate KNN search"));

static bool GVarMotionMatchProfileMaxHeapKNNSearch = false;
static FAutoConsoleVariableRef CVarMotionMatchProfileMaxHeapKNNSearch(TEXT("a.MotionMatch.ProfileMaxHeapKNNSearch"), GVarMotionMatchProfileMaxHeapKNNSearch, TEXT("Profile MaxHeap KNN Search"));

#endif

#if WITH_EDITOR
static bool GVarMotionMatchRequestAsyncBuildIndexOnPostLoad = true;
static FAutoConsoleVariableRef CVarMotionMatchRequestAsyncBuildIndexOnPostLoad(TEXT("a.MotionMatch.RequestAsyncBuildIndexOnPostLoad"), GVarMotionMatchRequestAsyncBuildIndexOnPostLoad, TEXT("Request AsyncBuildIndex On Database PostLoad. If false "));
#endif // WITH_EDITOR

// Experimental, this feature might be removed without warning, not for production use
static int32 GVarMotionMatchDebugWeightGroupID = 0;
static FAutoConsoleVariableRef CVarMotionMatchDebugWeightGroupID(TEXT("a.MotionMatch.DebugWeightGroupID"), GVarMotionMatchDebugWeightGroupID, TEXT("Only the channels with this or negative DebugWeightGroupID will have non zero weights"));

FPoseSearchDatabaseAnimationAsset UpgradeToDatabaseAnimationAsset(FInstancedStruct& AnimationAsset)
{
	FPoseSearchDatabaseAnimationAsset PoseSearchDatabaseAnimationAsset;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	if (const FPoseSearchDatabaseSequence* DatabaseSequence = AnimationAsset.GetPtr<FPoseSearchDatabaseSequence>())
	{
		PoseSearchDatabaseAnimationAsset.AnimAsset = DatabaseSequence->GetAnimationAsset();
#if WITH_EDITORONLY_DATA
		PoseSearchDatabaseAnimationAsset.bEnabled = DatabaseSequence->bEnabled;
		PoseSearchDatabaseAnimationAsset.bDisableReselection = DatabaseSequence->bDisableReselection;
		PoseSearchDatabaseAnimationAsset.MirrorOption = DatabaseSequence->MirrorOption;
		PoseSearchDatabaseAnimationAsset.BranchInId = DatabaseSequence->BranchInId;

		PoseSearchDatabaseAnimationAsset.SamplingRange = DatabaseSequence->SamplingRange;
#endif // WITH_EDITORONLY_DATA
	}
	else if (const FPoseSearchDatabaseBlendSpace* DatabaseBlendSpace = AnimationAsset.GetPtr<FPoseSearchDatabaseBlendSpace>())
	{
		PoseSearchDatabaseAnimationAsset.AnimAsset = DatabaseBlendSpace->GetAnimationAsset();
#if WITH_EDITORONLY_DATA
		PoseSearchDatabaseAnimationAsset.bEnabled = DatabaseBlendSpace->bEnabled;
		PoseSearchDatabaseAnimationAsset.bDisableReselection = DatabaseBlendSpace->bDisableReselection;
		PoseSearchDatabaseAnimationAsset.MirrorOption = DatabaseBlendSpace->MirrorOption;
		PoseSearchDatabaseAnimationAsset.BranchInId = DatabaseBlendSpace->BranchInId;

		PoseSearchDatabaseAnimationAsset.bUseSingleSample = DatabaseBlendSpace->bUseSingleSample;
		PoseSearchDatabaseAnimationAsset.bUseGridForSampling = DatabaseBlendSpace->bUseGridForSampling;
		PoseSearchDatabaseAnimationAsset.NumberOfHorizontalSamples = DatabaseBlendSpace->NumberOfHorizontalSamples;
		PoseSearchDatabaseAnimationAsset.NumberOfVerticalSamples = DatabaseBlendSpace->NumberOfVerticalSamples;
		PoseSearchDatabaseAnimationAsset.BlendParamX = DatabaseBlendSpace->BlendParamX;
		PoseSearchDatabaseAnimationAsset.BlendParamY = DatabaseBlendSpace->BlendParamY;
		PoseSearchDatabaseAnimationAsset.SamplingRange = DatabaseBlendSpace->SamplingRange;
#endif // WITH_EDITORONLY_DATA
	}
	else if (const FPoseSearchDatabaseAnimComposite* DatabaseAnimComposite = AnimationAsset.GetPtr<FPoseSearchDatabaseAnimComposite>())
	{
		PoseSearchDatabaseAnimationAsset.AnimAsset = DatabaseAnimComposite->GetAnimationAsset();
#if WITH_EDITORONLY_DATA
		PoseSearchDatabaseAnimationAsset.bEnabled = DatabaseAnimComposite->bEnabled;
		PoseSearchDatabaseAnimationAsset.bDisableReselection = DatabaseAnimComposite->bDisableReselection;
		PoseSearchDatabaseAnimationAsset.MirrorOption = DatabaseAnimComposite->MirrorOption;
		PoseSearchDatabaseAnimationAsset.BranchInId = DatabaseAnimComposite->BranchInId;

		PoseSearchDatabaseAnimationAsset.SamplingRange = DatabaseAnimComposite->SamplingRange;
#endif // WITH_EDITORONLY_DATA
	}
	else if (const FPoseSearchDatabaseAnimMontage* DatabaseAnimMontage = AnimationAsset.GetPtr<FPoseSearchDatabaseAnimMontage>())
	{
		PoseSearchDatabaseAnimationAsset.AnimAsset = DatabaseAnimMontage->GetAnimationAsset();
#if WITH_EDITORONLY_DATA
		PoseSearchDatabaseAnimationAsset.bEnabled = DatabaseAnimMontage->bEnabled;
		PoseSearchDatabaseAnimationAsset.bDisableReselection = DatabaseAnimMontage->bDisableReselection;
		PoseSearchDatabaseAnimationAsset.MirrorOption = DatabaseAnimMontage->MirrorOption;
		PoseSearchDatabaseAnimationAsset.BranchInId = DatabaseAnimMontage->BranchInId;

		PoseSearchDatabaseAnimationAsset.SamplingRange = DatabaseAnimMontage->SamplingRange;
#endif // WITH_EDITORONLY_DATA
	}
	else if (const FPoseSearchDatabaseMultiAnimAsset* DatabaseMultiAnimAsset = AnimationAsset.GetPtr<FPoseSearchDatabaseMultiAnimAsset>())
	{
		PoseSearchDatabaseAnimationAsset.AnimAsset = DatabaseMultiAnimAsset->GetAnimationAsset();
#if WITH_EDITORONLY_DATA
		PoseSearchDatabaseAnimationAsset.bEnabled = DatabaseMultiAnimAsset->bEnabled;
		PoseSearchDatabaseAnimationAsset.bDisableReselection = DatabaseMultiAnimAsset->bDisableReselection;
		PoseSearchDatabaseAnimationAsset.MirrorOption = DatabaseMultiAnimAsset->MirrorOption;
		PoseSearchDatabaseAnimationAsset.BranchInId = DatabaseMultiAnimAsset->BranchInId;

		PoseSearchDatabaseAnimationAsset.NumberOfHorizontalSamples = DatabaseMultiAnimAsset->NumberOfHorizontalSamples;
		PoseSearchDatabaseAnimationAsset.NumberOfVerticalSamples = DatabaseMultiAnimAsset->NumberOfVerticalSamples;
		PoseSearchDatabaseAnimationAsset.SamplingRange = DatabaseMultiAnimAsset->SamplingRange;
#endif // WITH_EDITORONLY_DATA
	}
		
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return PoseSearchDatabaseAnimationAsset;
}

template<bool bReconstructPoseValues, bool bAlignedAndPadded>
static inline void EvaluatePoseKernel(FSearchResults& SearchResults, const FSearchIndex& SearchIndex, TConstArrayView<float> QueryValues, TArrayView<float> ReconstructedPoseValuesBuffer,
	int32 PoseIdx, int32 EventPoseIdx, const FSearchFilters& SearchFilters, float ContinuingPoseCostAddend, float ContinuingInteractionCostAddend, float ContinuingContextInteractionCostAddend,
	const UPoseSearchDatabase* Database, TConstArrayView<float> DynamicWeightsSqrt
#if UE_POSE_SEARCH_TRACE_ENABLED
	, bool bTrackResult = true
#endif // UE_POSE_SEARCH_TRACE_ENABLED
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
	, int32 ResultIndex = INDEX_NONE
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
)
{
	using namespace UE::PoseSearch;

	const TConstArrayView<float> PoseValues = bReconstructPoseValues ? SearchIndex.GetReconstructedPoseValues(PoseIdx, ReconstructedPoseValuesBuffer) : SearchIndex.GetPoseValues(PoseIdx);

	if (SearchFilters.AreFiltersValid(SearchIndex, PoseValues, QueryValues, DynamicWeightsSqrt, PoseIdx
#if UE_POSE_SEARCH_TRACE_ENABLED
		, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, Database
#endif // UE_POSE_SEARCH_TRACE_ENABLED
	))
	{
		FSearchResult Result;
		Result.PoseCost = FPoseSearchCost(CompareFeatureVectors<bAlignedAndPadded>(PoseValues, QueryValues, DynamicWeightsSqrt), SearchIndex.PoseMetadata[PoseIdx].GetCostAddend(), ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend);
		Result.PoseIdx = PoseIdx;
		Result.EventPoseIdx = EventPoseIdx;
		Result.Database = Database;
		
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
		Result.BestPosePos = ResultIndex;
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED

		SearchResults.UpdateWith(Result);

#if UE_POSE_SEARCH_TRACE_ENABLED
		if (bTrackResult)
		{
			// @todo: add tracking for EventPoseIdx
			SearchFilters.GetSearchContext().Track(Database, PoseIdx, EPoseCandidateFlags::Valid_Pose, Result.PoseCost);
		}
#endif // UE_POSE_SEARCH_TRACE_ENABLED
	}
}

static void IterateOverBlendSpaceSamplingParameter(const UBlendSpace* BlendSpace, bool bUseSingleSample, const FVector& SingleSampleBlendParameters,
	bool bUseGridForSampling, int32 NumberOfHorizontalSamples, int32 NumberOfVerticalSamples,
	const TFunction<void(const FVector& BlendParameters)>& ProcessSamplingParameter)
{
	if (ensure(BlendSpace))
	{
		if (bUseSingleSample)
		{
			ProcessSamplingParameter(SingleSampleBlendParameters);
		}
		else if (bUseGridForSampling)
		{
			for (const FBlendSample& BlendSample : BlendSpace->GetBlendSamples())
			{
				ProcessSamplingParameter(BlendSample.SampleValue);
			}
		}
		else
		{
			const int32 HorizontalBlendNum = FMath::Max(NumberOfHorizontalSamples, 1);
			const int32 VerticalBlendNum = BlendSpace->IsA<UBlendSpace1D>() ? 1 : FMath::Max(NumberOfVerticalSamples, 1);

			const FBlendParameter& HorizontalBlendParameter = BlendSpace->GetBlendParameter(0);
			const FBlendParameter& VerticalBlendParameter = BlendSpace->GetBlendParameter(1);

			const int32 WrapInputHorizontalBlendNum = HorizontalBlendParameter.bWrapInput ? HorizontalBlendNum + 1 : HorizontalBlendNum;
			const int32 WrapInputVerticalBlendNum = VerticalBlendParameter.bWrapInput ? VerticalBlendNum + 1 : VerticalBlendNum;

			for (int32 HorizontalBlendIndex = 0; HorizontalBlendIndex < HorizontalBlendNum; HorizontalBlendIndex++)
			{
				for (int32 VerticalBlendIndex = 0; VerticalBlendIndex < VerticalBlendNum; VerticalBlendIndex++)
				{
					const FVector BlendParameters = FVector(
						WrapInputHorizontalBlendNum > 1 ?
						HorizontalBlendParameter.Min + (HorizontalBlendParameter.Max - HorizontalBlendParameter.Min) * ((float)HorizontalBlendIndex) / (WrapInputHorizontalBlendNum - 1) :
						HorizontalBlendParameter.Min,
						WrapInputVerticalBlendNum > 1 ?
						VerticalBlendParameter.Min + (VerticalBlendParameter.Max - VerticalBlendParameter.Min) * ((float)VerticalBlendIndex) / (WrapInputVerticalBlendNum - 1) :
						VerticalBlendParameter.Min,
						0.f);

					ProcessSamplingParameter(BlendParameters);
				}
			}
		}
	}
}

static bool IsBlendSpaceRootMotionEnabled(const UBlendSpace* BlendSpace)
{
	bool bIsRootMotionUsedInBlendSpace = false;
	if (ensure(BlendSpace))
	{
		BlendSpace->ForEachImmutableSample([&bIsRootMotionUsedInBlendSpace](const FBlendSample& Sample)
			{
				const UAnimSequence* Sequence = Sample.Animation.Get();
				if (IsValid(Sequence) && Sequence->HasRootMotion())
				{
					bIsRootMotionUsedInBlendSpace = true;
				}
			});
	}
	return bIsRootMotionUsedInBlendSpace;
}

} // namespace UE::PoseSearch

//////////////////////////////////////////////////////////////////////////
// FPoseSearchDatabaseAnimationAssetBase
float FPoseSearchDatabaseAnimationAssetBase::GetPlayLength(const FVector& BlendParameters) const
{
	if (const UAnimationAsset* AnimationAsset = Cast<UAnimationAsset>(GetAnimationAsset()))
	{
		return AnimationAsset->GetPlayLength();
	}
	return 0;
}

#if WITH_EDITOR
bool FPoseSearchDatabaseAnimationAssetBase::IsSkeletonCompatible(TObjectPtr<const UPoseSearchSchema> InSchema) const
{
	if (!InSchema)
	{
		return false;
	}

	if (!GetAnimationAsset())
	{
		return true;
	}

	const TArray<FPoseSearchRoledSkeleton>& RoledSkeletons = InSchema->GetRoledSkeletons();
	const int32 NumRoles = GetNumRoles();
	int32 NumValidRoles = 0;

	for (int RoleIdx = 0; RoleIdx < NumRoles; ++RoleIdx)
	{
		const UE::PoseSearch::FRole Role = GetRole(RoleIdx);
		const FAssetData AssetData = IAssetRegistry::Get()->GetAssetByObjectPath(FSoftObjectPath(GetAnimationAssetForRole(Role)));

		for (const FPoseSearchRoledSkeleton& RoledSkeleton : RoledSkeletons)
		{
			if (RoledSkeleton.Role == Role && RoledSkeleton.Skeleton && RoledSkeleton.Skeleton->IsCompatibleForEditor(AssetData))
			{
				++NumValidRoles;
			}
		}
	}

	return NumValidRoles == NumRoles;
}

bool FPoseSearchDatabaseAnimationAssetBase::IsMirrorOptionCompatible(TObjectPtr<const UPoseSearchSchema> InSchema) const
{
	if (!InSchema)
	{
		return false;
	}

	if (MirrorOption == EPoseSearchMirrorOption::UnmirroredOnly)
	{
		return true;
	}

	if (!GetAnimationAsset())
	{
		return true;
	}

	const TArray<FPoseSearchRoledSkeleton>& RoledSkeletons = InSchema->GetRoledSkeletons();
	const int32 NumRoles = GetNumRoles();
	int32 NumValidRoles = 0;

	for (int RoleIdx = 0; RoleIdx < NumRoles; ++RoleIdx)
	{
		const UE::PoseSearch::FRole Role = GetRole(RoleIdx);

		for (const FPoseSearchRoledSkeleton& RoledSkeleton : RoledSkeletons)
		{
			if (RoledSkeleton.Role == Role && RoledSkeleton.MirrorDataTable)
			{
				++NumValidRoles;
			}
		}
	}

	return NumValidRoles == NumRoles;
}

USkeletalMesh* FPoseSearchDatabaseAnimationAssetBase::GetPreviewMeshForRole(const UE::PoseSearch::FRole& Role) const
{
	return nullptr;
}

void FPoseSearchDatabaseAnimationAssetBase::IterateOverSamplingParameter(const TFunction<void(const FVector& BlendParameters)>& ProcessSamplingParameter) const
{
	if (GetAnimationAsset())
	{
		ProcessSamplingParameter(FVector::ZeroVector);
	}
}

#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
const FString FPoseSearchDatabaseAnimationAssetBase::GetName() const
{
	return GetNameSafe(GetAnimationAsset());
}
#endif //WITH_EDITORONLY_DATA

UAnimationAsset* FPoseSearchDatabaseAnimationAssetBase::GetAnimationAssetForRole(const UE::PoseSearch::FRole& Role) const
{
	ensure(GetNumRoles() == 1);
	return Cast<UAnimationAsset>(GetAnimationAsset());
}

FTransform FPoseSearchDatabaseAnimationAssetBase::GetRootTransformOriginForRole(const UE::PoseSearch::FRole& Role) const
{
	ensure(GetNumRoles() == 1);
	return FTransform::Identity;
}

#if WITH_EDITORONLY_DATA

bool FPoseSearchDatabaseAnimationAssetBase::UpdateFrom(const FPoseSearchDatabaseAnimationAssetBase& Source)
{
	if (BranchInId != 0 && BranchInId == Source.BranchInId)
	{
		SetSamplingRange(Source.GetSamplingRange());
		return true;
	}
	return false;
}

int64 FPoseSearchDatabaseAnimationAssetBase::GetEditorMemSize() const
{
	FArchiveCountMem EditorMemCount(GetAnimationAsset());
	return EditorMemCount.GetNum();
}

FFloatInterval FPoseSearchDatabaseAnimationAssetBase::GetEffectiveSamplingRange(const FVector& BlendParameters) const
{
	return GetEffectiveSamplingRange(GetPlayLength(BlendParameters), GetSamplingRange());
}

FFloatInterval FPoseSearchDatabaseAnimationAssetBase::GetEffectiveSamplingRange(float PlayLength, const FFloatInterval& SamplingRange)
{
	const bool bSampleAll = (SamplingRange.Min == 0.0f) && (SamplingRange.Max == 0.0f);
	FFloatInterval Range;
	Range.Min = bSampleAll ? 0.0f : SamplingRange.Min;
	Range.Max = bSampleAll ? PlayLength : FMath::Min(PlayLength, SamplingRange.Max);

	if (Range.Min > Range.Max)
	{
		UE_LOGF(LogPoseSearch, Warning, "Sampling range minimum (%f) is greated than max (%f). Setting min to be equal to max.", Range.Min, Range.Max)
		
		Range.Min = Range.Max;
	}

	return Range;
}
#endif // WITH_EDITORONLY_DATA

//////////////////////////////////////////////////////////////////////////
// FPoseSearchDatabaseAnimationAsset
UObject* FPoseSearchDatabaseAnimationAsset::GetAnimationAsset() const
{
	return AnimAsset.Get();
}

float FPoseSearchDatabaseAnimationAsset::GetPlayLength(const FVector& BlendParameters) const
{
	if (!AnimAsset)
	{
		return 0.f;
	}

	if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(AnimAsset))
	{
		int32 TriangulationIndex = 0;
		TArray<FBlendSampleData> BlendSamples;
		BlendSpace->GetSamplesFromBlendInput(BlendParameters, BlendSamples, TriangulationIndex, true);
		const float PlayLength = BlendSpace->GetAnimationLengthFromSampleData(BlendSamples);
		return PlayLength;
	}

	if (const UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(AnimAsset))
	{
		return MultiAnimAsset->GetPlayLength(BlendParameters);
	}

	return FPoseSearchDatabaseAnimationAssetBase::GetPlayLength(BlendParameters);
}

int32 FPoseSearchDatabaseAnimationAsset::GetNumRoles() const
{
	if (const UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(AnimAsset))
	{
		return MultiAnimAsset->GetNumRoles();
	}
	return FPoseSearchDatabaseAnimationAssetBase::GetNumRoles();
}

UE::PoseSearch::FRole FPoseSearchDatabaseAnimationAsset::GetRole(int32 RoleIndex) const
{
	if (const UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(AnimAsset))
	{
		return MultiAnimAsset->GetRole(RoleIndex);
	}
	return FPoseSearchDatabaseAnimationAssetBase::GetRole(RoleIndex);
}

UAnimationAsset* FPoseSearchDatabaseAnimationAsset::GetAnimationAssetForRole(const UE::PoseSearch::FRole& Role) const
{
	if (const UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(AnimAsset))
	{
		return MultiAnimAsset->GetAnimationAsset(Role);
	}
	return FPoseSearchDatabaseAnimationAssetBase::GetAnimationAssetForRole(Role);
}

FTransform FPoseSearchDatabaseAnimationAsset::GetRootTransformOriginForRole(const UE::PoseSearch::FRole& Role) const
{
	if (const UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(AnimAsset))
	{
		return MultiAnimAsset->GetOrigin(Role);
	}
	return FPoseSearchDatabaseAnimationAssetBase::GetRootTransformOriginForRole(Role);
}

#if WITH_EDITORONLY_DATA
// @todo: Fix me!
UClass* FPoseSearchDatabaseAnimationAsset::GetAnimationAssetStaticClass() const
{
	if (!AnimAsset)
	{
		return nullptr;
	}

	return AnimAsset->GetClass();
}

bool FPoseSearchDatabaseAnimationAsset::IsLooping(const UObject* AnimAsset, const FFloatInterval& SamplingRange)
{
	if (!AnimAsset)
	{
		return false;
	}

	if (const UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(AnimAsset))
	{
		return AnimSequenceBase->bLoop && SamplingRange.Min == 0.f && SamplingRange.Max == 0.f;
	}

	if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(AnimAsset))
	{
		return BlendSpace->bLoop;
	}

	if (const UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(AnimAsset))
	{
		return MultiAnimAsset->IsLooping() && SamplingRange.Min == 0.f && SamplingRange.Max == 0.f;
	}

	return false;
}

bool FPoseSearchDatabaseAnimationAsset::IsLooping() const
{
	return IsLooping(AnimAsset, SamplingRange);
}

bool FPoseSearchDatabaseAnimationAsset::IsRootMotionEnabled(const UObject* AnimAsset)
{
	if (!AnimAsset)
	{
		return false;
	}

	if (const UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(AnimAsset))
	{
		return AnimSequenceBase->HasRootMotion();
	}

	if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(AnimAsset))
	{
		return UE::PoseSearch::IsBlendSpaceRootMotionEnabled(BlendSpace);
	}

	if (const UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(AnimAsset))
	{
		return MultiAnimAsset->HasRootMotion();
	}

	return false;
}

bool FPoseSearchDatabaseAnimationAsset::IsRootMotionEnabled() const
{
	return IsRootMotionEnabled(AnimAsset);
}

FFloatInterval FPoseSearchDatabaseAnimationAsset::GetEffectiveSamplingRange(const FVector& BlendParameters) const
{
	if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(AnimAsset))
	{
		const float PlayLength = GetPlayLength(BlendParameters);

		// scaling blend space SamplingRange from the space [0, 1] to [0, PlayLength] with PlayLength calculated from the BlendSamples
		FFloatInterval ScaledSamplingRange = SamplingRange;
		ScaledSamplingRange.Min *= PlayLength;
		ScaledSamplingRange.Max *= PlayLength;

		return FPoseSearchDatabaseAnimationAssetBase::GetEffectiveSamplingRange(PlayLength, ScaledSamplingRange);
	}

	return FPoseSearchDatabaseAnimationAssetBase::GetEffectiveSamplingRange(BlendParameters);
}

void FPoseSearchDatabaseAnimationAsset::IterateOverSamplingParameter(const TFunction<void(const FVector& BlendParameters)>& ProcessSamplingParameter) const
{
	if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(AnimAsset))
	{
		UE::PoseSearch::IterateOverBlendSpaceSamplingParameter(BlendSpace, bUseSingleSample, FVector(BlendParamX, BlendParamY, 0.f),
			bUseGridForSampling, NumberOfHorizontalSamples, NumberOfVerticalSamples, ProcessSamplingParameter);
	}
	else if (const UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(AnimAsset))
	{
		const UBlendSpace* MultiAnimAssetBlendSpace = nullptr;
		for (int32 RoleIndex = 0; RoleIndex < MultiAnimAsset->GetNumRoles(); ++RoleIndex)
		{
			if (const UAnimationAsset* AnimationAsset = MultiAnimAsset->GetAnimationAsset(MultiAnimAsset->GetRole(RoleIndex)))
			{
				if (AnimationAsset->GetClass()->IsChildOf(UBlendSpace::StaticClass()))
				{
					// @todo: right now we just sample using the first blend space, but we should probably make sure if there're multiple MultiAnimAssetBlendSpace they are consistent with each other
					MultiAnimAssetBlendSpace = Cast<UBlendSpace>(AnimationAsset);
					ensure(MultiAnimAssetBlendSpace);
					break;
				}
			}
		}

		if (MultiAnimAssetBlendSpace)
		{
			UE::PoseSearch::IterateOverBlendSpaceSamplingParameter(MultiAnimAssetBlendSpace, false, FVector::ZeroVector,
				false, NumberOfHorizontalSamples, NumberOfVerticalSamples, ProcessSamplingParameter);
		}
		else
		{
			ProcessSamplingParameter(FVector::ZeroVector);
		}
	}
	else
	{
		FPoseSearchDatabaseAnimationAssetBase::IterateOverSamplingParameter(ProcessSamplingParameter);
	}
}

#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
USkeletalMesh* FPoseSearchDatabaseAnimationAsset::GetPreviewMeshForRole(const UE::PoseSearch::FRole& Role) const
{
	if (UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(AnimAsset))
	{
		return MultiAnimAsset->GetPreviewMesh(Role);
	}
	return FPoseSearchDatabaseAnimationAssetBase::GetPreviewMeshForRole(Role);
}
#endif // WITH_EDITOR

PRAGMA_DISABLE_DEPRECATION_WARNINGS
//////////////////////////////////////////////////////////////////////////
// FPoseSearchDatabaseSequence
UObject* FPoseSearchDatabaseSequence::GetAnimationAsset() const
{
	return Sequence.Get();
}

#if WITH_EDITORONLY_DATA
UClass* FPoseSearchDatabaseSequence::GetAnimationAssetStaticClass() const
{
	return UAnimSequence::StaticClass();
}

bool FPoseSearchDatabaseSequence::IsLooping() const
{
	return Sequence &&
		Sequence->bLoop &&
		SamplingRange.Min == 0.f &&
		SamplingRange.Max == 0.f;
}

bool FPoseSearchDatabaseSequence::IsRootMotionEnabled() const
{
	return Sequence ? Sequence->HasRootMotion() : false;
}
#endif // WITH_EDITORONLY_DATA

//////////////////////////////////////////////////////////////////////////
// FPoseSearchDatabaseBlendSpace
UObject* FPoseSearchDatabaseBlendSpace::GetAnimationAsset() const
{
	return BlendSpace.Get();
}

float FPoseSearchDatabaseBlendSpace::GetPlayLength(const FVector& BlendParameters) const
{
	int32 TriangulationIndex = 0;
	TArray<FBlendSampleData> BlendSamples;
	BlendSpace->GetSamplesFromBlendInput(BlendParameters, BlendSamples, TriangulationIndex, true);
	const float PlayLength = BlendSpace->GetAnimationLengthFromSampleData(BlendSamples);
	return PlayLength;
}

#if WITH_EDITORONLY_DATA
UClass* FPoseSearchDatabaseBlendSpace::GetAnimationAssetStaticClass() const
{
	return UBlendSpace::StaticClass();
}

bool FPoseSearchDatabaseBlendSpace::IsLooping() const
{
	return BlendSpace && BlendSpace->bLoop;
}

bool FPoseSearchDatabaseBlendSpace::IsRootMotionEnabled() const
{
	if (BlendSpace)
	{
		return UE::PoseSearch::IsBlendSpaceRootMotionEnabled(BlendSpace);
	}
	return false;
}

FFloatInterval FPoseSearchDatabaseBlendSpace::GetEffectiveSamplingRange(const FVector& BlendParameters) const
{
	if (BlendSpace)
	{
		const float PlayLength = GetPlayLength(BlendParameters);

		// scaling blend space SamplingRange from the space [0, 1] to [0, PlayLength] with PlayLength calculated from the BlendSamples
		FFloatInterval ScaledSamplingRange = SamplingRange;
		ScaledSamplingRange.Min *= PlayLength;
		ScaledSamplingRange.Max *= PlayLength;

		return FPoseSearchDatabaseAnimationAssetBase::GetEffectiveSamplingRange(PlayLength, ScaledSamplingRange);
	}

	return FFloatInterval(0.f, 0.f);
}

void FPoseSearchDatabaseBlendSpace::IterateOverSamplingParameter(const TFunction<void(const FVector& BlendParameters)>& ProcessSamplingParameter) const
{
	if (BlendSpace)
	{
		UE::PoseSearch::IterateOverBlendSpaceSamplingParameter(BlendSpace, bUseSingleSample, FVector(BlendParamX, BlendParamY, 0.f),
			bUseGridForSampling, NumberOfHorizontalSamples, NumberOfVerticalSamples, ProcessSamplingParameter);
	}
}

#endif // WITH_EDITORONLY_DATA

//////////////////////////////////////////////////////////////////////////
// FPoseSearchDatabaseAnimComposite
UObject* FPoseSearchDatabaseAnimComposite::GetAnimationAsset() const
{
	return AnimComposite.Get();
}

#if WITH_EDITORONLY_DATA
UClass* FPoseSearchDatabaseAnimComposite::GetAnimationAssetStaticClass() const
{
	return UAnimComposite::StaticClass();
}

bool FPoseSearchDatabaseAnimComposite::IsLooping() const
{
	return AnimComposite &&
		AnimComposite->bLoop &&
		SamplingRange.Min == 0.f &&
		SamplingRange.Max == 0.f;
}

bool FPoseSearchDatabaseAnimComposite::IsRootMotionEnabled() const
{
	return AnimComposite ? AnimComposite->HasRootMotion() : false;
}
#endif // WITH_EDITORONLY_DATA

//////////////////////////////////////////////////////////////////////////
// FPoseSearchDatabaseAnimMontage
UObject* FPoseSearchDatabaseAnimMontage::GetAnimationAsset() const
{
	return AnimMontage.Get();
}

#if WITH_EDITORONLY_DATA
UClass* FPoseSearchDatabaseAnimMontage::GetAnimationAssetStaticClass() const
{
	return UAnimMontage::StaticClass();
}

bool FPoseSearchDatabaseAnimMontage::IsLooping() const
{
	return AnimMontage &&
		AnimMontage->bLoop &&
		SamplingRange.Min == 0.f &&
		SamplingRange.Max == 0.f;
}

bool FPoseSearchDatabaseAnimMontage::IsRootMotionEnabled() const
{
	return AnimMontage ? AnimMontage->HasRootMotion() : false;
}
#endif // WITH_EDITORONLY_DATA

//////////////////////////////////////////////////////////////////////////
// FPoseSearchDatabaseMultiAnimAsset
#if WITH_EDITOR
USkeletalMesh* FPoseSearchDatabaseMultiAnimAsset::GetPreviewMeshForRole(const UE::PoseSearch::FRole& Role) const
{
	return MultiAnimAsset ? MultiAnimAsset->GetPreviewMesh(Role) : nullptr;
}
#endif // WITH_EDITOR

UObject* FPoseSearchDatabaseMultiAnimAsset::GetAnimationAsset() const
{
	return MultiAnimAsset.Get();
}

float FPoseSearchDatabaseMultiAnimAsset::GetPlayLength(const FVector& BlendParameters) const
{
	return MultiAnimAsset ? MultiAnimAsset->GetPlayLength(BlendParameters) : 0.f;
}

int32 FPoseSearchDatabaseMultiAnimAsset::GetNumRoles() const
{
	return MultiAnimAsset ? MultiAnimAsset->GetNumRoles() : 0;
}

UE::PoseSearch::FRole FPoseSearchDatabaseMultiAnimAsset::GetRole(int32 RoleIndex) const
{
	return MultiAnimAsset ? MultiAnimAsset->GetRole(RoleIndex) : UE::PoseSearch::DefaultRole;
}

UAnimationAsset* FPoseSearchDatabaseMultiAnimAsset::GetAnimationAssetForRole(const UE::PoseSearch::FRole& Role) const
{
	return MultiAnimAsset ? MultiAnimAsset->GetAnimationAsset(Role) : nullptr;
}

FTransform FPoseSearchDatabaseMultiAnimAsset::GetRootTransformOriginForRole(const UE::PoseSearch::FRole& Role) const
{
	return MultiAnimAsset ? MultiAnimAsset->GetOrigin(Role) : FTransform::Identity;
}

#if WITH_EDITORONLY_DATA
UClass* FPoseSearchDatabaseMultiAnimAsset::GetAnimationAssetStaticClass() const
{
	return UMultiAnimAsset::StaticClass();
}

bool FPoseSearchDatabaseMultiAnimAsset::IsLooping() const
{
	return MultiAnimAsset &&
		MultiAnimAsset->IsLooping() &&
		SamplingRange.Min == 0.f &&
		SamplingRange.Max == 0.f;
}

bool FPoseSearchDatabaseMultiAnimAsset::IsRootMotionEnabled() const
{
	return MultiAnimAsset ? MultiAnimAsset->HasRootMotion() : false;
}

void FPoseSearchDatabaseMultiAnimAsset::IterateOverSamplingParameter(const TFunction<void(const FVector& BlendParameters)>& ProcessSamplingParameter) const
{
	if (MultiAnimAsset)
	{
		const UBlendSpace* BlendSpace = nullptr;
		for (int32 RoleIndex = 0; RoleIndex < MultiAnimAsset->GetNumRoles(); ++RoleIndex)
		{
			if (const UAnimationAsset* AnimationAsset = MultiAnimAsset->GetAnimationAsset(MultiAnimAsset->GetRole(RoleIndex)))
			{
				if (AnimationAsset->GetClass()->IsChildOf(UBlendSpace::StaticClass()))
				{
					// @todo: right now we just sample using the first blend space, but we should probably make sure if there're multiple blendspaces they are consistent with each other
					BlendSpace = Cast<UBlendSpace>(AnimationAsset);
					ensure(BlendSpace);
					break;
				}
			}
		}

		if (BlendSpace)
		{
			UE::PoseSearch::IterateOverBlendSpaceSamplingParameter(BlendSpace, false, FVector::ZeroVector,
				false, NumberOfHorizontalSamples, NumberOfVerticalSamples, ProcessSamplingParameter);
		}
		else
		{
			ProcessSamplingParameter(FVector::ZeroVector);
		}
	}
}

#endif // WITH_EDITORONLY_DATA

//////////////////////////////////////////////////////////////////////////
// UPoseSearchDatabase
UPoseSearchDatabase::~UPoseSearchDatabase()
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UPoseSearchDatabase::SetSearchIndex(const UE::PoseSearch::FSearchIndex& SearchIndex)
{
	ensure(IsInGameThread());
	SearchIndexPrivate = SearchIndex;

	UpdateCachedProperties();
}

void UPoseSearchDatabase::UpdateCachedProperties()
{
	using namespace UE::PoseSearch;

	CachedAssetMap.Reset();
	for (int32 AssetIdx = 0; AssetIdx != SearchIndexPrivate.Assets.Num(); ++AssetIdx)
	{
		const FSearchIndexAsset& SearchIndexAsset = SearchIndexPrivate.Assets[AssetIdx];

		if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAssetBase = GetDatabaseAnimationAsset(SearchIndexAsset.GetSourceAssetIdx()))
		{
			CachedAssetMap.FindOrAdd(DatabaseAnimationAssetBase->GetAnimationAsset()).Add(AssetIdx);
		}
	}

	for (FCachedAssetMapPair& CachedAssetMapPair : CachedAssetMap)
	{
		CachedAssetMapPair.Value.Sort();
	}
}

TConstArrayView<int32> UPoseSearchDatabase::GetAssetIndexesForSourceAsset(const UObject* SourceAsset) const
{
	if (const TArray<int32>* IndexesForSourceAsset = CachedAssetMap.Find(SourceAsset))
	{
		return *IndexesForSourceAsset;
	}
	return TConstArrayView<int32>();
}

TConstArrayView<float> UPoseSearchDatabase::CalculateDynamicWeightsSqrt(TArrayView<float> DynamicWeightsSqrtBuffer) const
{
	using namespace UE::PoseSearch;
	const UE::PoseSearch::FSearchIndex& SearchIndex = GetSearchIndex();
	if (ensure(IsAligned(DynamicWeightsSqrtBuffer.GetData(), alignof(VectorRegister4Float))))
	{
		if (ensure(DynamicWeightsSqrtBuffer.Num() == SearchIndex.WeightsSqrt.Num()))
		{
			bool bInitialized = false;
			Schema->IterateChannels([&bInitialized, &SearchIndex, &DynamicWeightsSqrtBuffer](const UPoseSearchFeatureChannel* Channel)
				{
					if (const UPoseSearchFeatureChannel_GroupBase* ChannelGroupBase = Cast<UPoseSearchFeatureChannel_GroupBase>(Channel))
					{
						if (ChannelGroupBase->DebugWeightGroupID != INDEX_NONE && ChannelGroupBase->DebugWeightGroupID != GVarMotionMatchDebugWeightGroupID)
						{
							if (!bInitialized)
							{
								// initializing DynamicWeightsSqrtBuffer with the SearchIndex.WeightsSqrt
								FMemory::Memcpy(DynamicWeightsSqrtBuffer.GetData(), SearchIndex.WeightsSqrt.GetData(), SearchIndex.WeightsSqrt.Num() * sizeof(float));
								bInitialized = true;
							}

							// zeroing out interval of weights in DynamicWeightsSqrtBuffer associated with ChannelGroupBase since its GroupID mismatch ValidWeightChannelGroup
							FMemory::Memzero(DynamicWeightsSqrtBuffer.GetData() + ChannelGroupBase->GetChannelDataOffset(), ChannelGroupBase->GetChannelCardinality() * sizeof(float));
						}
					}
				});

			if (bInitialized)
			{
				return DynamicWeightsSqrtBuffer;
			}
		}
	}
	return SearchIndex.WeightsSqrt;
}

#if WITH_EDITORONLY_DATA
void UPoseSearchDatabase::AppendToClassSchema(FAppendToClassSchemaContext& Context)
{
	using namespace UE::PoseSearch;

	Super::AppendToClassSchema(Context);
	
	Context.Update(&DatabaseIndexDerivedDataCacheKeyVersion, sizeof(DatabaseIndexDerivedDataCacheKeyVersion));
	Context.Update(&FDevSystemGuids::Get().POSESEARCHDB_DERIVEDDATA_VER, sizeof(FDevSystemGuids::Get().POSESEARCHDB_DERIVEDDATA_VER));
}
#endif // WITH_EDITORONLY_DATA

const UE::PoseSearch::FSearchIndex& UPoseSearchDatabase::GetSearchIndex() const
{
	// making sure the search index is consistent. if it fails the calling code hasn't been protected by FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex
	ensure(Schema && !SearchIndexPrivate.IsEmpty() && SearchIndexPrivate.GetNumDimensions() == Schema->SchemaCardinality);
	return SearchIndexPrivate;
}

int32 UPoseSearchDatabase::GetPoseIndex(const UObject* AnimationAsset, float AnimationAssetTime, bool bMirrored, const FVector& BlendParameters, const UE::PoseSearch::FStackAssetIndexes* AssetIndexesToConsider) const
{
	using namespace UE::PoseSearch;

	int32 PoseIdx = INDEX_NONE;

	if (AnimationAsset)
	{
		float MinSquaredLength = UE_MAX_FLT;
		const float SampleRate = Schema->SampleRate;
		const TConstArrayView<int32> AssetIndexesForSourceAsset = GetAssetIndexesForSourceAsset(AnimationAsset);

		const FSearchIndex& SearchIndex = GetSearchIndex();
		for (int32 AssetIndex : AssetIndexesForSourceAsset)
		{
			const FSearchIndexAsset& SearchIndexAsset = SearchIndex.Assets[AssetIndex];
			const bool bIsAssetIndexValid = !AssetIndexesToConsider || AssetIndexesToConsider->Contains(SearchIndexAsset.GetSourceAssetIdx());
			if (bIsAssetIndexValid)
			{
				if (SearchIndexAsset.IsMirrored() == bMirrored)
				{
					// @todo: optimize this code, since there's no need to calculate BlendParametersSquaredLength if AssetIndexesForSourceAsset.Num() is 1
					const float BlendParametersSquaredLength = (BlendParameters - SearchIndexAsset.GetBlendParameters()).SquaredLength();

					if (BlendParametersSquaredLength <= MinSquaredLength)
					{
						const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAssetBase = GetDatabaseAnimationAsset(SearchIndexAsset.GetSourceAssetIdx());

						ensure(DatabaseAnimationAssetBase && DatabaseAnimationAssetBase->GetAnimationAsset() == AnimationAsset);

						const float RealAssetTime = AnimationAssetTime * SearchIndexAsset.GetToRealTimeFactor();
						const int32 CurrentIdx = SearchIndexAsset.GetPoseIndexFromTime(RealAssetTime, SampleRate);
						if (CurrentIdx != INDEX_NONE)
						{
							MinSquaredLength = BlendParametersSquaredLength;
							PoseIdx = CurrentIdx;
						}
					}
				}
			}
		}
	}

	return PoseIdx;
}

#if WITH_EDITOR
void UPoseSearchDatabase::AddAnimationAsset(FInstancedStruct AnimationAsset)
{
	using namespace UE::PoseSearch;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	AddAnimationAsset(UpgradeToDatabaseAnimationAsset(AnimationAsset));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UPoseSearchDatabase::AddAnimationAsset(const FPoseSearchDatabaseAnimationAsset& AnimationAsset)
{
	InsertAnimationAssetAt(AnimationAsset, GetNumAnimationAssets());
}

void UPoseSearchDatabase::InsertAnimationAssetAt(const FPoseSearchDatabaseAnimationAsset& AnimationAsset, int32 AnimationAssetIndex)
{
	if (DatabaseAnimationAssets.IsEmpty())
	{
		if (UChooserTable* Chooser = Cast<UChooserTable>(GetOuter()))
		{
			return FPoseSearchColumn::InsertAnimationAssetAt(AnimationAsset, AnimationAssetIndex, Chooser, this);
		}
	}

	ensure(AnimationAssetIndex >= 0 && AnimationAssetIndex <= DatabaseAnimationAssets.Num());
	DatabaseAnimationAssets.Insert(AnimationAsset, AnimationAssetIndex);
}

void UPoseSearchDatabase::RemoveAnimationAssetAt(int32 AnimationAssetIndex)
{
	if (DatabaseAnimationAssets.IsEmpty())
	{
		if (UChooserTable* Chooser = Cast<UChooserTable>(GetOuter()))
		{
			FPoseSearchColumn::RemoveAnimationAssetAt(AnimationAssetIndex, Chooser, this);
			return;
		}
	}

	ensure(DatabaseAnimationAssets.IsValidIndex(AnimationAssetIndex));
	DatabaseAnimationAssets.RemoveAt(AnimationAssetIndex);
}

void UPoseSearchDatabase::SetAnimationAssetAt(const FPoseSearchDatabaseAnimationAsset& AnimationAsset, int32 AnimationAssetIndex)
{
	if (DatabaseAnimationAssets.IsEmpty())
	{
		if (UChooserTable* Chooser = Cast<UChooserTable>(GetOuter()))
		{
			return FPoseSearchColumn::SetAnimationAssetAt(AnimationAsset, AnimationAssetIndex, Chooser, this);
		}
	}

	if (DatabaseAnimationAssets.IsValidIndex(AnimationAssetIndex))
	{
		DatabaseAnimationAssets[AnimationAssetIndex] = AnimationAsset;
	}
}
#endif // WITH_EDITOR

const FPoseSearchDatabaseAnimationAsset* UPoseSearchDatabase::GetDatabaseAnimationAsset(int32 AnimationAssetIndex) const
{
	if (DatabaseAnimationAssets.IsEmpty())
	{
		if (UChooserTable* Chooser = Cast<UChooserTable>(GetOuter()))
		{
			return FPoseSearchColumn::GetDatabaseAnimationAsset(AnimationAssetIndex, Chooser, this);
		}
	}

	if (DatabaseAnimationAssets.IsValidIndex(AnimationAssetIndex))
	{
		return &DatabaseAnimationAssets[AnimationAssetIndex];
	}
	return nullptr;
}

FPoseSearchDatabaseAnimationAsset* UPoseSearchDatabase::GetMutableDatabaseAnimationAsset(int32 AnimationAssetIndex)
{
	if (DatabaseAnimationAssets.IsEmpty())
	{
		if (UChooserTable* Chooser = Cast<UChooserTable>(GetOuter()))
		{
			// unimplemented
			ensure(false);
			return nullptr;
		}
	}

	if (DatabaseAnimationAssets.IsValidIndex(AnimationAssetIndex))
	{
		return &DatabaseAnimationAssets[AnimationAssetIndex];
	}
	return nullptr;
}

int32 UPoseSearchDatabase::GetNumAnimationAssets() const
{
	if (DatabaseAnimationAssets.IsEmpty())
	{
		if (UChooserTable* Chooser = Cast<UChooserTable>(GetOuter()))
		{
			return FPoseSearchColumn::GetNumAnimationAssets(Chooser, this);
		}
	}

	return DatabaseAnimationAssets.Num();
}

UObject* UPoseSearchDatabase::GetAnimationAsset(int32 Index) const
{
	if (const FPoseSearchDatabaseAnimationAsset* DatabaseAnimationAsset = GetDatabaseAnimationAsset(Index))
	{
		return DatabaseAnimationAsset->GetAnimationAsset();
	}
	return nullptr;
}

const FPoseSearchDatabaseAnimationAsset* UPoseSearchDatabase::GetDatabaseAnimationAsset(const UE::PoseSearch::FSearchIndexAsset& SearchIndexAsset) const
{
	return GetDatabaseAnimationAsset(SearchIndexAsset.GetSourceAssetIdx());
}

FPoseSearchDatabaseAnimationAsset* UPoseSearchDatabase::GetMutableDatabaseAnimationAsset(const UE::PoseSearch::FSearchIndexAsset& SearchIndexAsset)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GetMutableDatabaseAnimationAsset(SearchIndexAsset.GetSourceAssetIdx());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if WITH_EDITOR
int32 UPoseSearchDatabase::GetNumberOfPrincipalComponents() const
{
	return FMath::Min<int32>(NumberOfPrincipalComponents, Schema->SchemaCardinality);
}
#endif //WITH_EDITOR

bool UPoseSearchDatabase::GetSkipSearchIfPossible() const
{
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
	if (UE::PoseSearch::GVarMotionMatchCompareAgainstBruteForce)
	{
		return false;
	}
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
	return true;
}

void UPoseSearchDatabase::PostLoad()
{
	using namespace UE::PoseSearch;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!AnimationAssets_DEPRECATED.IsEmpty())
	{
		DatabaseAnimationAssets.Reserve(AnimationAssets_DEPRECATED.Num());
		for (FInstancedStruct& AnimationAsset : AnimationAssets_DEPRECATED)
		{
			DatabaseAnimationAssets.Add(UpgradeToDatabaseAnimationAsset(AnimationAsset));
		}
		AnimationAssets_DEPRECATED.Empty();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UpdateCachedProperties();

#if WITH_EDITOR

	// Delay indexing until BeginCacheForCookedPlatformData when running a CookCommandlet
	if (GVarMotionMatchRequestAsyncBuildIndexOnPostLoad && !IsRunningCookCommandlet())
	{
		ERequestAsyncBuildFlag Flag = ERequestAsyncBuildFlag::NewRequest;
#if WITH_ENGINE
		// @todo: after CL 39338693 waiting for completion on a UPoseSearchDatabase is no longer possible,
		//        because UAnimSequence are not implemented to wait for their DDC tasks (IAnimSequenceCompilingManager::FinishCompilation) in their PostLoad
		//        leading FKeyBuilder::TryAddDependency not being able to compose a DDC key for the database, therefore not being able to complete the indexing.
		// 
		// If there isn't an EditorEngine (ex. Standalone Game via -game argument) we WaitForCompletion
		//if (Cast<UEditorEngine>(GEngine) == nullptr)
		//{
		//	Flag |= ERequestAsyncBuildFlag::WaitForCompletion;
		//}
#endif // WITH_ENGINE
	
		FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(this, Flag);
	}
#endif

	Super::PostLoad();
}

bool UPoseSearchDatabase::Contains(const UObject* Object) const
{
	return !GetAssetIndexesForSourceAsset(Object).IsEmpty();
}

#if WITH_EDITOR
void UPoseSearchDatabase::SynchronizeWithExternalDependencies()
{
	TArray<FTopLevelAssetPath> AncestorClassNames;

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

	TArray<FAssetIdentifier> Referencers;
	AssetRegistry.GetReferencers(GetPackage()->GetFName(), Referencers);

	// Sort AssetRegistry results
	Algo::Sort(Referencers, [](FAssetIdentifier One, FAssetIdentifier Two) -> bool
	{
		return One.PackageName.Compare(Two.PackageName) < 0;
	});

	TArray<UAnimSequenceBase*> SequencesBase;
	for (const FAssetIdentifier& Referencer : Referencers)
	{
		TArray<FAssetData> Assets;
		const bool bSearchOnDiskAssetsOnly = IsRunningCookCommandlet();		// When cooking, only enumerate on-disk assets to ensure deterministic results
		AssetRegistry.GetAssetsByPackageName(Referencer.PackageName, Assets, bSearchOnDiskAssetsOnly);

		for (const FAssetData& Asset : Assets)
		{
			if (Asset.IsInstanceOf(UAnimSequenceBase::StaticClass()))
			{
				if (UAnimSequenceBase* SequenceBase = Cast<UAnimSequenceBase>(Asset.FastGetAsset(true)))
				{
					for (const FAnimNotifyEvent& NotifyEvent : SequenceBase->Notifies)
					{
						if (const UAnimNotifyState_PoseSearchBranchIn* BranchIn = Cast<UAnimNotifyState_PoseSearchBranchIn>(NotifyEvent.NotifyStateClass))
						{
							if (BranchIn->Database == this)
							{
								SequencesBase.AddUnique(SequenceBase);
								break;
							}
						}
					}
				}
			}
		}
	}

	if (!SequencesBase.IsEmpty())
	{
		SynchronizeWithExternalDependencies(SequencesBase);
	}
}

void UPoseSearchDatabase::SynchronizeWithExternalDependencies(TConstArrayView<UAnimSequenceBase*> SequencesBase)
{
	// cannot use TSet since FInstancedStruct doesn't implement GetTypeHash
	TArray<FPoseSearchDatabaseAnimationAsset> NewDatabaseAnimationAssets = DatabaseAnimationAssets;
	TArray<FPoseSearchDatabaseAnimationAsset> TempDatabaseAnimationAssets;

	// collecting all the database AnimationAsset(s) that don't require synchronization
	for (FPoseSearchDatabaseAnimationAsset& DatabaseAnimationAsset : NewDatabaseAnimationAssets)
	{
		const bool bRequiresSynchronization = DatabaseAnimationAsset.IsSynchronizedWithExternalDependency() && SequencesBase.Contains(DatabaseAnimationAsset.GetAnimationAsset());
		if (!bRequiresSynchronization)
		{
			TempDatabaseAnimationAssets.Add(DatabaseAnimationAsset);
		}
	}

	// collecting all the SequencesBase(s) requiring synchronization
	for (UAnimSequenceBase* SequenceBase : SequencesBase)
	{
		if (SequenceBase)
		{
			UPackage* SequenceBasePackage = SequenceBase->GetPackage();
			if (!SequenceBasePackage)
			{
				UE_LOGF(LogPoseSearch, Warning, "AnimSequenceBase '%ls' cannot be synchronized with PoseSearchDatabase '%ls' because it's not in a package", *SequenceBase->GetName(), *GetName());
			}
			else if (SequenceBasePackage->HasAnyFlags(RF_Transient | RF_DuplicateTransient | RF_MirroredGarbage))
			{
				UE_LOGF(LogPoseSearch, Warning, "AnimSequenceBase '%ls' cannot be synchronized with PoseSearchDatabase '%ls' because it's in a package transient or garbaged package", *SequenceBase->GetName(), *GetName());
			}
			else if (SequenceBase->HasAnyFlags(RF_Transient | RF_DuplicateTransient | RF_MirroredGarbage))
			{
				UE_LOGF(LogPoseSearch, Warning, "AnimSequenceBase '%ls' cannot be synchronized with PoseSearchDatabase '%ls' because it's transient or garbaged", *SequenceBase->GetName(), *GetName());
			}
			else
			{
				for (const FAnimNotifyEvent& NotifyEvent : SequenceBase->Notifies)
				{
					if (const UAnimNotifyState_PoseSearchBranchIn* PoseSearchBranchIn = Cast<UAnimNotifyState_PoseSearchBranchIn>(NotifyEvent.NotifyStateClass))
					{
						if (PoseSearchBranchIn->Database == this)
						{
							auto GetSamplingRange = [](const FAnimNotifyEvent& NotifyEvent, const UAnimSequenceBase* SequenceBase) -> FFloatInterval
							{
								FFloatInterval SamplingRange(NotifyEvent.GetTime(), NotifyEvent.GetTime() + NotifyEvent.GetDuration());
								if (SamplingRange.Min <= NotifyEvent.TriggerTimeOffset && SamplingRange.Max >= SequenceBase->GetPlayLength() - NotifyEvent.TriggerTimeOffset)
								{
									SamplingRange = FFloatInterval(0.f, 0.f);
								}
								return SamplingRange;
							};

							FPoseSearchDatabaseAnimationAsset DatabaseAnimationAsset;
							DatabaseAnimationAsset.AnimAsset = SequenceBase;
							DatabaseAnimationAsset.SamplingRange = GetSamplingRange(NotifyEvent, SequenceBase);
							DatabaseAnimationAsset.BranchInId = PoseSearchBranchIn->GetBranchInId();
							TempDatabaseAnimationAssets.Add(DatabaseAnimationAsset);
						}
					}
				}
			}
		}
	}

	// updating NewDatabaseAnimationAssets from TempDatabaseAnimationAssets preserving the original sorting
	bool bModified = false;
	for (int32 AnimationAssetIndex = GetNumAnimationAssets() - 1; AnimationAssetIndex >= 0; --AnimationAssetIndex)
	{
		FPoseSearchDatabaseAnimationAsset& DatabaseAnimationAsset = NewDatabaseAnimationAssets[AnimationAssetIndex];
		
		int32 FoundIndex = -1;
		for(int i=0; i < TempDatabaseAnimationAssets.Num(); i++)
		{
			const FPoseSearchDatabaseAnimationAssetBase& NewDatabaseAnimationAsset = TempDatabaseAnimationAssets[i];
			if (DatabaseAnimationAsset.UpdateFrom(NewDatabaseAnimationAsset))
			{
				FoundIndex = i;
				break;
			}
			
		}
		
		if (FoundIndex >= 0)
		{
			TempDatabaseAnimationAssets.RemoveAt(FoundIndex); 
		}
		else
		{
			NewDatabaseAnimationAssets.RemoveAt(AnimationAssetIndex);
		}
	}

	// adding the remaining TempDatabaseAnimationAsset(s) from TempDatabaseAnimationAssets
	for (const FPoseSearchDatabaseAnimationAsset& TempDatabaseAnimationAsset : TempDatabaseAnimationAssets)
	{
		NewDatabaseAnimationAssets.Add(TempDatabaseAnimationAsset);
	}

	if (DatabaseAnimationAssets != NewDatabaseAnimationAssets)
	{
		Modify();

		DatabaseAnimationAssets = NewDatabaseAnimationAssets;

		PostEditChange();

		NotifySynchronizeWithExternalDependencies();
	}
}

void UPoseSearchDatabase::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	using namespace UE::PoseSearch;
	Super::BeginCacheForCookedPlatformData(TargetPlatform);
	FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(this, ERequestAsyncBuildFlag::NewRequest);
}

bool UPoseSearchDatabase::IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform)
{
	using namespace UE::PoseSearch;
	ensure(IsInGameThread());
	return EAsyncBuildIndexResult::InProgress != FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(this, ERequestAsyncBuildFlag::ContinueRequest);
}
#endif // WITH_EDITOR

#if WITH_EDITOR && ENABLE_ANIM_DEBUG
void UPoseSearchDatabase::TestSynchronizeWithExternalDependencies()
{
	TArray<FPoseSearchDatabaseAnimationAsset> DatabaseAnimationAssetsCopy = DatabaseAnimationAssets;
	
	SynchronizeWithExternalDependencies();

	if (DatabaseAnimationAssetsCopy != DatabaseAnimationAssets)
	{
		UE_LOGF(LogPoseSearch, Error, "TestSynchronizeWithExternalDependencies failed");
		DatabaseAnimationAssets = DatabaseAnimationAssetsCopy;
	}
}
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG

void UPoseSearchDatabase::PreSaveRoot(FObjectPreSaveRootContext ObjectSaveContext)
{
#if WITH_EDITOR
	FScopedTransaction ScopedTransaction(LOCTEXT("PoseSearchDatabaseSynchronizeWithExternalDependencies", "Pose Search Database Synchronize With External Dependencies"));
	// in case the database desynchronized with the UAnimNotifyState_PoseSearchBranchIn referencing it, we need to resynchronize
	SynchronizeWithExternalDependencies();
#endif

	Super::PreSaveRoot(ObjectSaveContext);
}

void UPoseSearchDatabase::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FPoseSearchCustomVersion::GUID);

	const bool bSavingCooked = Ar.IsSaving() && Ar.IsCooking();
#if WITH_EDITORONLY_DATA
	// When cooking, strip AnimAsset from disabled entries so they aren't loaded at runtime
	if (bSavingCooked)
	{
		TArray<TObjectPtr<UObject>> SavedAnimAssets;
		SavedAnimAssets.Reserve(DatabaseAnimationAssets.Num());
		for (FPoseSearchDatabaseAnimationAsset& AnimationAsset : DatabaseAnimationAssets)
		{
			SavedAnimAssets.Add(AnimationAsset.AnimAsset);
			if (!AnimationAsset.bEnabled)
			{
				AnimationAsset.AnimAsset = nullptr;
			}
		}

		Super::Serialize(Ar);

		for (int32 i = 0; i < DatabaseAnimationAssets.Num(); ++i)
		{
			DatabaseAnimationAssets[i].AnimAsset = SavedAnimAssets[i];
		}
	}
	else
#endif // WITH_EDITORONLY_DATA
	{
		Super::Serialize(Ar);
	}

	if (!IsTemplate())
	{
		const bool bLoadingCooked = Ar.IsLoadingFromCookedPackage();
		if (bSavingCooked || bLoadingCooked)
		{
			Ar << SearchIndexPrivate;
		}
	}
}

float UPoseSearchDatabase::GetRealAssetTime(int32 PoseIdx) const
{
	if (ensure(Schema))
	{
		const UE::PoseSearch::FSearchIndexAsset& Asset = GetSearchIndex().GetAssetForPose(PoseIdx);
		return Asset.GetTimeFromPoseIndex(PoseIdx, Schema->SampleRate);
	}
	return 0.f;
}

float UPoseSearchDatabase::GetNormalizedAssetTime(int32 PoseIdx) const
{
	if (ensure(Schema))
	{
		const UE::PoseSearch::FSearchIndexAsset& Asset = GetSearchIndex().GetAssetForPose(PoseIdx);
		const float ToRealTimeFactor = Asset.GetToRealTimeFactor();
		if (ensure(ToRealTimeFactor > UE_KINDA_SMALL_NUMBER))
		{
			const float RealAssetTime = Asset.GetTimeFromPoseIndex(PoseIdx, Schema->SampleRate);
			const float NormalizedAssetTime = RealAssetTime / ToRealTimeFactor;
			return NormalizedAssetTime;
		}
	}
	return 0.f;
}

void UPoseSearchDatabase::Search(UE::PoseSearch::FSearchContext& SearchContext, UE::PoseSearch::FSearchResults& SearchResults) const
{
	using namespace UE::PoseSearch;

#if WITH_EDITOR
	if (EAsyncBuildIndexResult::Success != FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(this, ERequestAsyncBuildFlag::ContinueRequest))
	{
		SearchContext.SetAsyncBuildIndexInProgress();
		return;
	}
#else
	if (SearchIndexPrivate.IsEmpty())
	{
		return;
	}
#endif // WITH_EDITOR

	if (SearchContext.GetEventToSearch().IsValid())
	{
		SearchEvent(SearchContext, SearchResults);
	}
	else
	{
		switch (PoseSearchMode)
		{
		case EPoseSearchMode::BruteForce:
			SearchBruteForce(SearchContext, SearchResults);
			break;
		case EPoseSearchMode::VPTree:
			SearchVPTree(SearchContext, SearchResults);
			break;
		case EPoseSearchMode::PCAKDTree:
			SearchPCAKDTree(SearchContext, SearchResults);
			break;
		case EPoseSearchMode::EventOnly:
			break;
		}

#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
		if (GVarMotionMatchCompareAgainstBruteForce)
		{
			FPoseSearchCost BruteForcePoseCost;
			if (PoseSearchMode == EPoseSearchMode::VPTree || PoseSearchMode == EPoseSearchMode::PCAKDTree)
			{
				FSearchResults_Single BruteForceSearchResults;
				SearchBruteForce(SearchContext, BruteForceSearchResults);
				BruteForcePoseCost = BruteForceSearchResults.GetBestResult().PoseCost;
			}
			else
			{
				BruteForcePoseCost = SearchResults.GetBestResult().PoseCost;
			}

			SearchResults.IterateOverSearchResults([&BruteForcePoseCost](FSearchResult& SearchResult)
				{
					if (BruteForcePoseCost < SearchResult.BruteForcePoseCost)
					{
						SearchResult.BruteForcePoseCost = BruteForcePoseCost;
					}
					return false;
				});
		}
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
	}
}

void UPoseSearchDatabase::PopulateNonSelectableIdx(FNonSelectableIdx& NonSelectableIdx, UE::PoseSearch::FSearchContext& SearchContext
#if UE_POSE_SEARCH_TRACE_ENABLED
	, float ContinuingPoseCostAddend, float ContinuingInteractionCostAddend, float ContinuingContextInteractionCostAddend, TConstArrayView<float> DynamicWeightsSqrt
#endif //UE_POSE_SEARCH_TRACE_ENABLED
) const
{
	using namespace UE::PoseSearch;

	const FSearchIndex& SearchIndex = GetSearchIndex();

	NonSelectableIdx.Reset();
	if (SearchContext.IsContinuingPoseDatabase(this))
	{
		if (const FSearchIndexAsset* CurrentIndexAsset = SearchContext.GetContinuingPoseSearchResult().GetSearchIndexAsset(true))
		{
			if (CurrentIndexAsset->IsDisableReselection())
			{
				// excluding all the poses with CurrentIndexAsset->GetSourceAssetIdx()
				const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAssetBase = GetDatabaseAnimationAsset(CurrentIndexAsset->GetSourceAssetIdx());
				if (ensure(DatabaseAnimationAssetBase))
				{
					for (int32 AssetIndex : GetAssetIndexesForSourceAsset(DatabaseAnimationAssetBase->GetAnimationAsset()))
					{
						const FSearchIndexAsset& SearchIndexAsset = SearchIndex.Assets[AssetIndex];
						const int32 FirstPoseIdx = SearchIndexAsset.GetFirstPoseIdx();
						const int32 LastPoseIdx = FirstPoseIdx + SearchIndexAsset.GetNumPoses();
						for (int32 PoseIdx = FirstPoseIdx; PoseIdx < LastPoseIdx; ++PoseIdx)
						{
							// no need to AddUnique since there's no overlapping between pose indexes in the FSearchIndexAsset(s)
							NonSelectableIdx.Add(PoseIdx);

#if UE_POSE_SEARCH_TRACE_ENABLED
							SearchContext.Track(this, PoseIdx, EPoseCandidateFlags::DiscardedBy_AssetReselection, DynamicWeightsSqrt, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend);
#endif // UE_POSE_SEARCH_TRACE_ENABLED
						}
					}
				}
			}
			else if (!FMath::IsNearlyEqual(SearchContext.GetPoseJumpThresholdTime().Min, SearchContext.GetPoseJumpThresholdTime().Max))
			{
				const int32 ContinuingPoseIdx = SearchContext.GetContinuingPoseSearchResult().PoseIdx;
				if (ensure(ContinuingPoseIdx != INDEX_NONE))
				{
					const int32 UnboundMinPoseIdx = ContinuingPoseIdx + FMath::FloorToInt(SearchContext.GetPoseJumpThresholdTime().Min * Schema->SampleRate);
					const int32 UnboundMaxPoseIdx = ContinuingPoseIdx + FMath::CeilToInt(SearchContext.GetPoseJumpThresholdTime().Max * Schema->SampleRate);
					const int32 CurrentIndexAssetFirstPoseIdx = CurrentIndexAsset->GetFirstPoseIdx();
					const int32 CurrentIndexAssetNumPoses = CurrentIndexAsset->GetNumPoses();
					const bool bIsLooping = CurrentIndexAsset->IsLooping();

					if (bIsLooping)
					{
						for (int32 UnboundPoseIdx = UnboundMinPoseIdx; UnboundPoseIdx < UnboundMaxPoseIdx; ++UnboundPoseIdx)
						{
							const int32 Modulo = (UnboundPoseIdx - CurrentIndexAssetFirstPoseIdx) % CurrentIndexAssetNumPoses;
							const int32 CurrentIndexAssetFirstPoseIdxPlusModulo = CurrentIndexAssetFirstPoseIdx + Modulo;
							const int32 PoseIdx = Modulo >= 0 ? CurrentIndexAssetFirstPoseIdxPlusModulo : CurrentIndexAssetFirstPoseIdxPlusModulo + CurrentIndexAssetNumPoses;

							NonSelectableIdx.AddUnique(PoseIdx);

#if UE_POSE_SEARCH_TRACE_ENABLED
							SearchContext.Track(this, PoseIdx, EPoseCandidateFlags::DiscardedBy_PoseJumpThresholdTime, DynamicWeightsSqrt, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend);
#endif // UE_POSE_SEARCH_TRACE_ENABLED
						}
					}
					else
					{
						const int32 MinPoseIdx = FMath::Max(CurrentIndexAssetFirstPoseIdx, UnboundMinPoseIdx);
						const int32 MaxPoseIdx = FMath::Min(CurrentIndexAssetFirstPoseIdx + CurrentIndexAssetNumPoses, UnboundMaxPoseIdx);

						for (int32 PoseIdx = MinPoseIdx; PoseIdx < MaxPoseIdx; ++PoseIdx)
						{
							NonSelectableIdx.AddUnique(PoseIdx);
#if UE_POSE_SEARCH_TRACE_ENABLED
							SearchContext.Track(this, PoseIdx, EPoseCandidateFlags::DiscardedBy_PoseJumpThresholdTime, DynamicWeightsSqrt, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend);
#endif // UE_POSE_SEARCH_TRACE_ENABLED
						}
					}
				}
			}
		}
	}

	for (const IPoseHistory* PoseHistory : SearchContext.GetPoseHistories())
	{
		if (const FPoseIndicesHistory* PoseIndicesHistory = PoseHistory->GetPoseIndicesHistory())
		{
			const FObjectKey DatabaseKey(this);
			for (auto It = PoseIndicesHistory->IndexToTime.CreateConstIterator(); It; ++It)
			{
				const FHistoricalPoseIndex& HistoricalPoseIndex = It.Key();
				if (HistoricalPoseIndex.DatabaseKey == DatabaseKey)
				{
					NonSelectableIdx.AddUnique(HistoricalPoseIndex.PoseIndex);

#if UE_POSE_SEARCH_TRACE_ENABLED
					SearchContext.Track(this, HistoricalPoseIndex.PoseIndex, EPoseCandidateFlags::DiscardedBy_PoseReselectHistory, DynamicWeightsSqrt, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend);
#endif // UE_POSE_SEARCH_TRACE_ENABLED
				}
			}
		}
	}

	NonSelectableIdx.Sort();
}

void UPoseSearchDatabase::PopulateSelectableAssetIdx(FSelectableAssetIdx& SelectableAssetIdx, TConstArrayView<const UObject*> AssetsToConsider) const
{
	SelectableAssetIdx.Reset();
	if (!AssetsToConsider.IsEmpty())
	{
		for (const UObject* AssetToConsider : AssetsToConsider)
		{
			SelectableAssetIdx.Append(GetAssetIndexesForSourceAsset(AssetToConsider));
		}

		if (!SelectableAssetIdx.IsEmpty())
		{
			if (SelectableAssetIdx.Num() != GetSearchIndex().Assets.Num())
			{
				SelectableAssetIdx.Sort();
			}
			else
			{
				// SelectableAssetIdx contains ALL the Database->GetSearchIndex().Assets. 
				// We reset SelectableAssetIdx since it has the same meaning, and it'll perform better
				SelectableAssetIdx.Reset();
			}
		}
	}
}

void UPoseSearchDatabase::PopulateSelectableAssetIdx(FSelectableAssetIdx& SelectableAssetIdx, const UE::PoseSearch::FStackAssetSet* AssetsToConsider) const
{
	SelectableAssetIdx.Reset();
	if (ensure(AssetsToConsider) && !AssetsToConsider->IsEmpty())
	{
		for (const UObject* AssetToConsider : *AssetsToConsider)
		{
			SelectableAssetIdx.Append(GetAssetIndexesForSourceAsset(AssetToConsider));
		}

		if (!SelectableAssetIdx.IsEmpty())
		{
			if (SelectableAssetIdx.Num() != GetSearchIndex().Assets.Num())
			{
				SelectableAssetIdx.Sort();
			}
			else
			{
				// SelectableAssetIdx contains ALL the Database->GetSearchIndex().Assets. 
				// We reset SelectableAssetIdx since it has the same meaning, and it'll perform better
				SelectableAssetIdx.Reset();
			}
		}
	}
}

void UPoseSearchDatabase::PopulateSelectableAssetIdx(FSelectableAssetIdx& SelectableAssetIdx, const UE::PoseSearch::FStackDatabaseToAssetIndexes* AssetIndexesToConsiderPerDatabase) const
{
	using namespace UE::PoseSearch;
	
	SelectableAssetIdx.Reset();
	if (ensure(AssetIndexesToConsiderPerDatabase))
	{
		if (const FStackAssetIndexes* AssetIndexesToConsider = AssetIndexesToConsiderPerDatabase->Find(this))
		{
			const int32 NumAnimationAssets = GetNumAnimationAssets();
			ensure(AssetIndexesToConsider->Num() <= NumAnimationAssets);

			// if AssetIndexesToConsider->Num() == NumAnimationAssets we want to search the full database and and empty SelectableAssetIdx does that
			if (AssetIndexesToConsider->Num() != NumAnimationAssets)
			{
				for (int32 AssetIndexToConsider : *AssetIndexesToConsider)
				{
					if (const TArray<int32>* IndexesForSourceAsset = CachedAssetMap.Find(GetAnimationAsset(AssetIndexToConsider)))
					{
						// @todo: add a mapping like CachedAssetMap for asset indexes to SearchIndexPrivate.Assets indexes to avoid this cumbersome filtering
						for (int32 IndexForSourceAsset : *IndexesForSourceAsset)
						{
							if (SearchIndexPrivate.Assets[IndexForSourceAsset].GetSourceAssetIdx() == AssetIndexToConsider)
							{
								SelectableAssetIdx.Add(IndexForSourceAsset);
							}
						}
					}
				}
				SelectableAssetIdx.Sort();
			}
		}
	}
}

UE::PoseSearch::FSearchResult UPoseSearchDatabase::Search(UE::PoseSearch::FSearchContext& SearchContext) const
{
	using namespace UE::PoseSearch;
	FSearchResults_Single SearchResults;
	Search(SearchContext, SearchResults);
	return SearchResults.GetBestResult();
}

UE::PoseSearch::FSearchResult UPoseSearchDatabase::SearchContinuingPose(UE::PoseSearch::FSearchContext& SearchContext) const
{
	using namespace UE::PoseSearch;
	FSearchResults_Single SearchResults;
	SearchContinuingPose(SearchContext, SearchResults);
	return SearchResults.GetBestResult();
}

void UPoseSearchDatabase::ExtractCostAddends(const UE::PoseSearch::FSearchContext& SearchContext, bool bIsContinuingPoseSearch, float& ContinuingPoseCostAddend, float& ContinuingInteractionCostAddend, float& ContinuingContextInteractionCostAddend) const
{
	using namespace UE::PoseSearch;

	ContinuingPoseCostAddend = 0.f;
	ContinuingInteractionCostAddend = 0.f;
	ContinuingContextInteractionCostAddend = 0.f;

	bool bSearchForContinuingPoseCostAddend = false;
	bool bSearchForContinuingInteractionCostAddend = false;
	bool bSearchForNotifyStateContinuingContextInteractionCostBiasFound = false;

	if (bIsContinuingPoseSearch)
	{
		bSearchForContinuingPoseCostAddend = true;
		ContinuingPoseCostAddend = ContinuingPoseCostBias;
	}
	
	if (SearchContext.IsContinuingInteraction())
	{
		ContinuingInteractionCostAddend = ContinuingInteractionCostBias;
		bSearchForContinuingInteractionCostAddend = true;
	}

	if (SearchContext.IsContinuingContextInteraction())
	{
		bSearchForNotifyStateContinuingContextInteractionCostBiasFound = true;
		ContinuingContextInteractionCostAddend = ContinuingContextInteractionCostBias;
	}

	if (!bSearchForContinuingPoseCostAddend && !bSearchForContinuingInteractionCostAddend && !bSearchForNotifyStateContinuingContextInteractionCostBiasFound)
	{
		// no need to search!
		return;
	}

	const int32 ContinuingPoseIdx = SearchContext.GetContinuingPoseSearchResult().PoseIdx;
	if (ContinuingPoseIdx == INDEX_NONE)
	{
		// no need to search!
		return;
	}

	// extracting notifies from the database animation asset at time SampleTime
	const FSearchIndex& SearchIndex = GetSearchIndex();
	const FSearchIndexAsset& SearchIndexAsset = SearchIndex.GetAssetForPose(ContinuingPoseIdx);
	const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAssetBase = GetDatabaseAnimationAsset(SearchIndexAsset.GetSourceAssetIdx());
	if (!ensure(DatabaseAnimationAssetBase))
	{
		return;
	}

	const float SampleTime = GetRealAssetTime(ContinuingPoseIdx);
	FAnimNotifyContext PreAllocatedNotifyContext;
	for (int32 RoleIndex = 0; RoleIndex < DatabaseAnimationAssetBase->GetNumRoles(); ++RoleIndex)
	{
		if (const UAnimationAsset* AnimationAsset = DatabaseAnimationAssetBase->GetAnimationAssetForRole(DatabaseAnimationAssetBase->GetRole(RoleIndex)))
		{
			// sampler used only to extract the notify states. RootTransformOrigin can be set as Identity, since will not be relevant
			const FAnimationAssetSampler SequenceBaseSampler(AnimationAsset, FTransform::Identity, SearchIndexAsset.GetBlendParameters(), FAnimationAssetSampler::DefaultRootTransformSamplingRate, false, false);
			SequenceBaseSampler.ExtractAnimNotifyStates(SampleTime, PreAllocatedNotifyContext, 
				[&bSearchForContinuingPoseCostAddend, &bSearchForContinuingInteractionCostAddend, &bSearchForNotifyStateContinuingContextInteractionCostBiasFound,
				&ContinuingPoseCostAddend, &ContinuingInteractionCostAddend, &ContinuingContextInteractionCostAddend](const UAnimNotifyState* AnimNotifyState)
				{
					if (bSearchForContinuingPoseCostAddend)
					{
						if (const UAnimNotifyState_PoseSearchOverrideContinuingPoseCostBias* NotifyStateContinuingPoseCostBias = Cast<const UAnimNotifyState_PoseSearchOverrideContinuingPoseCostBias>(AnimNotifyState))
						{
							ContinuingPoseCostAddend = NotifyStateContinuingPoseCostBias->CostAddend;
							bSearchForContinuingPoseCostAddend = false;

							if (!bSearchForContinuingInteractionCostAddend && !bSearchForNotifyStateContinuingContextInteractionCostBiasFound)
							{
								// we're done searching!
								return false;
							}
						}
					}

					if (bSearchForContinuingInteractionCostAddend)
					{
						if (const UAnimNotifyState_PoseSearchOverrideContinuingInteractionPoseCostBias* NotifyStateContinuingInteractionPoseCostBias = Cast<const UAnimNotifyState_PoseSearchOverrideContinuingInteractionPoseCostBias>(AnimNotifyState))
						{
							ContinuingInteractionCostAddend = NotifyStateContinuingInteractionPoseCostBias->CostAddend;
							bSearchForContinuingInteractionCostAddend = false;

							if (!bSearchForContinuingPoseCostAddend && !bSearchForNotifyStateContinuingContextInteractionCostBiasFound)
							{
								// we're done searching!
								return false;
							}
						}
					}

					if (bSearchForNotifyStateContinuingContextInteractionCostBiasFound)
					{
						if (const UAnimNotifyState_PoseSearchOverrideContinuingContextInteractionCostBias* NotifyStateContinuingContextInteractionCostBias = Cast<const UAnimNotifyState_PoseSearchOverrideContinuingContextInteractionCostBias>(AnimNotifyState))
						{
							ContinuingContextInteractionCostAddend = NotifyStateContinuingContextInteractionCostBias->CostAddend;
							bSearchForNotifyStateContinuingContextInteractionCostBiasFound = false;

							if (!bSearchForContinuingPoseCostAddend && !bSearchForContinuingInteractionCostAddend)
							{
								// we're done searching!
								return false;
							}
						}
					}
					return true;
				});
		}

		if (!bSearchForContinuingPoseCostAddend && !bSearchForContinuingInteractionCostAddend && !bSearchForNotifyStateContinuingContextInteractionCostBiasFound)
		{
			// we're done searching!
			return;
		}
	}
}

void UPoseSearchDatabase::SearchContinuingPose(UE::PoseSearch::FSearchContext& SearchContext, UE::PoseSearch::FSearchResults& SearchResults) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PoseSearch_ContinuingPose);

	using namespace UE::PoseSearch;

	ensure(SearchContext.GetContinuingPoseSearchResult().Database.Get() == this);

#if WITH_EDITOR
	if (EAsyncBuildIndexResult::Success != FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(this, ERequestAsyncBuildFlag::ContinueRequest))
	{
		SearchContext.SetAsyncBuildIndexInProgress();
		return;
	}
#endif // WITH_EDITOR

	float ContinuingPoseCostAddend = 0.f, ContinuingInteractionCostAddend = 0.f, ContinuingContextInteractionCostAddend = 0.f;
	ExtractCostAddends(SearchContext, true, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend);

	// since any PoseCost calculated here is at least SearchIndex.MinCostAddend + ContinuingPoseCostAddend + ContinuingInteractionCostAddend + ContinuingContextInteractionCostAddend,
	// there's no point in performing the search if the worst search result is already better than that
	const FSearchIndex& SearchIndex = GetSearchIndex();
	if (SearchResults.ShouldPerformSearch(SearchIndex.MinCostAddend + ContinuingPoseCostAddend + ContinuingInteractionCostAddend + ContinuingContextInteractionCostAddend))
	{
		const int32 ContinuingPoseIdx = SearchContext.GetContinuingPoseSearchResult().PoseIdx;
		if (ensure(ContinuingPoseIdx != INDEX_NONE))
		{
			const int32 NumDimensions = Schema->SchemaCardinality;
			// FMemory_Alloca is forced 16 bytes aligned
			TArrayView<float> ReconstructedPoseValuesBuffer((float*)FMemory_Alloca(NumDimensions * sizeof(float)), NumDimensions);
			ensure(IsAligned(ReconstructedPoseValuesBuffer.GetData(), alignof(VectorRegister4Float)));
			const TConstArrayView<float> PoseValues = SearchIndex.IsValuesEmpty() ? SearchIndex.GetReconstructedPoseValues(ContinuingPoseIdx, ReconstructedPoseValuesBuffer) : SearchIndex.GetPoseValues(ContinuingPoseIdx);

			// @todo: perhaps store the DynamicWeightsSqrt into the GetOrBuildQuery to share between the continuing pose and the full search
			TArrayView<float> DynamicWeightsSqrtBuffer((float*)FMemory_Alloca(NumDimensions * sizeof(float)), NumDimensions);
			const TConstArrayView<float> DynamicWeightsSqrt = CalculateDynamicWeightsSqrt(DynamicWeightsSqrtBuffer);

			FSearchResult Result;
			Result.bIsContinuingPoseSearch = true;

			// is the data padded at 16 bytes (and 16 bytes aligned by construction)?
			if (NumDimensions % 4 == 0)
			{
				Result.PoseCost = FPoseSearchCost(CompareFeatureVectors<true>(PoseValues, SearchContext.GetOrBuildQuery(Schema), DynamicWeightsSqrt), SearchIndex.PoseMetadata[ContinuingPoseIdx].GetCostAddend(), ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend);
			}
			// data is not 16 bytes padded
			else
			{
				Result.PoseCost = FPoseSearchCost(CompareFeatureVectors<false>(PoseValues, SearchContext.GetOrBuildQuery(Schema), DynamicWeightsSqrt), SearchIndex.PoseMetadata[ContinuingPoseIdx].GetCostAddend(), ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend);
			}

			Result.SetAssetTime(SearchContext.GetContinuingPoseSearchResult().GetAssetTime());
			Result.PoseIdx = ContinuingPoseIdx;
			Result.Database = this;

#if UE_POSE_SEARCH_TRACE_ENABLED
			SearchContext.Track(this, ContinuingPoseIdx, EPoseCandidateFlags::Valid_ContinuingPose, Result.PoseCost);
#endif // UE_POSE_SEARCH_TRACE_ENABLED

			ensure(Result.DebugValidate());
			SearchResults.UpdateWith(Result);

#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
			if (GVarMotionMatchCompareAgainstBruteForce)
			{
				const FPoseSearchCost& BruteForcePoseCost = Result.PoseCost;
				SearchResults.IterateOverSearchResults([&BruteForcePoseCost](FSearchResult& SearchResult)
					{
						if (BruteForcePoseCost < SearchResult.BruteForcePoseCost)
						{
							SearchResult.BruteForcePoseCost = BruteForcePoseCost;
						}
						return false;
					});
			}
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
		}
	}
	else
	{
#if UE_POSE_SEARCH_TRACE_ENABLED
		// in case we skipped the search, we still have to track we requested to evaluate this database, so we keep track of this
		SearchContext.Track(this);
#endif // UE_POSE_SEARCH_TRACE_ENABLED
	}

	// NoTe: no need to SearchResults.FinalizeResults(), since SearchContinuingPose already calls SearchResults.UpdateWith with a Result with AssetTime already initialized
}

void UPoseSearchDatabase::SearchPCAKDTree(UE::PoseSearch::FSearchContext& SearchContext, UE::PoseSearch::FSearchResults& SearchResults) const
{
	SCOPE_CYCLE_COUNTER(STAT_PoseSearch_PCAKNN);

	using namespace UE::PoseSearch;

	float ContinuingPoseCostAddend = 0.f, ContinuingInteractionCostAddend = 0.f, ContinuingContextInteractionCostAddend = 0.f;
	ExtractCostAddends(SearchContext, false, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend);

	// since any PoseCost calculated here is at least SearchIndex.MinCostAddend + ContinuingPoseCostAddend + ContinuingInteractionCostAddend + ContinuingContextInteractionCostAddend,
	// there's no point in performing the search if the worst search result is already better than that
	const FSearchIndex& SearchIndex = GetSearchIndex();
	const int32 NumDimensions = Schema->SchemaCardinality;
	if (SearchResults.ShouldPerformSearch(SearchIndex.MinCostAddend + ContinuingPoseCostAddend + ContinuingInteractionCostAddend + ContinuingContextInteractionCostAddend))
	{
		const uint32 ClampedNumberOfPrincipalComponents = SearchIndex.GetNumberOfPrincipalComponents();
		const uint32 ClampedKDTreeQueryNumNeighbors = FMath::Clamp<uint32>(KDTreeQueryNumNeighbors, 1, SearchIndex.GetNumPoses());
		const bool bArePCAValuesPruned = SearchIndex.PCAValuesVectorToPoseIndexes.Num() > 0;

		// @todo: perhaps store the DynamicWeightsSqrt into the GetOrBuildQuery to share between the continuing pose and the full search
		TArrayView<float> DynamicWeightsSqrtBuffer((float*)FMemory_Alloca(NumDimensions * sizeof(float)), NumDimensions);
		const TConstArrayView<float> DynamicWeightsSqrt = CalculateDynamicWeightsSqrt(DynamicWeightsSqrtBuffer);

		TConstArrayView<float> QueryValues = SearchContext.GetOrBuildQuery(Schema);
		ensure(QueryValues.Num() == NumDimensions);

		FSelectableAssetIdx SelectableAssetIdx;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!SearchContext.GetInternalDeprecatedAssetsToConsider().IsEmpty())
		{
			PopulateSelectableAssetIdx(SelectableAssetIdx, SearchContext.GetInternalDeprecatedAssetsToConsider());
		}
		else if (SearchContext.GetAssetsToConsiderSet())
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			PopulateSelectableAssetIdx(SelectableAssetIdx, SearchContext.GetAssetsToConsiderSet());
		}
		else if (SearchContext.GetAssetIndexesToConsiderPerDatabase())
		{
			PopulateSelectableAssetIdx(SelectableAssetIdx, SearchContext.GetAssetIndexesToConsiderPerDatabase());
		}

		FNonSelectableIdx NonSelectableIdx;
		PopulateNonSelectableIdx(NonSelectableIdx, SearchContext
#if UE_POSE_SEARCH_TRACE_ENABLED
			, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, DynamicWeightsSqrt
#endif // UE_POSE_SEARCH_TRACE_ENABLED
		);

		bool bRunNonSelectableIdxPostKDTree = bArePCAValuesPruned;
#if WITH_EDITOR && ENABLE_ANIM_DEBUG
		bRunNonSelectableIdxPostKDTree |= GVarMotionMatchValidateKNNSearch;
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG

		// projecting QueryValues into the PCA space 
		TConstArrayView<float> PCAQueryValues = SearchContext.GetOrBuildPCAQuery(this);
		ensure(PCAQueryValues.Num() == ClampedNumberOfPrincipalComponents);

		int32 NumResults = 0;

		TArrayView<FKDTree::FKNNMaxHeapResultSet::FResult> Results((FKDTree::FKNNMaxHeapResultSet::FResult*)FMemory_Alloca(ClampedKDTreeQueryNumNeighbors * sizeof(FKDTree::FKNNMaxHeapResultSet::FResult)), ClampedKDTreeQueryNumNeighbors);

#if WITH_EDITOR && ENABLE_ANIM_DEBUG
		const double StartTimeMaxHeap = GVarMotionMatchProfileMaxHeapKNNSearch ? FPlatformTime::Seconds() : 0;
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG

		if (bRunNonSelectableIdxPostKDTree || NonSelectableIdx.IsEmpty())
		{
			FKDTree::FKNNMaxHeapResultSet ResultSet(Results);
			NumResults = SearchIndex.KDTree.FindNeighbors(ResultSet, PCAQueryValues);
		}
		else
		{
			FKDTree::FFilteredKNNMaxHeapResultSet ResultSet(Results, NonSelectableIdx);
			NumResults = SearchIndex.KDTree.FindNeighbors(ResultSet, PCAQueryValues);
		}

#if WITH_EDITOR && ENABLE_ANIM_DEBUG

		TArrayView<int32> ResultIndexes((int32*)FMemory_Alloca((ClampedKDTreeQueryNumNeighbors + 1) * sizeof(int32)), ClampedKDTreeQueryNumNeighbors + 1);
		TArrayView<float> ResultDistanceSqr((float*)FMemory_Alloca((ClampedKDTreeQueryNumNeighbors + 1) * sizeof(float)), ClampedKDTreeQueryNumNeighbors + 1);

		if (GVarMotionMatchProfileMaxHeapKNNSearch)
		{
			// debug code to log profiling comparison between FKNNResultSet/FFilteredKNNResultSet vs FKNNMaxHeapResultSet/FFilteredKNNMaxHeapResultSet
			const double EndTimeMaxHeap = FPlatformTime::Seconds();
			const double StartTimeRegular = EndTimeMaxHeap;
			
			if (bRunNonSelectableIdxPostKDTree || NonSelectableIdx.IsEmpty())
			{
				FKDTree::FKNNResultSet ResultSet(ClampedKDTreeQueryNumNeighbors, ResultIndexes, ResultDistanceSqr);
				SearchIndex.KDTree.FindNeighbors(ResultSet, PCAQueryValues);
			}
			else
			{
				FKDTree::FFilteredKNNResultSet ResultSet(ClampedKDTreeQueryNumNeighbors, ResultIndexes, ResultDistanceSqr, NonSelectableIdx);
				SearchIndex.KDTree.FindNeighbors(ResultSet, PCAQueryValues);
			}
			const double EndTimeRegular(FPlatformTime::Seconds());

			const double DeltaTimeRegular = EndTimeRegular - StartTimeRegular;
			const double DeltaTimeMaxHeap = EndTimeMaxHeap - StartTimeMaxHeap;
			const int32 Winner = DeltaTimeRegular == DeltaTimeMaxHeap ? 0 : DeltaTimeRegular < DeltaTimeMaxHeap ? -1 : 1;
			UE_LOGF(LogPoseSearch, Log, "Profiling: Regular(%f), MaxHeap(%f), Winner(%d)", DeltaTimeRegular, DeltaTimeMaxHeap, Winner);
		}

		// SortedResultsIndexes contains the sorted indexes by Distance of the Results. This is because FKNNMaxHeapResultSet doesn't return a sorted array like FKNNResultSet does
		// and we need to feed EvaluatePoseKernel with the position in the search (from the KDTree in PCA space) of the best result position, 
		// to be able to plot a graph SCostTimelineView::BestPosePosView->CurveData->Points, useful to understand how to tune the KDTreeQueryNumNeighbors property:
		// if your graph NEVER shows a value higher than 50, than it's safe to set KDTreeQueryNumNeighbors to 50! Since lowering KDTreeQueryNumNeighbors will improve performances!
		TArrayView<int32> SortedResultsIndexes((int32*)FMemory_Alloca(NumResults * sizeof(int32)), NumResults);
		for (int32 Index = 0; Index < NumResults; ++Index)
		{
			SortedResultsIndexes[Index] = Index;
		}
		SortedResultsIndexes.Sort([&Results](const int32 IndexA, const int32 IndexB) { return Results[IndexA].Distance < Results[IndexB].Distance; });

		if (GVarMotionMatchProfileMaxHeapKNNSearch)
		{
			for (int32 ResultIndex = 0; ResultIndex < NumResults; ++ResultIndex)
			{
				const int32 ExpectdIndex = ResultIndexes[ResultIndex];
				const int32 ActualIndex = Results[SortedResultsIndexes[ResultIndex]].Index;

				if (ExpectdIndex != ActualIndex)
				{
					const float ExpectdDistanceSqr = ResultDistanceSqr[ResultIndex];
					const float ActualDistanceSqr = Results[SortedResultsIndexes[ResultIndex]].Distance;
					UE_LOGF(LogPoseSearch, Warning, "Inconsistent Result at index %d (%d-%d) (%f-%f)", ResultIndex, ExpectdIndex, ActualIndex, ExpectdDistanceSqr, ActualDistanceSqr);
				}
			}
		}

		if (GVarMotionMatchValidateKNNSearch)
		{
			const int32 NumPCAValuesVectors = SearchIndex.GetNumPCAValuesVectors(ClampedNumberOfPrincipalComponents);

			TArray<TPair<int32, float>> PCAValueIndexCost;
			PCAValueIndexCost.SetNumUninitialized(NumPCAValuesVectors);

			// validating that the best n "ClampedKDTreeQueryNumNeighbors" are actually the best candidates
			for (int32 PCAValueIndex = 0; PCAValueIndex < NumPCAValuesVectors; ++PCAValueIndex)
			{
				PCAValueIndexCost[PCAValueIndex].Key = PCAValueIndex;
				PCAValueIndexCost[PCAValueIndex].Value = CompareFeatureVectors(SearchIndex.GetPCAPoseValues(PCAValueIndex), PCAQueryValues);
			}

			PCAValueIndexCost.Sort([](const TPair<int32, float>& A, const TPair<int32, float>& B)
				{
					return A.Value < B.Value;
				});

			for (int32 ResultIndex = 0; ResultIndex < NumResults; ++ResultIndex)
			{
				if (PCAValueIndexCost[ResultIndex].Key != Results[ResultIndex].Index)
				{
					if (!FMath::IsNearlyEqual(PCAValueIndexCost[ResultIndex].Value, Results[ResultIndex].Distance, UE_KINDA_SMALL_NUMBER))
					{
						UE_LOGF(LogPoseSearch, Error, "UPoseSearchDatabase::SearchPCAKDTree - KDTree search order is inconsistent with exaustive search in PCA space");
					}
					else
					{
						UE_LOGF(LogPoseSearch, Log, "UPoseSearchDatabase::SearchPCAKDTree - found two points at the same distance from the query in different order between KDTree and exaustive search");
					}
				}
				else if (!FMath::IsNearlyEqual(PCAValueIndexCost[ResultIndex].Value, Results[ResultIndex].Distance, UE_KINDA_SMALL_NUMBER))
				{
					UE_LOGF(LogPoseSearch, Error, "UPoseSearchDatabase::SearchPCAKDTree - KDTree search cost is inconsistent with exaustive search in PCA space");
				}
			}
		}
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG

		// NonSelectableIdx are already filtered out inside the kdtree search.
		// Also kdtrees don't contain block transition poses by construction, so FSearchFilters input bAddBlockTransitionFilter can be set to false
		const FSearchFilters SearchFilters(Schema, bRunNonSelectableIdxPostKDTree ? NonSelectableIdx : TConstArrayView<int32>(), SelectableAssetIdx, false, QueryValues, SearchContext);
		
		// are the PCAValues pruned out of duplicates (multiple poses are associated with the same PCAValuesVectorIdx)
		if (bArePCAValuesPruned)
		{
			const int32 MaxNumEvaluatePoseKernelCalls = KDTreeQueryNumNeighborsWithDuplicates > 0 ? KDTreeQueryNumNeighborsWithDuplicates : INT32_MAX;

			// do we need to reconstruct pose values?
			if (SearchIndex.IsValuesEmpty())
			{
				// FMemory_Alloca is forced 16 bytes aligned
				TArrayView<float> ReconstructedPoseValuesBuffer((float*)FMemory_Alloca(NumDimensions * sizeof(float)), NumDimensions);
				ensure(IsAligned(ReconstructedPoseValuesBuffer.GetData(), alignof(VectorRegister4Float)));
				if (NumDimensions % 4 == 0)
				{
					int32 NumEvaluatePoseKernelCalls = 0;
					for (int32 ResultIndex = 0; ResultIndex < NumResults; ++ResultIndex)
					{
						const TConstArrayView<int32> PoseIndexes = SearchIndex.PCAValuesVectorToPoseIndexes[Results[ResultIndex].Index];
						for (int32 Index = 0; Index < PoseIndexes.Num() && NumEvaluatePoseKernelCalls < MaxNumEvaluatePoseKernelCalls; ++Index, ++NumEvaluatePoseKernelCalls)
						{
							EvaluatePoseKernel<true, true>(SearchResults, SearchIndex, QueryValues, ReconstructedPoseValuesBuffer, PoseIndexes[Index], INDEX_NONE, SearchFilters, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, this, DynamicWeightsSqrt
#if UE_POSE_SEARCH_TRACE_ENABLED
								, true
#endif // UE_POSE_SEARCH_TRACE_ENABLED
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
								, SortedResultsIndexes[ResultIndex]
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
							);
						}
					}
				}
				else
				{
					int32 NumEvaluatePoseKernelCalls = 0;
					for (int32 ResultIndex = 0; ResultIndex < NumResults; ++ResultIndex)
					{
						const TConstArrayView<int32> PoseIndexes = SearchIndex.PCAValuesVectorToPoseIndexes[Results[ResultIndex].Index];
						for (int32 Index = 0; Index < PoseIndexes.Num() && NumEvaluatePoseKernelCalls < MaxNumEvaluatePoseKernelCalls; ++Index, ++NumEvaluatePoseKernelCalls)
						{
							EvaluatePoseKernel<true, false>(SearchResults, SearchIndex, QueryValues, ReconstructedPoseValuesBuffer, PoseIndexes[Index], INDEX_NONE, SearchFilters, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, this, DynamicWeightsSqrt
#if UE_POSE_SEARCH_TRACE_ENABLED
								, true
#endif // UE_POSE_SEARCH_TRACE_ENABLED
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
								, SortedResultsIndexes[ResultIndex]
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
							);
						}
					}
				}
			}
			else
			{
				// no need to reconstruct pose values
				if (NumDimensions % 4 == 0)
				{
					int32 NumEvaluatePoseKernelCalls = 0;
					for (int32 ResultIndex = 0; ResultIndex < NumResults; ++ResultIndex)
					{
						const TConstArrayView<int32> PoseIndexes = SearchIndex.PCAValuesVectorToPoseIndexes[Results[ResultIndex].Index];
						for (int32 Index = 0; Index < PoseIndexes.Num() && NumEvaluatePoseKernelCalls < MaxNumEvaluatePoseKernelCalls; ++Index, ++NumEvaluatePoseKernelCalls)
						{
							EvaluatePoseKernel<false, true>(SearchResults, SearchIndex, QueryValues, TArrayView<float>(), PoseIndexes[Index], INDEX_NONE, SearchFilters, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, this, DynamicWeightsSqrt
#if UE_POSE_SEARCH_TRACE_ENABLED
								, true
#endif // UE_POSE_SEARCH_TRACE_ENABLED
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
								, SortedResultsIndexes[ResultIndex]
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
							);
						}
					}
				}
				else
				{
					int32 NumEvaluatePoseKernelCalls = 0;
					for (int32 ResultIndex = 0; ResultIndex < NumResults; ++ResultIndex)
					{
						const TConstArrayView<int32> PoseIndexes = SearchIndex.PCAValuesVectorToPoseIndexes[Results[ResultIndex].Index];
						for (int32 Index = 0; Index < PoseIndexes.Num() && NumEvaluatePoseKernelCalls < MaxNumEvaluatePoseKernelCalls; ++Index, ++NumEvaluatePoseKernelCalls)
						{
							EvaluatePoseKernel<false, false>(SearchResults, SearchIndex, QueryValues, TArrayView<float>(), PoseIndexes[Index], INDEX_NONE, SearchFilters, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, this, DynamicWeightsSqrt
#if UE_POSE_SEARCH_TRACE_ENABLED
								, true
#endif // UE_POSE_SEARCH_TRACE_ENABLED
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
								, SortedResultsIndexes[ResultIndex]
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
							);
						}
					}
				}
			}
		}
		// do we need to reconstruct pose values?
		else if (SearchIndex.IsValuesEmpty())
		{
			// FMemory_Alloca is forced 16 bytes aligned
			TArrayView<float> ReconstructedPoseValuesBuffer((float*)FMemory_Alloca(NumDimensions * sizeof(float)), NumDimensions);
			ensure(IsAligned(ReconstructedPoseValuesBuffer.GetData(), alignof(VectorRegister4Float)));

			// is the data padded at 16 bytes (and 16 bytes aligned by construction)?
			if (NumDimensions % 4 == 0)
			{
				for (int32 ResultIndex = 0; ResultIndex < NumResults; ++ResultIndex)
				{
					EvaluatePoseKernel<true, true>(SearchResults, SearchIndex, QueryValues, ReconstructedPoseValuesBuffer, Results[ResultIndex].Index, INDEX_NONE, SearchFilters, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, this, DynamicWeightsSqrt
	#if UE_POSE_SEARCH_TRACE_ENABLED
						, true
	#endif // UE_POSE_SEARCH_TRACE_ENABLED
	#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
						, SortedResultsIndexes[ResultIndex]
	#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
					);
				}
			}
			else
			{
				for (int32 ResultIndex = 0; ResultIndex < NumResults; ++ResultIndex)
				{
					EvaluatePoseKernel<true, false>(SearchResults, SearchIndex, QueryValues, ReconstructedPoseValuesBuffer, Results[ResultIndex].Index, INDEX_NONE, SearchFilters, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, this, DynamicWeightsSqrt
#if UE_POSE_SEARCH_TRACE_ENABLED
						, true
#endif // UE_POSE_SEARCH_TRACE_ENABLED
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
						, SortedResultsIndexes[ResultIndex]
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
					);
				}
			}
		}
		else
		{
			// is the data padded at 16 bytes (and 16 bytes aligned by construction)?
			if (NumDimensions % 4 == 0)
			{
				for (int32 ResultIndex = 0; ResultIndex < NumResults; ++ResultIndex)
				{
					EvaluatePoseKernel<false, true>(SearchResults, SearchIndex, QueryValues, TArrayView<float>(), Results[ResultIndex].Index, INDEX_NONE, SearchFilters, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, this, DynamicWeightsSqrt
#if UE_POSE_SEARCH_TRACE_ENABLED
						, true
#endif // UE_POSE_SEARCH_TRACE_ENABLED
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
						, SortedResultsIndexes[ResultIndex]
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
					);
				}
			}
			// no reconstruction, but data is not 16 bytes padded
			else
			{
				for (int32 ResultIndex = 0; ResultIndex < NumResults; ++ResultIndex)
				{
					EvaluatePoseKernel<false, false>(SearchResults, SearchIndex, QueryValues, TArrayView<float>(), Results[ResultIndex].Index, INDEX_NONE, SearchFilters, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, this, DynamicWeightsSqrt
#if UE_POSE_SEARCH_TRACE_ENABLED
						, true
#endif // UE_POSE_SEARCH_TRACE_ENABLED
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
						, SortedResultsIndexes[ResultIndex]
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
					);
				}
			}
		}
	}
	else
	{
#if UE_POSE_SEARCH_TRACE_ENABLED
		// @todo: perhaps store the DynamicWeightsSqrt into the GetOrBuildQuery to share between the continuing pose and the full search
		TArrayView<float> DynamicWeightsSqrtBuffer((float*)FMemory_Alloca(NumDimensions * sizeof(float)), NumDimensions);
		const TConstArrayView<float> DynamicWeightsSqrt = CalculateDynamicWeightsSqrt(DynamicWeightsSqrtBuffer);

		// calling just for reporting non selectable poses
		FNonSelectableIdx NonSelectableIdx;
		PopulateNonSelectableIdx(NonSelectableIdx, SearchContext, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, DynamicWeightsSqrt);
#endif // UE_POSE_SEARCH_TRACE_ENABLED
	}

	SearchResults.FinalizeResults();

#if UE_POSE_SEARCH_TRACE_ENABLED
	// in case we skipped the search, or we didn't find any candidates we still have to track we requested to evaluate this database, so we keep track of this
	SearchContext.Track(this);
#endif // UE_POSE_SEARCH_TRACE_ENABLED
}

void UPoseSearchDatabase::SearchVPTree(UE::PoseSearch::FSearchContext& SearchContext, UE::PoseSearch::FSearchResults& SearchResults) const
{
	SCOPE_CYCLE_COUNTER(STAT_PoseSearch_VPTree);

	using namespace UE::PoseSearch;

	float ContinuingPoseCostAddend = 0.f, ContinuingInteractionCostAddend = 0.f, ContinuingContextInteractionCostAddend = 0.f;
	ExtractCostAddends(SearchContext, false, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend);

	// since any PoseCost calculated here is at least SearchIndex.MinCostAddend + ContinuingPoseCostAddend + ContinuingInteractionCostAddend + ContinuingContextInteractionCostAddend,
	// there's no point in performing the search if the worst search result is already better than that
	const FSearchIndex& SearchIndex = GetSearchIndex();
	if (SearchResults.ShouldPerformSearch(SearchIndex.MinCostAddend + ContinuingPoseCostAddend + ContinuingInteractionCostAddend + ContinuingContextInteractionCostAddend))
	{
		// @todo: perhaps store the DynamicWeightsSqrt into the GetOrBuildQuery to share between the continuing pose and the full search
		const int32 NumDimensions = Schema->SchemaCardinality;
		TArrayView<float> DynamicWeightsSqrtBuffer((float*)FMemory_Alloca(NumDimensions * sizeof(float)), NumDimensions);
		const TConstArrayView<float> DynamicWeightsSqrt = CalculateDynamicWeightsSqrt(DynamicWeightsSqrtBuffer);

		TConstArrayView<float> QueryValues = SearchContext.GetOrBuildQuery(Schema);
		ensure(QueryValues.Num() == NumDimensions);

		FSelectableAssetIdx SelectableAssetIdx;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!SearchContext.GetInternalDeprecatedAssetsToConsider().IsEmpty())
		{
			PopulateSelectableAssetIdx(SelectableAssetIdx, SearchContext.GetInternalDeprecatedAssetsToConsider());
		}
		else if (SearchContext.GetAssetsToConsiderSet())
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			PopulateSelectableAssetIdx(SelectableAssetIdx, SearchContext.GetAssetsToConsiderSet());
		}
		else if (SearchContext.GetAssetIndexesToConsiderPerDatabase())
		{
			PopulateSelectableAssetIdx(SelectableAssetIdx, SearchContext.GetAssetIndexesToConsiderPerDatabase());
		}

		// @todo: implement filtering within the VPTree as KDTree does
		FNonSelectableIdx NonSelectableIdx;
		PopulateNonSelectableIdx(NonSelectableIdx, SearchContext
#if UE_POSE_SEARCH_TRACE_ENABLED
			, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, DynamicWeightsSqrt
#endif // UE_POSE_SEARCH_TRACE_ENABLED
		);
		
		const FSearchFilters SearchFilters(Schema, NonSelectableIdx, SelectableAssetIdx, SearchIndex.bAnyBlockTransition, QueryValues, SearchContext);

		// @todo: implement a FVPTreeDataSource for aligned and padded features vector like CompareAlignedPoses does 
		FVPTreeDataSource DataSource(SearchIndex);
		FVPTreeResultSet ResultSet(KDTreeQueryNumNeighbors);
		SearchIndex.VPTree.FindNeighbors(QueryValues, ResultSet, DataSource);
		
		int32 NumEvaluatePoseKernelCalls = 0;
		const TConstArrayView<FIndexDistance> UnsortedResults = ResultSet.GetUnsortedResults();

		const bool bAreValuesPruned = SearchIndex.ValuesVectorToPoseIndexes.Num() > 0;
		if (bAreValuesPruned)
		{
			const int32 MaxNumEvaluatePoseKernelCalls = KDTreeQueryNumNeighborsWithDuplicates > 0 ? KDTreeQueryNumNeighborsWithDuplicates : INT32_MAX;
			for (int32 ResultIndex = 0; ResultIndex < UnsortedResults.Num(); ++ResultIndex)
			{
				const FIndexDistance& IndexDistance = UnsortedResults[ResultIndex];

				// @todo: IndexDistance.Distance is the Sqrt(DissimilarityCost), so there's no need to calculate it again in SearchIndex.ComparePoses
				const TConstArrayView<int32> PoseIndexes = SearchIndex.ValuesVectorToPoseIndexes[IndexDistance.Index];
				for (int32 Index = 0; Index < PoseIndexes.Num() && NumEvaluatePoseKernelCalls < MaxNumEvaluatePoseKernelCalls; ++Index, ++NumEvaluatePoseKernelCalls)
				{
					EvaluatePoseKernel<false, false>(SearchResults, SearchIndex, QueryValues, TArrayView<float>(), PoseIndexes[Index], INDEX_NONE, SearchFilters, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, this, DynamicWeightsSqrt
#if UE_POSE_SEARCH_TRACE_ENABLED
						, true
#endif // UE_POSE_SEARCH_TRACE_ENABLED
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
						, ResultIndex
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
					);
				}
			}
		}
		else
		{
			for (int32 ResultIndex = 0; ResultIndex < UnsortedResults.Num(); ++ResultIndex)
			{
				const FIndexDistance& IndexDistance = UnsortedResults[ResultIndex];

				// @todo: IndexDistance.Distance is the Sqrt(DissimilarityCost), so there's no need to calculate it again in SearchIndex.ComparePoses
				EvaluatePoseKernel<false, false>(SearchResults, SearchIndex, QueryValues, TArrayView<float>(), IndexDistance.Index, INDEX_NONE, SearchFilters, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, this, DynamicWeightsSqrt
#if UE_POSE_SEARCH_TRACE_ENABLED
					, true
#endif // UE_POSE_SEARCH_TRACE_ENABLED
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
					, ResultIndex
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
				);
			}
		}
	}
	else
	{
#if UE_POSE_SEARCH_TRACE_ENABLED
		// @todo: perhaps store the DynamicWeightsSqrt into the GetOrBuildQuery to share between the continuing pose and the full search
		const int32 NumDimensions = Schema->SchemaCardinality;
		TArrayView<float> DynamicWeightsSqrtBuffer((float*)FMemory_Alloca(NumDimensions * sizeof(float)), NumDimensions);
		const TConstArrayView<float> DynamicWeightsSqrt = CalculateDynamicWeightsSqrt(DynamicWeightsSqrtBuffer);

		// calling just for reporting non selectable poses
		FNonSelectableIdx NonSelectableIdx;
		PopulateNonSelectableIdx(NonSelectableIdx, SearchContext, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, DynamicWeightsSqrt);
#endif // UE_POSE_SEARCH_TRACE_ENABLED
	}

	SearchResults.FinalizeResults();

#if UE_POSE_SEARCH_TRACE_ENABLED
	// in case we skipped the search, or we didn't find any candidates we still have to track we requested to evaluate this database, so we keep track of this
	SearchContext.Track(this);
#endif // UE_POSE_SEARCH_TRACE_ENABLED
}

void UPoseSearchDatabase::SearchBruteForce(UE::PoseSearch::FSearchContext& SearchContext, UE::PoseSearch::FSearchResults& SearchResults) const
{
	SCOPE_CYCLE_COUNTER(STAT_PoseSearch_BruteForce);
	
	using namespace UE::PoseSearch;
	
	float ContinuingPoseCostAddend = 0.f, ContinuingInteractionCostAddend = 0.f, ContinuingContextInteractionCostAddend = 0.f;
	ExtractCostAddends(SearchContext, false, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend);

	const bool bTrackResults = PoseSearchMode == EPoseSearchMode::BruteForce;

	// since any PoseCost calculated here is at least SearchIndex.MinCostAddend + ContinuingPoseCostAddend + ContinuingInteractionCostAddend + ContinuingContextInteractionCostAddend,
	// there's no point in performing the search if the worst search result is already better than that
	const FSearchIndex& SearchIndex = GetSearchIndex();
	if (SearchResults.ShouldPerformSearch(SearchIndex.MinCostAddend + ContinuingPoseCostAddend + ContinuingInteractionCostAddend + ContinuingContextInteractionCostAddend))
	{
		// @todo: perhaps store the DynamicWeightsSqrt into the GetOrBuildQuery to share between the continuing pose and the full search
		const int32 NumDimensions = Schema->SchemaCardinality;
		TArrayView<float> DynamicWeightsSqrtBuffer((float*)FMemory_Alloca(NumDimensions * sizeof(float)), NumDimensions);
		const TConstArrayView<float> DynamicWeightsSqrt = CalculateDynamicWeightsSqrt(DynamicWeightsSqrtBuffer);

		TConstArrayView<float> QueryValues = SearchContext.GetOrBuildQuery(Schema);

		FSelectableAssetIdx SelectableAssetIdx;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!SearchContext.GetInternalDeprecatedAssetsToConsider().IsEmpty())
		{
			PopulateSelectableAssetIdx(SelectableAssetIdx, SearchContext.GetInternalDeprecatedAssetsToConsider());
		}
		else if (SearchContext.GetAssetsToConsiderSet())
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			PopulateSelectableAssetIdx(SelectableAssetIdx, SearchContext.GetAssetsToConsiderSet());
		}
		else if (SearchContext.GetAssetIndexesToConsiderPerDatabase())
		{
			PopulateSelectableAssetIdx(SelectableAssetIdx, SearchContext.GetAssetIndexesToConsiderPerDatabase());
		}

		FNonSelectableIdx NonSelectableIdx;
		PopulateNonSelectableIdx(NonSelectableIdx, SearchContext
#if UE_POSE_SEARCH_TRACE_ENABLED
			, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, DynamicWeightsSqrt
#endif // UE_POSE_SEARCH_TRACE_ENABLED
		);

		const FSearchFilters SearchFilters(Schema, NonSelectableIdx, FSelectableAssetIdx(), SearchIndex.bAnyBlockTransition, QueryValues, SearchContext);

		if (SelectableAssetIdx.IsEmpty())
		{
			// do we need to reconstruct pose values?
			if (SearchIndex.IsValuesEmpty())
			{
				// FMemory_Alloca is forced 16 bytes aligned
				TArrayView<float> ReconstructedPoseValuesBuffer((float*)FMemory_Alloca(NumDimensions * sizeof(float)), NumDimensions);
				ensure(IsAligned(ReconstructedPoseValuesBuffer.GetData(), alignof(VectorRegister4Float)));
				for (int32 PoseIdx = 0; PoseIdx < SearchIndex.GetNumPoses(); ++PoseIdx)
				{
					EvaluatePoseKernel<true, false>(SearchResults, SearchIndex, QueryValues, ReconstructedPoseValuesBuffer, PoseIdx, INDEX_NONE, SearchFilters, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, this, DynamicWeightsSqrt
#if UE_POSE_SEARCH_TRACE_ENABLED
						, bTrackResults
#endif // UE_POSE_SEARCH_TRACE_ENABLED
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
						, PoseIdx
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
					);
				}
			}
			// is the data padded at 16 bytes (and 16 bytes aligned by construction)?
			else if (NumDimensions % 4 == 0)
			{
				for (int32 PoseIdx = 0; PoseIdx < SearchIndex.GetNumPoses(); ++PoseIdx)
				{
					EvaluatePoseKernel<false, true>(SearchResults, SearchIndex, QueryValues, TArrayView<float>(), PoseIdx, INDEX_NONE, SearchFilters, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, this, DynamicWeightsSqrt
#if UE_POSE_SEARCH_TRACE_ENABLED
						, bTrackResults
#endif // UE_POSE_SEARCH_TRACE_ENABLED
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
						, PoseIdx
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
					);
				}
			}
			// no reconstruction, but data is not 16 bytes padded
			else
			{
				for (int32 PoseIdx = 0; PoseIdx < SearchIndex.GetNumPoses(); ++PoseIdx)
				{
					EvaluatePoseKernel<false, false>(SearchResults, SearchIndex, QueryValues, TArrayView<float>(), PoseIdx, INDEX_NONE, SearchFilters, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, this, DynamicWeightsSqrt
#if UE_POSE_SEARCH_TRACE_ENABLED
						, bTrackResults
#endif // UE_POSE_SEARCH_TRACE_ENABLED
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
						, PoseIdx
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
					);
				}
			}
		}
		else
		{
			int32 ResultIndex = -1;

			// do we need to reconstruct pose values?
			if (SearchIndex.IsValuesEmpty())
			{
				// FMemory_Alloca is forced 16 bytes aligned
				TArrayView<float> ReconstructedPoseValuesBuffer((float*)FMemory_Alloca(NumDimensions * sizeof(float)), NumDimensions);
				ensure(IsAligned(ReconstructedPoseValuesBuffer.GetData(), alignof(VectorRegister4Float)));

				for (int32 AssetIdx : SelectableAssetIdx)
				{
					const FSearchIndexAsset& SearchIndexAsset = SearchIndex.Assets[AssetIdx];
					const int32 FirstPoseIdx = SearchIndexAsset.GetFirstPoseIdx();
					const int32 LastPoseIdx = FirstPoseIdx + SearchIndexAsset.GetNumPoses();
					for (int32 PoseIdx = FirstPoseIdx; PoseIdx < LastPoseIdx; ++PoseIdx)
					{
						EvaluatePoseKernel<true, false>(SearchResults, SearchIndex, QueryValues, ReconstructedPoseValuesBuffer, PoseIdx, INDEX_NONE, SearchFilters, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, this, DynamicWeightsSqrt
#if UE_POSE_SEARCH_TRACE_ENABLED
							, bTrackResults
#endif // UE_POSE_SEARCH_TRACE_ENABLED
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
							, ++ResultIndex
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
						);
					}
				}
			}
			// is the data padded at 16 bytes (and 16 bytes aligned by construction)?
			else if (NumDimensions % 4 == 0)
			{
				for (int32 AssetIdx : SelectableAssetIdx)
				{
					const FSearchIndexAsset& SearchIndexAsset = SearchIndex.Assets[AssetIdx];
					const int32 FirstPoseIdx = SearchIndexAsset.GetFirstPoseIdx();
					const int32 LastPoseIdx = FirstPoseIdx + SearchIndexAsset.GetNumPoses();
					for (int32 PoseIdx = FirstPoseIdx; PoseIdx < LastPoseIdx; ++PoseIdx)
					{
						EvaluatePoseKernel<false, true>(SearchResults, SearchIndex, QueryValues, TArrayView<float>(), PoseIdx, INDEX_NONE, SearchFilters, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, this, DynamicWeightsSqrt
#if UE_POSE_SEARCH_TRACE_ENABLED
							, bTrackResults
#endif // UE_POSE_SEARCH_TRACE_ENABLED
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
							, ++ResultIndex
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
						);
					}
				}
			}
			// no reconstruction, but data is not 16 bytes padded
			else
			{
				for (int32 AssetIdx : SelectableAssetIdx)
				{
					const FSearchIndexAsset& SearchIndexAsset = SearchIndex.Assets[AssetIdx];
					const int32 FirstPoseIdx = SearchIndexAsset.GetFirstPoseIdx();
					const int32 LastPoseIdx = FirstPoseIdx + SearchIndexAsset.GetNumPoses();
					for (int32 PoseIdx = FirstPoseIdx; PoseIdx < LastPoseIdx; ++PoseIdx)
					{
						EvaluatePoseKernel<false, false>(SearchResults, SearchIndex, QueryValues, TArrayView<float>(), PoseIdx, INDEX_NONE, SearchFilters, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, this, DynamicWeightsSqrt
#if UE_POSE_SEARCH_TRACE_ENABLED
							, bTrackResults
#endif // UE_POSE_SEARCH_TRACE_ENABLED
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
							, ++ResultIndex
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
						);
					}
				}
			}
		}
	}
	else
	{
#if UE_POSE_SEARCH_TRACE_ENABLED
		// @todo: perhaps store the DynamicWeightsSqrt into the GetOrBuildQuery to share between the continuing pose and the full search
		const int32 NumDimensions = Schema->SchemaCardinality;
		TArrayView<float> DynamicWeightsSqrtBuffer((float*)FMemory_Alloca(NumDimensions * sizeof(float)), NumDimensions);
		const TConstArrayView<float> DynamicWeightsSqrt = CalculateDynamicWeightsSqrt(DynamicWeightsSqrtBuffer);

		// calling just for reporting non selectable poses
		FNonSelectableIdx NonSelectableIdx;
		PopulateNonSelectableIdx(NonSelectableIdx, SearchContext, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, DynamicWeightsSqrt);
#endif // UE_POSE_SEARCH_TRACE_ENABLED
	}

	SearchResults.FinalizeResults();

#if UE_POSE_SEARCH_TRACE_ENABLED
	if (bTrackResults)
	{
		// in case we skipped the search, or we didn't find any candidates we still have to track we requested to evaluate this database, so we keep track of this
		SearchContext.Track(this);
	}
#endif // UE_POSE_SEARCH_TRACE_ENABLED
}

void UPoseSearchDatabase::SearchEvent(UE::PoseSearch::FSearchContext& SearchContext, UE::PoseSearch::FSearchResults& SearchResults) const
{
	SCOPE_CYCLE_COUNTER(STAT_PoseSearch_Event);

	using namespace UE::PoseSearch;

	const FSearchIndex& SearchIndex = GetSearchIndex();
	if (SearchIndex.IsValuesEmpty())
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchDatabase::SearchEvent unimplemented for reconstructed poses");
	}
	else
	{
		const FPoseSearchEvent& EventToSearch = SearchContext.GetEventToSearch();
		const TConstArrayView<int32> PosesWithEvent = SearchIndex.EventData.GetPosesWithEvent(EventToSearch.EventTag);
		if (!PosesWithEvent.IsEmpty())
		{
			float ContinuingPoseCostAddend = 0.f, ContinuingInteractionCostAddend = 0.f, ContinuingContextInteractionCostAddend = 0.f;
			ExtractCostAddends(SearchContext, false, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend);

			TConstArrayView<float> QueryValues = SearchContext.GetOrBuildQuery(Schema);

			const int32 NumDimensions = Schema->SchemaCardinality;
			TArrayView<float> DynamicWeightsSqrtBuffer((float*)FMemory_Alloca(NumDimensions * sizeof(float)), NumDimensions);
			const TConstArrayView<float> DynamicWeightsSqrt = CalculateDynamicWeightsSqrt(DynamicWeightsSqrtBuffer);

			FSelectableAssetIdx SelectableAssetIdx;
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (!SearchContext.GetInternalDeprecatedAssetsToConsider().IsEmpty())
			{
				PopulateSelectableAssetIdx(SelectableAssetIdx, SearchContext.GetInternalDeprecatedAssetsToConsider());
			}
			else if (SearchContext.GetAssetsToConsiderSet())
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				PopulateSelectableAssetIdx(SelectableAssetIdx, SearchContext.GetAssetsToConsiderSet());
			}
			else if (SearchContext.GetAssetIndexesToConsiderPerDatabase())
			{
				PopulateSelectableAssetIdx(SelectableAssetIdx, SearchContext.GetAssetIndexesToConsiderPerDatabase());
			}

			FNonSelectableIdx NonSelectableIdx;
			if (EventToSearch.bEnablePoseFilters)
			{
				PopulateNonSelectableIdx(NonSelectableIdx, SearchContext
#if UE_POSE_SEARCH_TRACE_ENABLED
					, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, DynamicWeightsSqrt
#endif // UE_POSE_SEARCH_TRACE_ENABLED
				);
			}

			FSearchFilters SearchFilters(Schema, NonSelectableIdx, SelectableAssetIdx, SearchIndex.bAnyBlockTransition, QueryValues, SearchContext);

			if (FMath::IsNearlyZero(EventToSearch.TimeToEvent))
			{
				for (int32 EventPoseIdx : PosesWithEvent)
				{
					EvaluatePoseKernel<false, false>(SearchResults, SearchIndex, QueryValues, TArrayView<float>(), EventPoseIdx, EventPoseIdx, SearchFilters, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, this, DynamicWeightsSqrt
#if UE_POSE_SEARCH_TRACE_ENABLED
						, true
#endif // UE_POSE_SEARCH_TRACE_ENABLED
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
						, EventPoseIdx
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
					);
				}
			}
			else
			{
				const float TimeToEventStart = EventToSearch.TimeToEvent * EventToSearch.PlayRateRangeOverride.Min;
				const float TimeToEventEnd = EventToSearch.TimeToEvent * EventToSearch.PlayRateRangeOverride.Max;
				for (int32 EventPoseIdx : PosesWithEvent)
				{
					// Calculating the pose which is TimeToEvent seconds before the event
					const FSearchIndexAsset& SearchIndexAsset = SearchIndex.GetAssetForPose(EventPoseIdx);
					const float EventTime = SearchIndexAsset.GetTimeFromPoseIndex(EventPoseIdx, Schema->SampleRate);
					SearchIndexAsset.IteratePoseIndexesOverTime(EventTime - TimeToEventEnd, EventTime - TimeToEventStart, Schema->SampleRate,
						[this, &SearchResults, &SearchIndex, &QueryValues, EventPoseIdx, &SearchFilters, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, &SearchContext, &DynamicWeightsSqrt](int32 PoseIdx)
						{
							EvaluatePoseKernel<false, false>(SearchResults, SearchIndex, QueryValues, TArrayView<float>(), PoseIdx, EventPoseIdx, SearchFilters, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend, this, DynamicWeightsSqrt
#if UE_POSE_SEARCH_TRACE_ENABLED
								, true
#endif // UE_POSE_SEARCH_TRACE_ENABLED
#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
								, PoseIdx
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
							);
						});
				}
			}
		}
	}

	SearchResults.FinalizeResults();

#if UE_POSE_SEARCH_TRACE_ENABLED
	// in case we skipped the search, or we didn't find any candidates we still have to track we requested to evaluate this database, so we keep track of this
	SearchContext.Track(this);
#endif // UE_POSE_SEARCH_TRACE_ENABLED
}

#undef LOCTEXT_NAMESPACE