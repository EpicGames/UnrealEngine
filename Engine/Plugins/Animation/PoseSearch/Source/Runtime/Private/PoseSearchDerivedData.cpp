// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "PoseSearch/PoseSearchDerivedData.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/CoreDelegates.h"
#if ENABLE_ANIM_DEBUG
#include "Misc/FileHelper.h"
#endif //ENABLE_ANIM_DEBUG
#include "PoseSearch/PoseSearchAnimNotifies.h"
#include "PoseSearch/PoseSearchAssetIndexer.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "PoseSearch/PoseSearchDerivedDataKey.h"
#include "PoseSearch/PoseSearchFeatureChannel.h"
#include "PoseSearch/PoseSearchNormalizationSet.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearchEigenHelper.h"
#include "ProfilingDebugging/CookStats.h"
#include "ScopedTransaction.h"
#include "Serialization/BulkDataRegistry.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/NoExportTypes.h"
#include "UObject/PackageReload.h"

#define LOCTEXT_NAMESPACE "PoseSearchDerivedData"

namespace UE::PoseSearch
{
#if ENABLE_ANIM_DEBUG

static bool GVarMotionMatchTestInvalidateCache = false;
static FAutoConsoleVariableRef CVarMotionMatchTestInvalidateCache(TEXT("a.MotionMatch.Test.InvalidateCache"), GVarMotionMatchTestInvalidateCache, TEXT("Invalidate DDC cache on a frame basis (to stress test DDC cancellation while tasks are flying if !WaitForTaskCompletion)"));

static bool GVarMotionMatchTestWaitForTaskCompletion = false;
static FAutoConsoleVariableRef CVarMotionMatchTestWaitForTaskCompletion(TEXT("a.MotionMatch.Test.WaitForTaskCompletion"), GVarMotionMatchTestWaitForTaskCompletion, TEXT("Wait synchronously for DDC tasks to complete"));

static bool GVarMotionMatchTestForceIndexing = false;
static FAutoConsoleVariableRef CVarMotionMatchTestForceIndexing(TEXT("a.MotionMatch.Test.ForceIndexing"), GVarMotionMatchTestForceIndexing, TEXT("we'll force the database re-indexing and compare the result SearchIndex with the one retrieved via DDC"));

static bool GVarMotionMatchTestDeterminismKDTreeConstruct = false;
static FAutoConsoleVariableRef CVarMotionMatchTestDeterminismKDTreeConstruct(TEXT("a.MotionMatch.Test.Determinism.KDTreeConstruct"), GVarMotionMatchTestDeterminismKDTreeConstruct, TEXT("test KDTree Construct determinism"));

static bool GVarMotionMatchTestKDTreeConstruct = false;
static FAutoConsoleVariableRef CVarMotionMatchTestKDTreeConstruct(TEXT("a.MotionMatch.Test.KDTreeConstruct"), GVarMotionMatchTestKDTreeConstruct, TEXT("validating the kdtree construction"));

static bool GVarMotionMatchTestDeterminismVPTreeConstruct = false;
static FAutoConsoleVariableRef CVarMotionMatchTestDeterminismVPTreeConstruct(TEXT("a.MotionMatch.Test.Determinism.VPTreeConstruct"), GVarMotionMatchTestDeterminismVPTreeConstruct, TEXT("test VPTree Construct determinism"));

static bool GVarMotionMatchTestVPTreeConstruct = false;
static FAutoConsoleVariableRef CVarMotionMatchTestVPTreeConstruct(TEXT("a.MotionMatch.Test.VPTreeConstruct"), GVarMotionMatchTestVPTreeConstruct, TEXT("validating the vptree construction"));

static bool GVarMotionMatchTestDeterminismIndexDatabase = false;
static FAutoConsoleVariableRef CVarMotionMatchTestDeterminismIndexDatabase(TEXT("a.MotionMatch.Test.Determinism.IndexDatabase"), GVarMotionMatchTestDeterminismIndexDatabase, TEXT("test IndexDatabase determinism"));

static bool GVarMotionMatchTestDeterminismPruneDuplicateValues = false;
static FAutoConsoleVariableRef CVarMotionMatchTestDeterminismPruneDuplicateValues(TEXT("a.MotionMatch.Test.Determinism.PruneDuplicateValues"), GVarMotionMatchTestDeterminismPruneDuplicateValues, TEXT("test PruneDuplicateValues determinism"));

static bool GVarMotionMatchTestDeterminismPruneDuplicatePCAValues = false;
static FAutoConsoleVariableRef CVarMotionMatchTestDeterminismPruneDuplicatePCAValues(TEXT("a.MotionMatch.Test.Determinism.PruneDuplicatePCAValues"), GVarMotionMatchTestDeterminismPruneDuplicatePCAValues, TEXT("test PruneDuplicatePCAValues determinism"));

static bool GVarMotionMatchTestDDCConsistency = false;
static FAutoConsoleVariableRef CVarMotionMatchTestDDCConsistency(TEXT("a.MotionMatch.Test.DDCConsistency"), GVarMotionMatchTestDDCConsistency, TEXT("validating the data we gave to DDC is stored correctly"));

static bool GVarMotionMatchTestDeterminismSynchronizeWithExternalDependencies = false;
static FAutoConsoleVariableRef CVarMotionMatchTestDeterminismSynchronizeWithExternalDependencies(TEXT("a.MotionMatch.Test.Determinism.SynchronizeWithExternalDependencies"), GVarMotionMatchTestDeterminismSynchronizeWithExternalDependencies, TEXT("validating SynchronizeWithExternalDependencies doesn't alter the database AnimationAssets order"));

static bool GVarMotionMatchTestDeterminismAssetSampler = false;
static FAutoConsoleVariableRef CVarMotionMatchTestDeterminismAssetSampler(TEXT("a.MotionMatch.Test.Determinism.AssetSampler"), GVarMotionMatchTestDeterminismAssetSampler, TEXT("test FAnimationAssetSampler determinism"));

static bool GVarMotionMatchTestDeterminismAssetSamplerFromPreviousExecution = false;
static FAutoConsoleVariableRef CVarMotionMatchTestDeterminismAssetSamplerFromPreviousExecution(TEXT("a.MotionMatch.Test.Determinism.AssetSamplerFromPreviousExecution"), GVarMotionMatchTestDeterminismAssetSamplerFromPreviousExecution, TEXT("test FAnimationAssetSampler determinism across multiple editor executions. It'll store some bin files in \\Engine\\TestAssetSamplerDeterminism"));

static bool GVarMotionMatchTestDeterminismDDCKey = false;
static FAutoConsoleVariableRef CVarMotionMatchTestDeterminismDDCKey(TEXT("a.MotionMatch.Test.Determinism.DDCKey"), GVarMotionMatchTestDeterminismDDCKey, TEXT("test DDC key generation determinism"));

static int32 GVarMotionMatchTestNumIterations = 10;
static FAutoConsoleVariableRef CVarMotionMatchTestNumIterations(TEXT("a.MotionMatch.Test.NumIterations"), GVarMotionMatchTestNumIterations, TEXT("Test Motion Matching Num Iterations"));

static bool GVarMotionMatchTestDeterminismPartialKeyHashesSaveFile = false;
static FAutoConsoleVariableRef CVarMotionMatchTestDeterminismPartialKeyHashesSaveFile(TEXT("a.MotionMatch.Test.Determinism.PartialKeyHashesSaveFile"), GVarMotionMatchTestDeterminismPartialKeyHashesSaveFile, TEXT("save file \\Engine\\TestPartialKeyHashesDeterminism\\PartialKeyHashes.bin for testing PartialKeyHashes determinism across multiple editor executions"));

static bool GVarMotionMatchTestDeterminismPartialKeyHashes = false;
static FAutoConsoleVariableRef CVarMotionMatchTestDeterminismPartialKeyHashes(TEXT("a.MotionMatch.Test.Determinism.PartialKeyHashes"), GVarMotionMatchTestDeterminismPartialKeyHashes, TEXT("test PartialKeyHashes determinism across multiple editor executions from stored file \\Engine\\TestPartialKeyHashesDeterminism\\PartialKeyHashes.bin"));

static bool GVarMotionMatchReindexCancelledDatabases = false;
static FAutoConsoleVariableRef CVarMotionMatchReindexCancelledDatabases(TEXT("a.MotionMatch.ReindexCancelledDatabases"), GVarMotionMatchReindexCancelledDatabases, TEXT("Reindex Cancelled Databases"));

#endif // ENABLE_ANIM_DEBUG

static bool GVarMotionMatchUseRawAnimationData = false;
static FAutoConsoleVariableRef CVarMotionMatchUseRawAnimationData(TEXT("a.MotionMatch.UseRawAnimationData"), GVarMotionMatchUseRawAnimationData, TEXT("Use Raw Animation Data"), FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		FAsyncPoseSearchDatabasesManagement::Clear();
	}));

static bool GVarMotionMatchReindexAllReferencedDatabases = true;
static FAutoConsoleVariableRef CVarMotionMatchReindexAllReferencedDatabases(TEXT("a.MotionMatch.ReindexAllReferencedDatabases"), GVarMotionMatchReindexAllReferencedDatabases, TEXT("Reindex All Referenced Databases"));

static const UE::DerivedData::FValueId Id(UE::DerivedData::FValueId::FromName("Data"));
static const UE::DerivedData::FCacheBucket Bucket("PoseSearchDatabase");

#if ENABLE_COOK_STATS
static FCookStats::FDDCResourceUsageStats UsageStats;
static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("MotionMatching.Usage"), TEXT(""));
	});
#endif // ENABLE_COOK_STATS

#if ENABLE_ANIM_DEBUG
struct FSerializablePartialKeyHashes
{
	FSerializablePartialKeyHashes() = default;

	explicit FSerializablePartialKeyHashes(const FPartialKeyHashes& PartialKeyHashes)
	{
		Entries.Reserve(PartialKeyHashes.GetEntries().Num());
		for (const FPartialKeyHashes::FPair& Pair : PartialKeyHashes.GetEntries())
		{
			if (UObject* Object = Pair.Key.ResolveObjectPtr())
			{
				FString ObjectName = Object->GetFullName();
				if (Entries.Find(ObjectName))
				{
					UE_LOGF(LogPoseSearch, Error, "FSerializablePartialKeyHashes duplicate asset name '%ls'", *ObjectName);
				}
				else
				{
					FEntry& Entry = Entries.Add(ObjectName);

					Entry.Hash = Pair.Value.Hash;
					Entry.Dependencies.Reserve(Pair.Value.Dependencies.Num());

					for (const FObjectKey& DependencyKey : Pair.Value.Dependencies)
					{
						if (UObject* Dependency = DependencyKey.ResolveObjectPtr())
						{
							FString DependencyName = Dependency->GetFullName();
							Entry.Dependencies.Add(DependencyName);
						}
						else
						{
							Entry.Dependencies.Add("");
						}
					}
				}
			}
		}
	}

	// checking if the intersection of entries between this and Other are consistent
	bool LogDifferencesAgains(const FSerializablePartialKeyHashes& Other) const
	{
		bool bSuccess = true;
		for (const TPair<FString, FEntry>& Pair : Entries)
		{
			if (const FEntry* OtherEntry = Other.Entries.Find(Pair.Key))
			{
				if (Pair.Value.Hash != OtherEntry->Hash)
				{
					UE_LOGF(LogPoseSearch, Error, "FSerializablePartialKeyHashes hash mismatch for asset '%ls'", *Pair.Key);
					bSuccess = false;
				}

				if (Pair.Value.Dependencies != OtherEntry->Dependencies)
				{
					UE_LOGF(LogPoseSearch, Warning, "FSerializablePartialKeyHashes Dependencies mismatch for asset '%ls'", *Pair.Key);
					bSuccess = false;
				}
			}
		}
		return bSuccess;
	}

	friend FArchive& operator<<(FArchive& Ar, FSerializablePartialKeyHashes& SerializablePartialKeyHashes)
	{
		Ar << SerializablePartialKeyHashes.Entries;
		return Ar;
	}

	struct FEntry
	{
		HashDigestType Hash;
		TArray<FString> Dependencies;

		friend FArchive& operator<<(FArchive& Ar, FEntry& Entry)
		{
			Ar << Entry.Hash;
			Ar << Entry.Dependencies;
			return Ar;
		}
	};

	TMap<FString, FEntry> Entries;
};
#endif // ENABLE_ANIM_DEBUG

typedef TSet<const UPoseSearchDatabase*, DefaultKeyFuncs<const UPoseSearchDatabase*>, TInlineSetAllocator<256>> FDatabaseSet;

static void RecursivePopulateDependentDatabases(const UPoseSearchDatabase* Database, FDatabaseSet& DatabaseSet)
{
	if (Database)
	{
		bool bIsAlreadyInSet = false;
		DatabaseSet.Add(Database, &bIsAlreadyInSet);

		if (!bIsAlreadyInSet)
		{
			if (const UPoseSearchNormalizationSet* NormalizationSet = Database->NormalizationSet)
			{
				for (const UPoseSearchDatabase* DependentDatabase : NormalizationSet->Databases)
				{
					RecursivePopulateDependentDatabases(DependentDatabase, DatabaseSet);
				}
			}
		}
	}
}

// helper struct to calculate mean deviations
struct FMeanDeviationCalculator
{
private:
	struct FEntry
	{
		int32 SchemaIndex = INDEX_NONE; // index of the Channel associated Schemas / SearchIndexBases index
		const UPoseSearchFeatureChannel* Channel = nullptr;
	};

	typedef TArray<FEntry, TInlineAllocator<16>> FEntries;		// array of FEntry with channels that can be normalized together (for example it contains all the phases of the left foot from different schemas)
	typedef TArray<FEntries, TInlineAllocator<16>> FEntriesGroup;	// array of FEntries incompatible between each other

	static void Add(const UPoseSearchFeatureChannel* Channel, int32 SchemaIndex, FEntriesGroup& EntriesGroup)
	{
		bool bEntryFound = false;
		for (FEntries& Entries : EntriesGroup)
		{
			if (Entries[0].Channel->CanBeNormalizedWith(Channel))
			{
				Entries.Add({ SchemaIndex, Channel });
				bEntryFound = true;
				break;
			}
		}

		if (!bEntryFound)
		{
			FEntries& Entries = EntriesGroup[EntriesGroup.AddDefaulted()];
			Entries.Add({ SchemaIndex, Channel });
		}
	}

	static void AnalyzeChannelRecursively(const UPoseSearchFeatureChannel* Channel, int32 SchemaIndex, FEntriesGroup& EntriesGroup)
	{
		const TConstArrayView<TObjectPtr<UPoseSearchFeatureChannel>> SubChannels = Channel->GetSubChannels();
		if (SubChannels.Num() == 0)
		{
			Add(Channel, SchemaIndex, EntriesGroup);
		}
		else
		{
			// the channel is a group channel, so we AnalyzeChannelRecursively
			for (const TObjectPtr<UPoseSearchFeatureChannel>& SubChannelPtr : Channel->GetSubChannels())
			{
				if (const UPoseSearchFeatureChannel* SubChannel = SubChannelPtr.Get())
				{
					AnalyzeChannelRecursively(SubChannel, SchemaIndex, EntriesGroup);
				}
			}
		}
	}

	static void AnalyzeSchemas(TConstArrayView<const UPoseSearchSchema*> Schemas, FEntriesGroup& EntriesGroup)
	{
		for (int32 SchemaIndex = 0; SchemaIndex < Schemas.Num(); ++SchemaIndex)
		{
			const UPoseSearchSchema* Schema = Schemas[SchemaIndex];
			for (const TObjectPtr<UPoseSearchFeatureChannel>& ChannelPtr : Schema->GetChannels())
			{
				AnalyzeChannelRecursively(ChannelPtr.Get(), SchemaIndex, EntriesGroup);
			}
		}
	}

	// given an array of channels that can be normalized together (Entries), with the same cardinality (Entries[0].Channel->GetChannelCardinality()),
	// it'll calculate the mean deviation of the associated data (from SearchIndexBases)
	static float CalculateEntriesMeanDeviation(const FEntries& Entries, TConstArrayView<FSearchIndexBase> SearchIndexBases, TConstArrayView<const UPoseSearchSchema*> Schemas)
	{
		check(Schemas.Num() == SearchIndexBases.Num());
	
		const int32 EntriesNum = Entries.Num();
		check(EntriesNum > 0);

		const int32 Cardinality = Entries[0].Channel->GetChannelCardinality();
		check(Cardinality > 0);

		int32 TotalNumValuesVectors = 0;
		for (int32 EntryIdx = 0; EntryIdx < EntriesNum; ++EntryIdx)
		{
			const FEntry& Entry = Entries[EntryIdx];
			check(Cardinality == Entry.Channel->GetChannelCardinality());

			const int32 DataSetIdx = Entry.SchemaIndex;
			const UPoseSearchSchema* Schema = Schemas[DataSetIdx];
			const FSearchIndexBase& SearchIndexBase = SearchIndexBases[DataSetIdx];

			TotalNumValuesVectors += SearchIndexBase.GetNumValuesVectors(Schema->SchemaCardinality);
		}

		int32 AccumulatedNumValuesVectors = 0;
		RowMajorMatrix CenteredSubPoseMatrix(TotalNumValuesVectors, Cardinality);
		for (int32 EntryIdx = 0; EntryIdx < EntriesNum; ++EntryIdx)
		{
			const FEntry& Entry = Entries[EntryIdx];
			const int32 DataSetIdx = Entry.SchemaIndex;

			const UPoseSearchSchema* Schema = Schemas[DataSetIdx];
			const FSearchIndexBase& SearchIndexBase = SearchIndexBases[DataSetIdx];

			const int32 NumValuesVectors = SearchIndexBase.GetNumValuesVectors(Schema->SchemaCardinality);

			// Map input buffer with NumValuesVectors as rows and NumDimensions	as cols
			RowMajorMatrixMapConst PoseMatrixSourceMap(SearchIndexBase.Values.GetData(), NumValuesVectors, Schema->SchemaCardinality);

			// Given the sub matrix for the features, find the average distance to the feature's centroid.
			CenteredSubPoseMatrix.block(AccumulatedNumValuesVectors, 0, NumValuesVectors, Cardinality) = PoseMatrixSourceMap.block(0, Entry.Channel->GetChannelDataOffset(), NumValuesVectors, Cardinality);
			AccumulatedNumValuesVectors += NumValuesVectors;
		}

		RowMajorVector SampleMean = CenteredSubPoseMatrix.colwise().mean();
		CenteredSubPoseMatrix = CenteredSubPoseMatrix.rowwise() - SampleMean;

		// after mean centering the data, the average distance to the centroid is simply the average norm.
		const float FeatureMeanDeviation = CenteredSubPoseMatrix.rowwise().norm().mean();

		return FeatureMeanDeviation;
	}

public:

	// it returns an array of dimension Schemas[0]->SchemaCardinality containing the mean deviation calculated from the data passed in with SearchIndexBases following the layout described in the schemas channels:
	// channels from all the schemas get collected in groups that can be normalized together (FEntriesGroup, populated in AnalyzeSchemas) and then those homogeneous (in cardinality and meaning) groups get processed 
	// one by one in CalculateEntriesMeanDeviation to extract the group mean deviation against the input data contained in SearchIndexBases
	static TArray<float> Calculate(TConstArrayView<FSearchIndexBase> SearchIndexBases, TConstArrayView<const UPoseSearchSchema*> Schemas)
	{
		// This method performs a modified z-score normalization where features are normalized
		// by mean absolute deviation rather than standard deviation. Both methods are preferable
		// here to min-max scaling because they preserve outliers.
		// 
		// Mean absolute deviation is preferred here over standard deviation because the latter
		// emphasizes outliers since squaring the distance from the mean increases variance 
		// exponentially rather than additively and square rooting the sum of squares does not 
		// remove that bias. [1]
		//
		// References:
		// [1] Gorard, S. (2005), "Revisiting a 90-Year-Old Debate: The Advantages of the Mean Deviation."
		//     British Journal of Educational Studies, 53: 417-430.

		int32 ThisSchemaIndex = 0;
		check(SearchIndexBases.Num() == Schemas.Num() && Schemas.Num() > ThisSchemaIndex);
		const UPoseSearchSchema* ThisSchema = Schemas[ThisSchemaIndex];
		const int32 NumDimensions = ThisSchema->SchemaCardinality;

		TArray<float> MeanDeviations;
		MeanDeviations.Init(1.f, NumDimensions);
		RowMajorVectorMap MeanDeviationsMap(MeanDeviations.GetData(), 1, NumDimensions);

		const EPoseSearchDataPreprocessor DataPreprocessor = ThisSchema->DataPreprocessor;
		if (SearchIndexBases[ThisSchemaIndex].GetNumPoses() > 0 && (DataPreprocessor != EPoseSearchDataPreprocessor::None))
		{
			FEntriesGroup EntriesGroup;

			AnalyzeSchemas(Schemas, EntriesGroup);

			for (const FEntries& Entries : EntriesGroup)
			{
				for (const FEntry& Entry : Entries)
				{
					if (Entry.Channel->GetChannelCardinality() > 0 && Entry.SchemaIndex == ThisSchemaIndex)
					{
						const float FeatureMeanDeviation = CalculateEntriesMeanDeviation(Entries, SearchIndexBases, Schemas);
						// the associated data to all the Entries data is going to be used to calculate the deviation of Deviation[Entry.Channel->GetChannelDataOffset()] to Deviation[Entry.Channel->GetChannelDataOffset() + Entry.Channel->GetChannelCardinality()]

						// Fill the feature's corresponding scaling axes with the average distance
						// Avoid scaling by zero by leaving near-zero deviations as 1.0
						static const float MinFeatureMeanDeviation = 0.1f;
						MeanDeviationsMap.segment(Entry.Channel->GetChannelDataOffset(), Entry.Channel->GetChannelCardinality()).setConstant(FeatureMeanDeviation > MinFeatureMeanDeviation ? FeatureMeanDeviation : 1.f);
					}
				}
			}
		}
	
		return MeanDeviations;
	}
};

typedef TArray<FFloatRange, TInlineAllocator<32>> FValidRanges;

static bool FindValidRanges(const FPoseSearchDatabaseAnimationAssetBase* DatabaseAsset, const FVector& BlendParameters, const FFloatInterval& ExcludeFromDatabaseParameters, FValidRanges& ValidRanges)
{
	check(DatabaseAsset);

	const bool bIsLooping = DatabaseAsset->IsLooping();
	const float PlayLength = DatabaseAsset->GetPlayLength(BlendParameters);

	const FFloatInterval EffectiveSamplingInterval = DatabaseAsset->GetEffectiveSamplingRange(BlendParameters);
	FFloatRange EffectiveSamplingRange = FFloatRange::Inclusive(EffectiveSamplingInterval.Min, EffectiveSamplingInterval.Max);
	if (!bIsLooping)
	{
		const FFloatRange ExcludeFromDatabaseRange(ExcludeFromDatabaseParameters.Min, PlayLength + ExcludeFromDatabaseParameters.Max);
		EffectiveSamplingRange = FFloatRange::Intersection(EffectiveSamplingRange, ExcludeFromDatabaseRange);
	}

	// start from a single interval defined by the database sequence sampling range
	ValidRanges.Reset();
	ValidRanges.Add(EffectiveSamplingRange);

	bool bAnyExclusionNotifyStateFound = false;
	bool bAnySamplingRangeNotifyStateFound = false;

	for (int32 RoleIndex = 0; RoleIndex < DatabaseAsset->GetNumRoles(); ++RoleIndex)
	{
		const FRole Role = DatabaseAsset->GetRole(RoleIndex);
		if (UAnimSequenceBase* SequenceBase = Cast<UAnimSequenceBase>(DatabaseAsset->GetAnimationAssetForRole(Role)))
		{
			FAnimNotifyContext NotifyContext;
			SequenceBase->GetAnimNotifies(0.0f, PlayLength, NotifyContext);
			
			for (const FAnimNotifyEventReference& EventReference : NotifyContext.ActiveNotifies)
			{
				if (const FAnimNotifyEvent* NotifyEvent = EventReference.GetNotify())
				{
					if (const UAnimNotifyState_PoseSearchExcludeFromDatabase* ExclusionNotifyState = Cast<const UAnimNotifyState_PoseSearchExcludeFromDatabase>(NotifyEvent->NotifyStateClass))
					{
						if (bAnySamplingRangeNotifyStateFound)
						{
							UE_LOGF(LogPoseSearch, Error, "FindValidRanges conflicting UAnimNotifyState_PoseSearchExcludeFromDatabase and UAnimNotifyState_PoseSearchSamplingRange found in '%ls'.", *DatabaseAsset->GetAnimationAsset()->GetName());
							ValidRanges.Reset();
							return false;
						}

						bAnyExclusionNotifyStateFound = true;
						const FFloatRange ExclusionRange = FFloatRange::Inclusive(NotifyEvent->GetTime(), NotifyEvent->GetTime() + NotifyEvent->GetDuration());

						// Split every valid range based on the exclusion range just found. Because this might increase the 
						// number of ranges in ValidRanges, the algorithm iterates from end to start.
						for (int32 RangeIdx = ValidRanges.Num() - 1; RangeIdx >= 0; --RangeIdx)
						{
							const FFloatRange EvaluatedRange = ValidRanges[RangeIdx];
							ValidRanges.RemoveAt(RangeIdx);

							const TArray<FFloatRange> Diff = FFloatRange::Difference(EvaluatedRange, ExclusionRange);
							ValidRanges.Append(Diff);
						}
					}
					else if (const UAnimNotifyState_PoseSearchSamplingRange* SamplingRangeNotifyState = Cast<const UAnimNotifyState_PoseSearchSamplingRange>(NotifyEvent->NotifyStateClass))
					{
						if (bAnyExclusionNotifyStateFound)
						{
							UE_LOGF(LogPoseSearch, Error, "FindValidRanges conflicting UAnimNotifyState_PoseSearchExcludeFromDatabase and UAnimNotifyState_PoseSearchSamplingRange found in '%ls'.", *DatabaseAsset->GetAnimationAsset()->GetName());
							ValidRanges.Reset();
							return false;
						}

						FFloatRange InclusionRange = FFloatRange::Inclusive(NotifyEvent->GetTime(), NotifyEvent->GetTime() + NotifyEvent->GetDuration());
						if (!bAnySamplingRangeNotifyStateFound)
						{
							ValidRanges.Reset();
							ValidRanges.Add(InclusionRange);
							bAnySamplingRangeNotifyStateFound = true;
						}
						else
						{
							for (int32 RangeIdx = ValidRanges.Num() - 1; RangeIdx >= 0; --RangeIdx)
							{
								const FFloatRange EvaluatedRange = ValidRanges[RangeIdx];
								if (InclusionRange.Contiguous(EvaluatedRange))
								{
									ValidRanges.RemoveAt(RangeIdx);
									const TArray<FFloatRange> Union = FFloatRange::Union(InclusionRange, EvaluatedRange);
									if (ensure(Union.Num() == 1))
									{
										InclusionRange = Union[0];
									}
								}
							}

							ValidRanges.Add(InclusionRange);
						}
					}
				}
			}
		}
	}
	
	return true;
}

// returns false in case of errors
static bool IsSchemaCompatible(const FPoseSearchDatabaseAnimationAssetBase* DatabaseAsset, const UPoseSearchSchema* Schema)
{
	const int32 DatabaseAssetNumRoles = DatabaseAsset->GetNumRoles();
	if (DatabaseAssetNumRoles != Schema->GetRoledSkeletons().Num())
	{
		return false;
	}

	for (int32 DatabaseAssetRoleIndex = 0; DatabaseAssetRoleIndex < DatabaseAssetNumRoles; ++DatabaseAssetRoleIndex)
	{
		const FRole DatabaseAssetRole = DatabaseAsset->GetRole(DatabaseAssetRoleIndex);
		if (!Schema->GetRoledSkeleton(DatabaseAssetRole))
		{
			return false;
		}
	}
	
	return true;
}

// returns false in case of errors
static bool InitSearchIndexAssets(FSearchIndexBase& SearchIndex, const UPoseSearchDatabase* DatabaseToLookForAssets, const UPoseSearchSchema* Schema, const FFloatInterval& ExcludeFromDatabaseParameters)
{
	using namespace UE::PoseSearch;

	check(Schema);

	SearchIndex.Assets.Reset();
	FValidRanges ValidRanges;

	bool bAnyErrors = false;

	int32 TotalPoses = 0;
	for (int32 AnimationAssetIndex = 0; AnimationAssetIndex < DatabaseToLookForAssets->GetNumAnimationAssets(); ++AnimationAssetIndex)
	{
		if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAsset = DatabaseToLookForAssets->GetDatabaseAnimationAsset(AnimationAssetIndex))
		{
			if (!DatabaseAsset->IsEnabled() || !DatabaseAsset->GetAnimationAsset())
			{
				continue;
			}

			// checking for duplicated roles in DatabaseAsset
			TSet<FRole, DefaultKeyFuncs<FRole>, TInlineSetAllocator<PreallocatedRolesNum>> DatabaseAssetRoles;
			const int32 NumRoles = DatabaseAsset->GetNumRoles();
			for (int32 RoleIndex = 0; RoleIndex < NumRoles; ++RoleIndex)
			{
				bool bIsAlreadyInSet = false;
				const FRole Role = DatabaseAsset->GetRole(RoleIndex);
				DatabaseAssetRoles.Add(Role, &bIsAlreadyInSet);
				if (bIsAlreadyInSet)
				{
					UE_LOGF(LogPoseSearch, Error, "DatabaseAsset '%ls' contains duplicate Role '%ls'", *DatabaseAsset->GetAnimationAsset()->GetName(), *Role.ToString());
					bAnyErrors = true;
				}
			}

			// checking for valid roles in DatabaseMultiAnimAsset against the Schema
			bool bAreAllRolesSupported = true;
			for (const FPoseSearchRoledSkeleton& RoledSkeleton : Schema->GetRoledSkeletons())
			{
				if (!DatabaseAsset->GetAnimationAssetForRole(RoledSkeleton.Role))
				{
					UE_LOGF(LogPoseSearch, Error, "DatabaseAsset '%ls' doesn't support Role '%ls' required by Schema '%ls' Skeletons", *DatabaseAsset->GetAnimationAsset()->GetName(), *RoledSkeleton.Role.ToString(), *Schema->GetName());
					bAreAllRolesSupported = false;
				}
			}
			if (!bAreAllRolesSupported)
			{
				bAnyErrors = true;
				continue;
			}

			const bool bAddUnmirrored = DatabaseAsset->GetMirrorOption() == EPoseSearchMirrorOption::UnmirroredOnly || DatabaseAsset->GetMirrorOption() == EPoseSearchMirrorOption::UnmirroredAndMirrored;
			const bool bAddMirrored = DatabaseAsset->GetMirrorOption() == EPoseSearchMirrorOption::MirroredOnly || DatabaseAsset->GetMirrorOption() == EPoseSearchMirrorOption::UnmirroredAndMirrored;
			const bool bIsLooping = DatabaseAsset->IsLooping();
			const bool bDisableReselection = DatabaseAsset->IsDisableReselection();

			// @todo: add better support for IMultiAnimAsset: currently we fix blend space parameters for only one role, 
			//        that implies having homogeneous blendspaces for ALL the roles, and having either ONLY blend spaces or only NOT blendspaces..
			bool bIsBlendSpace = false;
			if (UAnimationAsset* DefaultRoleAnimationAsset = DatabaseAsset->GetAnimationAssetForRole(DatabaseToLookForAssets->Schema->GetDefaultRole()))
			{
				bIsBlendSpace = DefaultRoleAnimationAsset->IsA<UBlendSpace>();
			}

			DatabaseAsset->IterateOverSamplingParameter(
				[DatabaseAsset, ExcludeFromDatabaseParameters, &ValidRanges, Schema, bAddUnmirrored, bAddMirrored, 
				bIsLooping, bIsBlendSpace, bDisableReselection, AnimationAssetIndex, &TotalPoses, &SearchIndex, &bAnyErrors]
				(const FVector& BlendParameters)
				{
					if (FindValidRanges(DatabaseAsset, BlendParameters, ExcludeFromDatabaseParameters, ValidRanges))
					{
						for (const FFloatRange& Range : ValidRanges)
						{
							for (int32 PermutationIdx = 0; PermutationIdx < Schema->NumberOfPermutations; ++PermutationIdx)
							{
								const FFloatInterval RangeInterval(Range.GetLowerBoundValue(), Range.GetUpperBoundValue());

								float ToRealTimeFactor = 1.f;
								if (bIsBlendSpace)
								{
									const float PlayLength = DatabaseAsset->GetPlayLength(BlendParameters);
									if (PlayLength > UE_KINDA_SMALL_NUMBER)
									{
										ToRealTimeFactor = PlayLength;
									}
								}

								if (bAddUnmirrored)
								{
									const FSearchIndexAsset PoseSearchIndexAsset(AnimationAssetIndex, TotalPoses, false, bIsLooping,
										bDisableReselection, RangeInterval, Schema->SampleRate, PermutationIdx, BlendParameters, ToRealTimeFactor);
									if (PoseSearchIndexAsset.GetNumPoses() > 0)
									{
										SearchIndex.Assets.Add(PoseSearchIndexAsset);
										TotalPoses += PoseSearchIndexAsset.GetNumPoses();
									}
								}

								if (bAddMirrored)
								{
									const FSearchIndexAsset PoseSearchIndexAsset(AnimationAssetIndex, TotalPoses, true, bIsLooping,
										bDisableReselection, RangeInterval, Schema->SampleRate, PermutationIdx, BlendParameters, ToRealTimeFactor);
									if (PoseSearchIndexAsset.GetNumPoses() > 0)
									{
										SearchIndex.Assets.Add(PoseSearchIndexAsset);
										TotalPoses += PoseSearchIndexAsset.GetNumPoses();
									}
								}
							}
						}
					}
					else
					{
						bAnyErrors = true;
					}
				});
		}
	}

	return !bAnyErrors;
}

static bool PreprocessSearchIndexWeights(FSearchIndex& SearchIndex, const UPoseSearchSchema* Schema, TConstArrayView<float> Deviation)
{
#if WITH_EDITORONLY_DATA
	// Copy deviation into search index for weights display in editor
	SearchIndex.DeviationEditorOnly = Deviation;
#endif // WITH_EDITORONLY_DATA

	const int32 NumDimensions = Schema->SchemaCardinality;
	SearchIndex.WeightsSqrt.Init(1.f, NumDimensions);

	for (const TObjectPtr<UPoseSearchFeatureChannel>& ChannelPtr : Schema->GetChannels())
	{
		ChannelPtr->FillWeights(SearchIndex.WeightsSqrt);

		const int32 ChannelDataStart = ChannelPtr->GetChannelDataOffset();
		const int32 ChannelDataEnd = ChannelDataStart + ChannelPtr->GetChannelCardinality();

		for (int32 ChannelDataIndex = ChannelDataStart; ChannelDataIndex < ChannelDataEnd; ++ChannelDataIndex)
		{
			if (SearchIndex.WeightsSqrt[ChannelDataIndex] < 0.f)
			{
				UE_LOGF(LogPoseSearch, Error, "Error in PreprocessSearchIndexWeights: channel '%ls' in schema '%ls' has a negative weight!", *ChannelPtr->GetName(), *Schema->GetName());
				return false;
			}
		}
	}

	EPoseSearchDataPreprocessor DataPreprocessor = Schema->DataPreprocessor;
	if (DataPreprocessor == EPoseSearchDataPreprocessor::Normalize || DataPreprocessor == EPoseSearchDataPreprocessor::NormalizeWithCommonSchema)
	{
		// normalizing user weights: the idea behind this step is to be able to compare poses from databases using different schemas
		RowMajorVectorMap MapWeights(SearchIndex.WeightsSqrt.GetData(), 1, NumDimensions);
		const float WeightsSum = MapWeights.sum();
		if (WeightsSum > UE_SMALL_NUMBER)
		{
			MapWeights *= (1.f / WeightsSum);
		}
	}

	// extracting the square root
	for (int32 Dimension = 0; Dimension != NumDimensions; ++Dimension)
	{
		SearchIndex.WeightsSqrt[Dimension] = FMath::Sqrt(SearchIndex.WeightsSqrt[Dimension]);
	}

	if (DataPreprocessor != EPoseSearchDataPreprocessor::None)
	{
		for (int32 Dimension = 0; Dimension != NumDimensions; ++Dimension)
		{
			// the idea here is to pre-multiply the weights by the inverse of the variance (proportional to the square of the deviation) to have a "weighted Mahalanobis" distance
			if (Deviation[Dimension] > UE_SMALL_NUMBER)
			{
				SearchIndex.WeightsSqrt[Dimension] /= Deviation[Dimension];
			}
		}
	}

	return true;
}

// it calculates Mean, PCAValues, and PCAProjectionMatrix
static Eigen::ComputationInfo PreprocessSearchIndexPCAData(FSearchIndex& SearchIndex, int32 NumDimensions, int32 NumberOfPrincipalComponents, EPoseSearchMode PoseSearchMode)
{
#if ENABLE_ANIM_DEBUG
	if (GVarMotionMatchTestKDTreeConstruct)
	{
		// @todo: move this into a unit test.
		// this code will fail with nanoflann 1.5.5

		int NumPoses = 61;
		int DataCardinality = 8;
		TArray<float> Values;
		Values.SetNumZeroed(NumPoses * DataCardinality);

		for (int PoseIndex = 0; PoseIndex < NumPoses; ++PoseIndex)
		{
			Values[PoseIndex * DataCardinality + 0] = -5.54383405e-07f;
			Values[PoseIndex * DataCardinality + 1] = 2.77555756e-16f;
		}

		FKDTree KDTree(NumPoses, DataCardinality, Values.GetData());
	}
#endif // ENABLE_ANIM_DEBUG


	// binding SearchIndex.Values and SearchIndex.PCAValues Eigen row major matrix maps
	const int32 NumPoses = SearchIndex.GetNumPoses();

#if WITH_EDITORONLY_DATA
	SearchIndex.PCAExplainedVarianceEditorOnly = 0.f;
#endif

	SearchIndex.PCAValues.Reset();
	SearchIndex.Mean.Reset();
	SearchIndex.PCAProjectionMatrix.Reset();

	Eigen::ComputationInfo ComputationInfo = Eigen::Success;
	if (PoseSearchMode == EPoseSearchMode::PCAKDTree && NumDimensions > 0 && NumPoses > 0 && NumberOfPrincipalComponents > 0)
	{
		SearchIndex.PCAValues.AddZeroed(NumPoses * NumberOfPrincipalComponents);
		SearchIndex.Mean.AddZeroed(NumDimensions);
		SearchIndex.PCAProjectionMatrix.AddZeroed(NumDimensions * NumberOfPrincipalComponents);

		// recreating the full pose values data to have a 1:1 mapping between PCAValues/NumDimensions and PoseIdx
		TArray<float> AllValuesWithDuplicateData;
		AllValuesWithDuplicateData.SetNum(NumPoses * NumDimensions);
		for (int32 PoseIdx = 0; PoseIdx < NumPoses; ++PoseIdx)
		{
			FMemory::Memcpy(AllValuesWithDuplicateData.GetData() + PoseIdx * NumDimensions, SearchIndex.GetPoseValuesBase(PoseIdx, NumDimensions).GetData(), NumDimensions * sizeof(float));
		}

		const RowMajorVectorMapConst MapWeightsSqrt(SearchIndex.WeightsSqrt.GetData(), 1, NumDimensions);
		const RowMajorMatrixMapConst MapValues(AllValuesWithDuplicateData.GetData(), NumPoses, NumDimensions);
		const RowMajorMatrix WeightedValues = MapValues.array().rowwise() * MapWeightsSqrt.array();
		RowMajorMatrixMap MapPCAValues(SearchIndex.PCAValues.GetData(), NumPoses, NumberOfPrincipalComponents);

		// calculating the mean
		RowMajorVectorMap MapMean(SearchIndex.Mean.GetData(), 1, NumDimensions);
		MapMean = WeightedValues.colwise().mean();

		// use the mean to center the data points
		const RowMajorMatrix CenteredValues = WeightedValues.rowwise() - MapMean;

		// estimating the covariance matrix (with dimensionality of NumDimensions, NumDimensions)
		// formula: https://en.wikipedia.org/wiki/Covariance_matrix#Estimation
		// details: https://en.wikipedia.org/wiki/Estimation_of_covariance_matrices
		const ColMajorMatrix CovariantMatrix = (CenteredValues.transpose() * CenteredValues) / float(NumPoses - 1);
		const Eigen::SelfAdjointEigenSolver<ColMajorMatrix> EigenSolver(CovariantMatrix);

		ComputationInfo = EigenSolver.info();
		if (ComputationInfo == Eigen::Success)
		{
			// validating EigenSolver results
			const ColMajorMatrix EigenVectors = EigenSolver.eigenvectors().real();

#if ENABLE_ANIM_DEBUG
			if (GVarMotionMatchTestKDTreeConstruct && NumberOfPrincipalComponents == NumDimensions)
			{
				const RowMajorVector ReciprocalWeightsSqrt = MapWeightsSqrt.cwiseInverse();
				const RowMajorMatrix ProjectedValues = CenteredValues * EigenVectors;
				for (Eigen::Index RowIndex = 0; RowIndex < MapValues.rows(); ++RowIndex)
				{
					const RowMajorVector WeightedReconstructedPoint = ProjectedValues.row(RowIndex) * EigenVectors.transpose() + MapMean;
					const RowMajorVector ReconstructedPoint = WeightedReconstructedPoint.array() * ReciprocalWeightsSqrt.array();
					const float Error = (ReconstructedPoint - MapValues.row(RowIndex)).squaredNorm();
					check(Error < UE_KINDA_SMALL_NUMBER);
				}
			}
#endif // ENABLE_ANIM_DEBUG

			// sorting EigenVectors by EigenValues, so we pick the most significant ones to compose our PCA projection matrix.
			const RowMajorVector EigenValues = EigenSolver.eigenvalues().real();
			TArray<int32> Indexer;
			Indexer.Reserve(NumDimensions);
			for (int32 DimensionIndex = 0; DimensionIndex < NumDimensions; ++DimensionIndex)
			{
				Indexer.Push(DimensionIndex);
			}
			Indexer.Sort([&EigenValues](int32 a, int32 b)
				{
					return EigenValues[a] > EigenValues[b];
				});

			// composing the PCA projection matrix with the PCANumComponents most significant EigenVectors
			ColMajorMatrixMap PCAProjectionMatrix(SearchIndex.PCAProjectionMatrix.GetData(), NumDimensions, NumberOfPrincipalComponents);
			float AccumulatedVariance = 0.f;
			for (int32 PCAComponentIndex = 0; PCAComponentIndex < NumberOfPrincipalComponents; ++PCAComponentIndex)
			{
				PCAProjectionMatrix.col(PCAComponentIndex) = EigenVectors.col(Indexer[PCAComponentIndex]);
				AccumulatedVariance += EigenValues[Indexer[PCAComponentIndex]];
			}

#if WITH_EDITORONLY_DATA
			// calculating the total variance knowing that eigen values measure variance along the principal components:
			const float TotalVariance = EigenValues.sum();
			// and explained variance as ratio between AccumulatedVariance and TotalVariance: https://ro-che.info/articles/2017-12-11-pca-explained-variance
			SearchIndex.PCAExplainedVarianceEditorOnly = TotalVariance > UE_KINDA_SMALL_NUMBER ? AccumulatedVariance / TotalVariance : 0.f;
#endif // WITH_EDITORONLY_DATA

			MapPCAValues = CenteredValues * PCAProjectionMatrix;

#if ENABLE_ANIM_DEBUG
			if (GVarMotionMatchTestKDTreeConstruct && NumberOfPrincipalComponents == NumDimensions)
			{
				const RowMajorVector ReciprocalWeightsSqrt = MapWeightsSqrt.cwiseInverse();
				for (Eigen::Index RowIndex = 0; RowIndex < MapValues.rows(); ++RowIndex)
				{
					const RowMajorVector WeightedReconstructedValues = MapPCAValues.row(RowIndex) * PCAProjectionMatrix.transpose() + MapMean;
					const RowMajorVector ReconstructedValues = WeightedReconstructedValues.array() * ReciprocalWeightsSqrt.array();
					const float Error = (ReconstructedValues - MapValues.row(RowIndex)).squaredNorm();
					check(Error < UE_KINDA_SMALL_NUMBER);
				}

				TArray<float> ReconstructedPoseValues;
				ReconstructedPoseValues.SetNumZeroed(NumDimensions);
				for (int32 PoseIdx = 0; PoseIdx < NumPoses; ++PoseIdx)
				{
					SearchIndex.GetReconstructedPoseValues(PoseIdx, ReconstructedPoseValues);
					TConstArrayView<float> PoseValues = SearchIndex.GetPoseValues(PoseIdx);

					check(ReconstructedPoseValues.Num() == PoseValues.Num());
					Eigen::Map<const Eigen::ArrayXf> VA(ReconstructedPoseValues.GetData(), ReconstructedPoseValues.Num());
					Eigen::Map<const Eigen::ArrayXf> VB(PoseValues.GetData(), PoseValues.Num());

					const float Error = (VA - VB).square().sum();
					check(Error < UE_KINDA_SMALL_NUMBER);
				}
			}
#endif // ENABLE_ANIM_DEBUG
		}
	}

	return ComputationInfo;
}

static void PreprocessSearchIndexKDTree(FSearchIndex& SearchIndex, const UPoseSearchDatabase* Database)
{
	const int32 NumDimensions = Database->Schema->SchemaCardinality;
	const EPoseSearchMode PoseSearchMode = Database->PoseSearchMode;

	SearchIndex.KDTree.Reset();
	if (NumDimensions > 0 && PoseSearchMode == EPoseSearchMode::PCAKDTree)
	{
		const uint32 NumberOfPrincipalComponents = Database->GetNumberOfPrincipalComponents();
		const int32 KDTreeMaxLeafSize = Database->KDTreeMaxLeafSize;

		const int32 NumPCAValuesVectors = SearchIndex.GetNumPCAValuesVectors(NumberOfPrincipalComponents);
		SearchIndex.KDTree.Construct(NumPCAValuesVectors, NumberOfPrincipalComponents, SearchIndex.PCAValues.GetData(), KDTreeMaxLeafSize);

#if ENABLE_ANIM_DEBUG
		// testing kdtree Construct determinism
		if (GVarMotionMatchTestDeterminismKDTreeConstruct)
		{
			TAlignedArray<float> PCAValuesTest;
			for (int32 Iteration = 0; Iteration < GVarMotionMatchTestNumIterations; ++Iteration)
			{
				// copy PCAValues in a different container to ensure input data has different memory addresses
				PCAValuesTest = SearchIndex.PCAValues;

				FKDTree KDTreeTest;
				KDTreeTest.Construct(NumPCAValuesVectors, NumberOfPrincipalComponents, PCAValuesTest.GetData(), KDTreeMaxLeafSize);

				if (KDTreeTest != SearchIndex.KDTree)
				{
					UE_LOGF(LogPoseSearch, Warning, "PreprocessSearchIndexKDTree - FKDTree::Construct is not deterministic");
				}
			}
		}

		if (GVarMotionMatchTestKDTreeConstruct)
		{
			// testing the KDTree is returning the proper searches for all the points in pca space
			const int32 KDTreeQueryNumNeighbors = Database->KDTreeQueryNumNeighbors;

			TArray<FKDTree::FKNNMaxHeapResultSet::FResult, TInlineAllocator<256>> Results;
			Results.SetNumUninitialized(KDTreeQueryNumNeighbors);
			int32 MaxNumNeighborToFindAPoint = 0;
			for (int32 PointIndex = 0; PointIndex < NumPCAValuesVectors; ++PointIndex)
			{
				// searching the kdtree for PointIndex
				FKDTree::FRadiusMaxHeapResultSet ResultSet(Results, UE_SMALL_NUMBER);
				const int32 NumResults = SearchIndex.KDTree.FindNeighbors(ResultSet, MakeArrayView(&SearchIndex.PCAValues[PointIndex * NumberOfPrincipalComponents], NumberOfPrincipalComponents));

				bool bFound = false;
				for (int32 ResultIndex = 0; ResultIndex < NumResults; ++ResultIndex)
				{
					if (PointIndex == Results[ResultIndex].Index)
					{
						// PointIndex is the ResultIndex-th candidates out of the kdtree. if ResultIndex-th is greater than KDTreeQueryNumNeighbors,
						// we wouldn't have found it in a runtime search, so we log the errot (later on only once, with the worst case scenario)
						check(Results[ResultIndex].Distance < UE_KINDA_SMALL_NUMBER);
						MaxNumNeighborToFindAPoint = FMath::Max(MaxNumNeighborToFindAPoint, ResultIndex);
						bFound = true;
						break;
					}
				}
				
				if (!bFound)
				{
					UE_LOGF(LogPoseSearch, Error, "PreprocessSearchIndexKDTree - kdtree for %ls is not properly constructed! Couldn't find the Point %d in it", *Database->GetName(), PointIndex);
				}
			}

			if (MaxNumNeighborToFindAPoint >= KDTreeQueryNumNeighbors)
			{
				UE_LOGF(LogPoseSearch, Warning, "Not enough 'KDTreeQueryNumNeighbors' (%d) for database '%ls'. Pose values projected in PCA space have too many duplicates, so try to prune duplicates by tuning 'PCAValuesPruningSimilarityThreshold' or increase 'KDTreeQueryNumNeighbors' at least to %d", KDTreeQueryNumNeighbors, *Database->GetName(), MaxNumNeighborToFindAPoint);
			}

			// if bArePCAValuesPruned PointIndex is the index of the point in the kdtree, NOT necessary the pose index, so doing the PCAProject would lead to the wrong data
			const bool bArePCAValuesPruned = SearchIndex.PCAValuesVectorToPoseIndexes.Num() > 0;
			if (!bArePCAValuesPruned)
			{
				// testing the KDTree is returning the proper searches for all the original points transformed in pca space
				TArrayView<float> ProjectedValues((float*)FMemory_Alloca(NumberOfPrincipalComponents * sizeof(float)), NumberOfPrincipalComponents);
				for (int32 PointIndex = 0; PointIndex < NumPCAValuesVectors; ++PointIndex)
				{
					FKDTree::FKNNMaxHeapResultSet ResultSet(Results);
					const int32 NumResults = SearchIndex.KDTree.FindNeighbors(ResultSet, SearchIndex.PCAProject(SearchIndex.GetPoseValuesBase(PointIndex, NumDimensions), ProjectedValues));

					int32 ResultIndex = 0;
					for (; ResultIndex < NumResults; ++ResultIndex)
					{
						if (PointIndex == Results[ResultIndex].Index)
						{
							if (Results[ResultIndex].Distance > UE_KINDA_SMALL_NUMBER)
							{
								UE_LOGF(LogPoseSearch, Error, "PreprocessSearchIndexKDTree - kdtree for %ls is not properly constructed! Couldn't find the Point %d in it within UE_KINDA_SMALL_NUMBER tolerance, after PCA projection", *Database->GetName(), PointIndex);
							}
							break;
						}
					}
					if (ResultIndex == NumResults)
					{
						UE_LOGF(LogPoseSearch, Error, "PreprocessSearchIndexKDTree - kdtree for %ls is not properly constructed! Couldn't find the Point %d in it, after PCA projection", *Database->GetName(), PointIndex);
					}
				}
			}
		}
#endif // ENABLE_ANIM_DEBUG
	}
}

// creating a vantage point tree
static void PreprocessSearchIndexVPTree(FSearchIndex& SearchIndex, const UPoseSearchDatabase* Database, int32 RandomSeed)
{
	const int32 NumDimensions = Database->Schema->SchemaCardinality;
	const int32 NumValuesVectors = SearchIndex.GetNumValuesVectors(NumDimensions);
	const EPoseSearchMode PoseSearchMode = Database->PoseSearchMode;

	SearchIndex.VPTree.Reset();
	if (PoseSearchMode == EPoseSearchMode::VPTree && NumValuesVectors > 0)
	{
		const FVPTreeDataSource DataSource(SearchIndex);
		FRandomStream RandStream(RandomSeed);
		SearchIndex.VPTree.Construct(DataSource, RandStream);

#if ENABLE_ANIM_DEBUG

		if (GVarMotionMatchTestDeterminismVPTreeConstruct)
		{
			if (!SearchIndex.VPTree.TestConstruct(DataSource))
			{
				UE_LOGF(LogPoseSearch, Error, "PreprocessSearchIndexVPTree - FVPTree construction failed");
			}

			const int32 NumIterations = GVarMotionMatchTestNumIterations;
			for (int32 Iteration = 0; Iteration < NumIterations; ++Iteration)
			{
				FVPTree VPTreeTest;
				FRandomStream RandStreamTest(RandomSeed);
				VPTreeTest.Construct(DataSource, RandStreamTest);

				if (VPTreeTest != SearchIndex.VPTree)
				{
					UE_LOGF(LogPoseSearch, Warning, "PreprocessSearchIndexVPTree - FVPTree construction is not deterministic");
				}
			}
		}

		if (GVarMotionMatchTestVPTreeConstruct)
		{
			// we can validate vantage point tree only if there are no duplicates poses (points)
			if (Database->PosePruningSimilarityThreshold > 0.f)
			{
				// testing the VPTree is returning the proper searches for all the points
				for (int32 PointIndex = 0; PointIndex < NumValuesVectors; ++PointIndex)
				{
					FVPTreeResultSet ResultSet(Database->KDTreeQueryNumNeighbors);
					SearchIndex.VPTree.FindNeighbors(SearchIndex.GetValuesVector(PointIndex, NumDimensions), ResultSet, DataSource);
		
					bool bFound = false;
					for (const FIndexDistance& Result : ResultSet.GetUnsortedResults())
					{
						if (Result.Index == PointIndex)
						{
							if (!FMath::IsNearlyZero(Result.Distance))
							{
								UE_LOGF(LogPoseSearch, Error, "PreprocessSearchIndexVPTree - VPTree for %ls is malformed because foud PointIndex %d distance %f from itself (distance should be zero)!", *Database->GetName(), PointIndex, Result.Distance);
							}

							bFound = true;
							break;
						}
					}

					if (!bFound)
					{
						UE_LOGF(LogPoseSearch, Error, "PreprocessSearchIndexVPTree - VPTree for %ls is malformed and couldn't find PointIndex %d", *Database->GetName(), PointIndex);
					}
				}
			}
			else
			{
				UE_LOGF(LogPoseSearch, Warning, "PreprocessSearchIndexVPTree - cannot VPTreeConstruct for %ls if there could be potential duplicate poses: set PosePruningSimilarityThreshold > 0 to enforce duplicate pruning", *Database->GetName());
			}
		}
#endif // ENABLE_ANIM_DEBUG
	}
}

// this struct exists because FTransform doesn't implement operator== (or it'd be TTuple<const UAnimationAsset*, FTransform, FVector>)
struct FSamplerMapKey
{
	FSamplerMapKey(const UAnimationAsset* InAnimationAsset, const FTransform& InRootTransformOrigin, const FVector& InBlendParameters = FVector::ZeroVector)
		: AnimationAsset(InAnimationAsset)
		, RootTransformOrigin(InRootTransformOrigin)
		, BlendParameters(InBlendParameters)
	{
	}

	bool operator==(const FSamplerMapKey& Other) const
	{
		return AnimationAsset == Other.AnimationAsset &&
			RootTransformOrigin.Equals(Other.RootTransformOrigin, 0.f) &&
			BlendParameters == Other.BlendParameters;
	}

	friend FORCEINLINE uint32 GetTypeHash(const FSamplerMapKey& SamplerMapKey)
	{
		const uint32 AnimationAssetHash = GetTypeHash(SamplerMapKey.AnimationAsset);
		const uint32 RootTransformOriginHash = GetTypeHash(SamplerMapKey.RootTransformOrigin);
		const uint32 BlendParametersHash = GetTypeHash(SamplerMapKey.BlendParameters);
		return HashCombineFast(HashCombineFast(AnimationAssetHash, RootTransformOriginHash), BlendParametersHash);
	}

	const UAnimationAsset* AnimationAsset = nullptr;
	FTransform RootTransformOrigin = FTransform::Identity;
	FVector BlendParameters = FVector::ZeroVector;
};

static void InitRoledBoneContainers(TMap<FRole, FBoneContainer>& RoledBoneContainers, const UPoseSearchDatabase* DatabaseToLookForAssets, const UPoseSearchSchema* Schema)
{
	auto AddBoneIndex = [](int32 BoneIndex, TArray<uint16>& BoneIndicesWithParents, const UMirrorDataTable* MirrorDataTable)
	{
		BoneIndicesWithParents.AddUnique(BoneIndex);

		// adding mirrored BoneIndex if there's a valid MirrorDataTable
		if (MirrorDataTable)
		{
			if (MirrorDataTable->BoneToMirrorBoneIndex.IsValidIndex(BoneIndex))
			{
				const FSkeletonPoseBoneIndex MirroredBoneIndex = MirrorDataTable->BoneToMirrorBoneIndex[BoneIndex];
				if (MirroredBoneIndex.IsValid())
				{
					BoneIndicesWithParents.AddUnique(MirroredBoneIndex.GetInt());
				}
			}
			else
			{
				UE_LOGF(LogPoseSearch, Warning, "InitBoneContainersFromRoledSkeleton: MirrorDataTable %ls doesn't contain bone with index %d.", *MirrorDataTable->GetName(), BoneIndex);
			}
		}
	};

	RoledBoneContainers.Reset();
	RoledBoneContainers.Reserve(Schema->GetRoledSkeletons().Num());

	// filling up BoneIndicesWithParents with all the bone indexes from the bones in the 
	// schema roled skeletons and from UAnimNotifyState_PoseSearchSamplingAttribute
	TArray<uint16> BoneIndicesWithParents;
	BoneIndicesWithParents.Reserve(128);

	for (const FPoseSearchRoledSkeleton& RoledSkeleton : Schema->GetRoledSkeletons())
	{
		BoneIndicesWithParents.Reset();

		FBoneContainer& RoledBoneContainer = RoledBoneContainers.Add(RoledSkeleton.Role);
		// Add a curve filter to our bone container to only eval curves actually used by the schema.
		const UE::Anim::FCurveFilterSettings CurveFilterSettings(UE::Anim::ECurveFilterMode::AllowOnlyFiltered, &RoledSkeleton.RequiredCurves);

		// Initialize references to obtain bone indices and fill out bone index array
		for (const FBoneReference& BoneRef : RoledSkeleton.BoneReferences)
		{
			check(BoneRef.HasValidSetup());
			AddBoneIndex(BoneRef.BoneIndex, BoneIndicesWithParents, RoledSkeleton.MirrorDataTable);
		}

		for (int32 AnimationAssetIndex = 0; AnimationAssetIndex < DatabaseToLookForAssets->GetNumAnimationAssets(); ++AnimationAssetIndex)
		{
			if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAsset = DatabaseToLookForAssets->GetDatabaseAnimationAsset(AnimationAssetIndex))
			{
				if (!DatabaseAsset->IsEnabled() || !DatabaseAsset->GetAnimationAsset())
				{
					continue;
				}

				const FAnimationAssetSampler Sampler(DatabaseAsset->GetAnimationAssetForRole(RoledSkeleton.Role), FTransform::Identity, FVector::ZeroVector, FAnimationAssetSampler::DefaultRootTransformSamplingRate, false, true);
				for (const FAnimNotifyEvent& AnimNotifyEvent : Sampler.GetAllAnimNotifyEvents())
				{
					if (const UAnimNotifyState_PoseSearchSamplingAttribute* SamplingAttribute = Cast<UAnimNotifyState_PoseSearchSamplingAttribute>(AnimNotifyEvent.NotifyStateClass))
					{
						FBoneReference TempBoneReference = SamplingAttribute->Bone;
						TempBoneReference.Initialize(RoledSkeleton.Skeleton);
						if (TempBoneReference.HasValidSetup())
						{
							AddBoneIndex(TempBoneReference.BoneIndex, BoneIndicesWithParents, RoledSkeleton.MirrorDataTable);
						}
					}
				}
			}
		}

		// Sort bone indexes and add eventual missing parent bone indexes
		BoneIndicesWithParents.Sort();
		FAnimationRuntime::EnsureParentsPresent(BoneIndicesWithParents, RoledSkeleton.Skeleton->GetReferenceSkeleton());

		RoledBoneContainer.InitializeTo(BoneIndicesWithParents, CurveFilterSettings, *RoledSkeleton.Skeleton);
		RoledBoneContainer.SetUseRAWData(GVarMotionMatchUseRawAnimationData);
	}
}

static bool IndexDatabase(FSearchIndexBase& SearchIndexBase, const UPoseSearchDatabase* DatabaseToLookForAssets, const UPoseSearchSchema* Schema, const FAssetSamplingContext& SamplingContext, const FFloatInterval& AdditionalExtrapolationTime)
{
	if (!ensure(Schema))
	{
		return false;
	}

	// Prepare samplers for all animation assets.
	TArray<FAnimationAssetSampler> Samplers;
	Samplers.Reserve(256);
	
	TMap<FSamplerMapKey, int32> SamplerMap;
	SamplerMap.Reserve(256);

	for (int32 AssetIdx = 0; AssetIdx != SearchIndexBase.Assets.Num(); ++AssetIdx)
	{
		FSearchIndexAsset& SearchIndexAsset = SearchIndexBase.Assets[AssetIdx];

		const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAssetBase = DatabaseToLookForAssets->GetDatabaseAnimationAsset(SearchIndexAsset.GetSourceAssetIdx());
		check(DatabaseAnimationAssetBase);

		const int32 NumRoles = DatabaseAnimationAssetBase->GetNumRoles();
		for (int32 RoleIndex = 0; RoleIndex < NumRoles; ++RoleIndex)
		{
			const FRole& Role = DatabaseAnimationAssetBase->GetRole(RoleIndex);
			UAnimationAsset* AnimationAsset = DatabaseAnimationAssetBase->GetAnimationAssetForRole(Role);
			const FTransform RootTransformOrigin = DatabaseAnimationAssetBase->GetRootTransformOriginForRole(Role);
			const FVector BlendParameters = SearchIndexAsset.GetBlendParameters();

			const FSamplerMapKey SamplerMapKey(AnimationAsset, RootTransformOrigin, BlendParameters);
			if (!SamplerMap.Contains(SamplerMapKey))
			{
				SamplerMap.Add(SamplerMapKey, Samplers.Num());
				Samplers.Emplace(AnimationAsset, RootTransformOrigin, BlendParameters, FAnimationAssetSampler::DefaultRootTransformSamplingRate, false, !GVarMotionMatchUseRawAnimationData);
			}
		}
	}

	ParallelFor(Samplers.Num(), [&Samplers](int32 SamplerIdx) { Samplers[SamplerIdx].Process(); }, ParallelForFlags);

	// prepare indexers
	TArray<FAssetIndexer> Indexers;
	Indexers.Reserve(SearchIndexBase.Assets.Num());

	TMap<FRole, FBoneContainer> RoledBoneContainers;
	InitRoledBoneContainers(RoledBoneContainers, DatabaseToLookForAssets, Schema);

	TMap<FRole, FMirrorDataCache> RoledMirrorDataCaches;
	RoledMirrorDataCaches.Reserve(RoledBoneContainers.Num());
	for (const TPair<FRole, FBoneContainer>& RoledBoneContainerPair : RoledBoneContainers)
	{
		const FRole& Role = RoledBoneContainerPair.Key;
		RoledMirrorDataCaches.Add(Role).Init(Schema->GetMirrorDataTable(Role), RoledBoneContainers[Role]);
	}

	FAnimationAssetSamplers TempAssetSamplers;
	TArray<FBoneContainer, TInlineAllocator<PreallocatedRolesNum>> TempBoneContainers;
	FRoleToIndex TempRoleToIndex;

	int32 TotalPoses = 0;
	for (int32 AssetIdx = 0; AssetIdx != SearchIndexBase.Assets.Num(); ++AssetIdx)
	{
		FSearchIndexAsset& SearchIndexAsset = SearchIndexBase.Assets[AssetIdx];
		check(SearchIndexAsset.GetFirstPoseIdx() == TotalPoses);

		const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAssetBase = DatabaseToLookForAssets->GetDatabaseAnimationAsset(SearchIndexAsset.GetSourceAssetIdx());
		check(DatabaseAnimationAssetBase);

		TempAssetSamplers.Reset();
		TempBoneContainers.Reset();
		TempRoleToIndex.Reset();

		const int32 NumRoles = DatabaseAnimationAssetBase->GetNumRoles();
		for (int32 RoleIndex = 0; RoleIndex < NumRoles; ++RoleIndex)
		{
			const FRole& Role = DatabaseAnimationAssetBase->GetRole(RoleIndex);
			if (const FMirrorDataCache* MirrorDataCache = RoledMirrorDataCaches.Find(Role))
			{
				UAnimationAsset* AnimationAsset = DatabaseAnimationAssetBase->GetAnimationAssetForRole(Role);
				const FTransform RootTransformOrigin = DatabaseAnimationAssetBase->GetRootTransformOriginForRole(Role);
				const FVector BlendParameters = SearchIndexAsset.GetBlendParameters();

				const FSamplerMapKey SamplerMapKey(AnimationAsset, RootTransformOrigin, BlendParameters);
				const int32* SamplerIndex = SamplerMap.Find(SamplerMapKey);
				check(SamplerIndex && Samplers.IsValidIndex(*SamplerIndex));

				TempAssetSamplers.AnimationAssetSamplers.Emplace(&Samplers[*SamplerIndex]);
				TempAssetSamplers.MirrorDataCaches.Emplace(MirrorDataCache);
				TempBoneContainers.Emplace(RoledBoneContainers[Role]);
				TempRoleToIndex.Add(Role) = RoleIndex;
			}
		}

		const FFloatInterval ExtrapolationTimeInterval = SearchIndexAsset.GetExtrapolationTimeInterval(Schema->SampleRate, AdditionalExtrapolationTime);
		Indexers.Emplace(TempBoneContainers, SearchIndexAsset, SamplingContext, *Schema, TempAssetSamplers, TempRoleToIndex, ExtrapolationTimeInterval);
		TotalPoses += SearchIndexAsset.GetNumPoses();
	}

	// allocating Values and PoseMetadata
	SearchIndexBase.AllocateData(Schema->SchemaCardinality, TotalPoses);

	// assigning local data to each Indexer
	TotalPoses = 0;
	for (int32 AssetIdx = 0; AssetIdx != SearchIndexBase.Assets.Num(); ++AssetIdx)
	{
		Indexers[AssetIdx].AssignWorkingData(TotalPoses, SearchIndexBase.Values, SearchIndexBase.PoseMetadata);
		TotalPoses += Indexers[AssetIdx].GetNumIndexedPoses();
	}

	// Index asset data
	ParallelFor(Indexers.Num(), [&Indexers](int32 AssetIdx) { Indexers[AssetIdx].Process(AssetIdx); }, ParallelForFlags);

	for (const FAssetIndexer& Indexer : Indexers)
	{
		if (Indexer.IsProcessFailed())
		{
			return false;
		}
	}

	// Joining EventData
	FEventDataCollector EventDataCollector;
	for (int32 AssetIdx = 0; AssetIdx != SearchIndexBase.Assets.Num(); ++AssetIdx)
	{
		EventDataCollector.MergeWith(Indexers[AssetIdx].GetEventDataCollector());
	}
	// sorting EventData to make it deterministic across multiple indexing
	SearchIndexBase.EventData.Initialize(EventDataCollector);

	// Joining Metadata.Flags into OverallFlags
	SearchIndexBase.bAnyBlockTransition = false;
	for (const FPoseMetadata& Metadata : SearchIndexBase.PoseMetadata)
	{
		if (Metadata.IsBlockTransition())
		{
			SearchIndexBase.bAnyBlockTransition = true;
			break;
		}
	}

	// Joining Stats
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	int32 NumAccumulatedSamples = 0;
	SearchIndexBase.Stats = FSearchStats();
	for (int32 AssetIdx = 0; AssetIdx != SearchIndexBase.Assets.Num(); ++AssetIdx)
	{
		const FAssetIndexer::FStats& Stats = Indexers[AssetIdx].GetStats();
		SearchIndexBase.Stats.AverageSpeed += Stats.AccumulatedSpeed;
		SearchIndexBase.Stats.MaxSpeed = FMath::Max(SearchIndexBase.Stats.MaxSpeed, Stats.MaxSpeed);
		SearchIndexBase.Stats.AverageAcceleration += Stats.AccumulatedAcceleration;
		SearchIndexBase.Stats.MaxAcceleration = FMath::Max(SearchIndexBase.Stats.MaxAcceleration, Stats.MaxAcceleration);

		NumAccumulatedSamples += Stats.NumAccumulatedSamples;
	}

	if (NumAccumulatedSamples > 0)
	{
		const float Denom = 1.f / float(NumAccumulatedSamples);
		SearchIndexBase.Stats.AverageSpeed *= Denom;
		SearchIndexBase.Stats.AverageAcceleration *= Denom;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Calculate Min Cost Addend
	SearchIndexBase.MinCostAddend = 0.f;
	if (!SearchIndexBase.PoseMetadata.IsEmpty())
	{
		SearchIndexBase.MinCostAddend = MAX_FLT;
		for (const FPoseMetadata& PoseMetadata : SearchIndexBase.PoseMetadata)
		{
			if (PoseMetadata.GetCostAddend() < SearchIndexBase.MinCostAddend)
			{
				SearchIndexBase.MinCostAddend = PoseMetadata.GetCostAddend();
			}
		}
	}

	return true;
}

static FString GetDatabaseAndOuterName(const UPoseSearchDatabase* Database)
{
	if (Database)
	{
		const UObject* Outer = Database->GetOuter();
		if (Outer && !Outer->IsA<UPackage>())
		{
			return Database->GetName() + " [" + Outer->GetName() + "]";
		}

		return Database->GetName();
	}
	
	return TEXT("None");
}

// validating the SearchIndex against the Database to catch any data corruption, that can be caused by not updating or having conflicts over DDC key
static bool ValidateSearchInfoAgainstDatabase(const FSearchIndex& SearchIndex, const UPoseSearchDatabase* Database, const UE::DerivedData::FCacheKey& FullIndexKey)
{
	check(Database && Database->Schema);

	if (SearchIndex.GetNumDimensions() != Database->Schema->SchemaCardinality)
	{
		UE_LOGF(LogPoseSearch, Warning, "%ls - %ls BuildIndex SearchIndex Corrupted! SchemaCardinality mismatch %d vs %d", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(Database), SearchIndex.GetNumDimensions(), Database->Schema->SchemaCardinality);
		return false;
	}

	const int32 NumAnimationAssets = Database->GetNumAnimationAssets();

	for (const FSearchIndexAsset& SearchIndexAsset : SearchIndex.Assets)
	{
		const int32 SourceAssetIdx = SearchIndexAsset.GetSourceAssetIdx();
		if (SourceAssetIdx < 0 || SourceAssetIdx >= NumAnimationAssets)
		{
			UE_LOGF(LogPoseSearch, Warning, "%ls - %ls BuildIndex SearchIndex Corrupted! SearchIndex.Assets referencing missing asset with index %d", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(Database), SourceAssetIdx);
			return false;
		}

		if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAsset = Database->GetDatabaseAnimationAsset(SourceAssetIdx))
		{
			if (const UObject* AnimationAsset = DatabaseAsset->GetAnimationAsset())
			{
				if (!DatabaseAsset->IsEnabled())
				{
					UE_LOGF(LogPoseSearch, Warning, "%ls - %ls BuildIndex SearchIndex Corrupted! Found references to disabled asset %ls in the SearchIndex", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(Database), *AnimationAsset->GetName());
					return false;
				}
			}
		}
		else
		{
			UE_LOGF(LogPoseSearch, Warning, "%ls - %ls BuildIndex SearchIndex Corrupted! null FPoseSearchDatabaseAnimationAssetBase asset at index %d!?", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(Database), SourceAssetIdx);
			return false;
		}
	}

	return true;
}

#if ENABLE_ANIM_DEBUG

static void CompareChannelValues(int32 RecursionIndex, int32 PoseIndex, const TConstArrayView<float>& PoseA, const TConstArrayView<float>& PoseB, const TConstArrayView<TObjectPtr<UPoseSearchFeatureChannel>>& Channels, FStringBuilderBase& StringBuilder)
{
	bool bPrintHeader = true;
	for (const TObjectPtr<UPoseSearchFeatureChannel>& ChannelPtr : Channels)
	{
		const int32 ChannelCardinality = ChannelPtr->GetChannelCardinality();
		const int32 ChannelDataOffset = ChannelPtr->GetChannelDataOffset();

		for (int32 Index = 0; Index < ChannelCardinality; ++Index)
		{
			const int32 DataOffset = ChannelDataOffset + Index;
			const float ValueA = PoseA[DataOffset];
			const float ValueB = PoseB[DataOffset];
			if (ValueA != ValueB)
			{
				if (bPrintHeader && RecursionIndex == 0)
				{
					StringBuilder.Appendf(TEXT("Values mismatch at pose %d\n"), PoseIndex);
					bPrintHeader = false;
				}

				for (int32 Indentation = 0; Indentation < RecursionIndex; ++Indentation)
				{
					StringBuilder.Append(TEXT("    "));
				}

				StringBuilder.Appendf(TEXT("%s - %d (%f, %f)\n"), *ChannelPtr->GetName(), Index, ValueA, ValueB);
			}
		}

		CompareChannelValues(RecursionIndex + 1, PoseIndex, PoseA, PoseB, ChannelPtr->GetSubChannels(), StringBuilder);
	}
}

static void CompareSearchIndexBase(const FSearchIndexBase& A, const FSearchIndexBase& B, const UPoseSearchSchema* Schema, FStringBuilderBase& StringBuilder)
{
	check(Schema);

	if (A.Values.Num() != B.Values.Num())
	{
		StringBuilder.Append(TEXT("Values.Num mismatch\n"));
	}
	else if ((A.Values.Num() % Schema->SchemaCardinality) != 0)
	{
		StringBuilder.Append(TEXT("Values.Num is not a multiple of Schema->SchemaCardinality!\n"));
	}
	else if (Schema->SchemaCardinality > 0)
	{
		// cannot use A.GetNumPoses() since A.Values can be pruned out from duplicates
		int32 ValuesDataOffset = 0;
		const int32 NumValuePoses = A.Values.Num() / Schema->SchemaCardinality;
		for (int32 ValuePoseIndex = 0; ValuePoseIndex < NumValuePoses; ++ValuePoseIndex)
		{
			const TConstArrayView<float> PoseA = MakeArrayView(A.Values).Slice(ValuePoseIndex * Schema->SchemaCardinality, Schema->SchemaCardinality);
			const TConstArrayView<float> PoseB = MakeArrayView(B.Values).Slice(ValuePoseIndex * Schema->SchemaCardinality, Schema->SchemaCardinality);
			CompareChannelValues(0, ValuePoseIndex, PoseA, PoseB, Schema->GetChannels(), StringBuilder);
		}
	}

	if (A.ValuesVectorToPoseIndexes != B.ValuesVectorToPoseIndexes)
	{
		StringBuilder.Append(TEXT("ValuesVectorToPoseIndexes mismatch\n"));
	}

	if (A.PoseMetadata != B.PoseMetadata)
	{
		StringBuilder.Append(TEXT("PoseMetadata mismatch\n"));
	}

	if (A.bAnyBlockTransition != B.bAnyBlockTransition)
	{
		StringBuilder.Append(TEXT("bAnyBlockTransition mismatch\n"));
	}

	if (A.Assets != B.Assets)
	{
		StringBuilder.Append(TEXT("Assets mismatch\n"));
	}
	 
	if (A.MinCostAddend != B.MinCostAddend)
	{
		StringBuilder.Append(TEXT("MinCostAddend mismatch\n"));
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (A.Stats != B.Stats)
	{
		StringBuilder.Append(TEXT("Stats mismatch\n"));
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

static void CompareSearchIndex(const FSearchIndex& A, const FSearchIndex& B, const UPoseSearchSchema* Schema, FStringBuilderBase& StringBuilder)
{
	CompareSearchIndexBase(A, B, Schema, StringBuilder);

	if (A.WeightsSqrt != B.WeightsSqrt)
	{
		StringBuilder.Append(TEXT("WeightsSqrt mismatch\n"));
	}
	
	if (A.PCAValues != B.PCAValues)
	{
		StringBuilder.Append(TEXT("PCAValues mismatch\n"));
	}

	if (A.PCAValuesVectorToPoseIndexes != B.PCAValuesVectorToPoseIndexes)
	{
		StringBuilder.Append(TEXT("PCAValuesVectorToPoseIndexes mismatch\n"));
	}

	if (A.PCAProjectionMatrix != B.PCAProjectionMatrix)
	{
		StringBuilder.Append(TEXT("PCAProjectionMatrix mismatch\n"));
	}

	if (A.Mean != B.Mean)
	{
		StringBuilder.Append(TEXT("Mean mismatch\n"));
	}

	if (A.KDTree != B.KDTree)
	{
		StringBuilder.Append(TEXT("KDTree mismatch\n"));
	}

	if (A.VPTree != B.VPTree)
	{
		StringBuilder.Append(TEXT("VPTree mismatch\n"));
	}

#if WITH_EDITORONLY_DATA
	if (A.DeviationEditorOnly != B.DeviationEditorOnly)
	{
		StringBuilder.Append(TEXT("DeviationEditorOnly mismatch\n"));
	}

	if (A.PCAExplainedVarianceEditorOnly != B.PCAExplainedVarianceEditorOnly)
	{
		StringBuilder.Append(TEXT("PCAExplainedVarianceEditorOnly mismatch\n"));
	}
#endif // WITH_EDITORONLY_DATA
}

#endif // ENABLE_ANIM_DEBUG

static bool IndexDatabase(const UPoseSearchDatabase* DependentDatabase, const UPoseSearchSchema* DependentDatabaseSchema, FSearchIndexBase& SearchIndexBase, 
	const UPoseSearchDatabase* MainDatabase, const UE::DerivedData::FCacheKey& FullIndexKey)
{
	check(DependentDatabase);

	const FString DependentDatabaseName = GetDatabaseAndOuterName(DependentDatabase);

	const FAssetSamplingContext DependentSamplingContext(*DependentDatabase);
	const FFloatInterval& DependentExcludeFromDatabaseParameters = DependentDatabase->ExcludeFromDatabaseParameters;
	const FFloatInterval& DependentAdditionalExtrapolationTime = DependentDatabase->AdditionalExtrapolationTime;

	// early out for invalid indexing conditions
	if (!DependentDatabaseSchema || DependentDatabaseSchema->SchemaCardinality <= 0)
	{
		if (DependentDatabase == MainDatabase)
		{
			UE_LOGF(LogPoseSearch, Error, "%ls - %ls BuildIndex Failed because of invalid Schema", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase));
		}
		else
		{
			UE_LOGF(LogPoseSearch, Error, "%ls - %ls BuildIndex Failed because dependent database '%ls' has an invalid Schema", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase), *DependentDatabaseName);
		}
		return false;
	}

	// validating that the missing MirrorDataTable(s) are not necessary 
	bool bAllRoledSkeletonHaveMirrorDataTable = true;
	for (const FPoseSearchRoledSkeleton& RoledSkeleton : DependentDatabaseSchema->GetRoledSkeletons())
	{
		if (RoledSkeleton.MirrorDataTable)
		{
			if (!RoledSkeleton.MirrorDataTable->Skeleton)
			{
				UE_LOGF(LogPoseSearch, Error, "%ls - %ls BuildIndex Failed because '%ls' schema MirrorDataTable Skeleton is not set for Role '%ls' ", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase), *DependentDatabaseName, *RoledSkeleton.Role.ToString());
				return false;
			}
		}
		else
		{
			bAllRoledSkeletonHaveMirrorDataTable = false;
		}
	}

	if (!bAllRoledSkeletonHaveMirrorDataTable)
	{
		for (int32 AnimationAssetIndex = 0; AnimationAssetIndex < DependentDatabase->GetNumAnimationAssets(); ++AnimationAssetIndex)
		{
			if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAsset = DependentDatabase->GetDatabaseAnimationAsset(AnimationAssetIndex))
			{
				if (DatabaseAsset->GetMirrorOption() == EPoseSearchMirrorOption::MirroredOnly || DatabaseAsset->GetMirrorOption() == EPoseSearchMirrorOption::UnmirroredAndMirrored)
				{
					// want to sample a mirrored asset
					UE_LOGF(LogPoseSearch, Error, "%ls - %ls BuildIndex Failed because '%ls' schema requires MirrorDataTable(s) to sample mirrored animation assets", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase), *DependentDatabaseName);
					return false;
				}
			}
		}
	}

	for (int32 AnimationAssetIndex = 0; AnimationAssetIndex < DependentDatabase->GetNumAnimationAssets(); ++AnimationAssetIndex)
	{
		if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAsset = DependentDatabase->GetDatabaseAnimationAsset(AnimationAssetIndex))
		{
			const UAnimationAsset* AnimationAsset = Cast<const UAnimationAsset>(DatabaseAsset->GetAnimationAsset());
			if (DatabaseAsset->GetAnimationAsset() == nullptr)
			{
				UE_LOGF(LogPoseSearch, Warning, "OnGetComplete - Database %ls entry %d has no associated animation asset.", *DatabaseAsset->GetName(), AnimationAssetIndex);
			}
			else if (!DatabaseAsset->GetAnimationAsset()->GetPackage() || DatabaseAsset->GetAnimationAsset()->HasAnyFlags(RF_MirroredGarbage))
			{
				UE_LOGF(LogPoseSearch, Error, "%ls - %ls BuildIndex Failed because '%ls' referenced asset '%ls' is garbage or is not contained in a package", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase), *DependentDatabaseName, *DatabaseAsset->GetAnimationAsset()->GetName());
				return false;
			}
			else if (AnimationAsset && AnimationAsset->GetSkeleton() == nullptr)
			{
				UE_LOGF(LogPoseSearch, Error, "%ls - %ls BuildIndex Failed because '%ls' referenced asset '%ls' which has an invalid (null) Skeleton", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase), *DependentDatabaseName, *DatabaseAsset->GetAnimationAsset()->GetName());
				return false;
			}
			else if (!IsSchemaCompatible(DatabaseAsset, MainDatabase->Schema))
			{
				UE_LOGF(LogPoseSearch, Warning, "OnGetComplete - %ls is not compatible with the schema %ls", *DatabaseAsset->GetName(), *GetNameSafe(MainDatabase->Schema));
			}
			else if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(DatabaseAsset->GetAnimationAsset()))
			{
				if (!BlendSpace->bShouldMatchSyncPhases)
				{
					UE_LOGF(LogPoseSearch, Warning, "OnGetComplete - %ls's bShouldMatchSyncPhases flag is not enabled. This is required for properly pose matching blendspaces.", *DatabaseAsset->GetName());
				}
				else
				{
					const TConstArrayView<FBlendSample> BlendSamples = BlendSpace->GetBlendSamples();
					for (int i = 0; i < BlendSamples.Num() - 1; ++i)
					{
						const FBlendSample& CurrSample = BlendSamples[i];
						const FBlendSample& NextSample = BlendSamples[i + 1];

						if (CurrSample.Animation && NextSample.Animation)
						{
							bool bWarning = false;

							if (CurrSample.Animation->AuthoredSyncMarkers.Num() == NextSample.Animation->AuthoredSyncMarkers.Num())
							{
								for (int j = 0; j < CurrSample.Animation->AuthoredSyncMarkers.Num(); ++j)
								{
									if (CurrSample.Animation->AuthoredSyncMarkers[j].MarkerName != NextSample.Animation->AuthoredSyncMarkers[j].MarkerName)
									{
										bWarning = true;
										break;
									}
								}
							}
							else
							{
								bWarning = true;
							}

							if (bWarning)
							{
								UE_LOGF(LogPoseSearch, Warning, "OnGetComplete - %ls's samples don't share the same layout of sync markers. This is required for properly pose matching blendspaces.", *DatabaseAsset->GetName());
							}
						}
					}
				}
			}
		}
	}

	// Building all the related FPoseSearchBaseIndex first
	if (!InitSearchIndexAssets(SearchIndexBase, DependentDatabase, DependentDatabaseSchema, DependentExcludeFromDatabaseParameters))
	{
		UE_LOGF(LogPoseSearch, Error, "%ls - %ls BuildIndex Failed because of invalid assets", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase));
		return false;
	}

	if (!IndexDatabase(SearchIndexBase, DependentDatabase, DependentDatabaseSchema, DependentSamplingContext, DependentAdditionalExtrapolationTime))
	{
		UE_LOGF(LogPoseSearch, Log, "%ls - %ls BuildIndex Failed in IndexDatabase", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase));
		return false;
	}

#if ENABLE_ANIM_DEBUG
	if (GVarMotionMatchTestDeterminismIndexDatabase)
	{
		for (int32 Iteration = 0; Iteration < GVarMotionMatchTestNumIterations; ++Iteration)
		{
			FSearchIndexBase TestSearchIndexBase = SearchIndexBase;
			if (IndexDatabase(TestSearchIndexBase, DependentDatabase, DependentDatabaseSchema, DependentSamplingContext, DependentAdditionalExtrapolationTime))
			{
				if (TestSearchIndexBase != SearchIndexBase)
				{
					TStringBuilder<0> Message;
					CompareSearchIndexBase(TestSearchIndexBase, SearchIndexBase, DependentDatabaseSchema, Message);
					UE_LOGF(LogPoseSearch, Warning, "OnGetComplete - IndexDatabase is not deterministic\n%ls", *Message);
				}
			}
		}
	}
#endif //ENABLE_ANIM_DEBUG

	return true;
}

//////////////////////////////////////////////////////////////////////////
// FPoseSearchDatabaseAsyncCacheTask
struct FPoseSearchDatabaseAsyncCacheTask
{
	enum class EState
	{
		Notstarted,	// key generation failed (not all the asset has been post loaded). It'll be retried to StartNewRequestIfNeeded the next Update
		Prestarted,	// key has been successfully generated and we kicked the DDC get
		PreCancelled,	// the task has been requested to be cancelled
		Cancelled,		// the task cancellation has been finalized
		Ended,		// the task has ended successfully
		Failed		// the task has ended unsuccessfully
	};

	FPoseSearchDatabaseAsyncCacheTask(UPoseSearchDatabase* InDatabase, bool bPerformConditionalPostLoadIfRequired, FPartialKeyHashes& PartialKeyHashes);
	void ClearAnimSequenceResidency();
	void StartNewRequestIfNeeded(bool bPerformConditionalPostLoadIfRequired, FPartialKeyHashes& PartialKeyHashes);
	void Update(FCriticalSection& OuterMutex, FPartialKeyHashes& PartialKeyHashes);
	void Wait(FCriticalSection& OuterMutex);
	void PreCancelIfDependsOn(const UObject* Object);
	void Cancel();
	bool Poll() const;
	void AddReferencedObjects(FReferenceCollector& Collector);
	bool IsValid() const;
	const FIoHash& GetDerivedDataKey() const { return DerivedDataKey; }
	const UPoseSearchDatabase* GetDatabase() const { return Database.Get(); }

	~FPoseSearchDatabaseAsyncCacheTask();
	EState GetState() const { return EState(ThreadSafeState.GetValue()); }

#if ENABLE_ANIM_DEBUG
	void TestSynchronizeWithExternalDependencies();
#endif //ENABLE_ANIM_DEBUG

private:
	FPoseSearchDatabaseAsyncCacheTask(const FPoseSearchDatabaseAsyncCacheTask& Other) = delete;
	FPoseSearchDatabaseAsyncCacheTask(FPoseSearchDatabaseAsyncCacheTask&& Other) = delete;
	FPoseSearchDatabaseAsyncCacheTask& operator=(const FPoseSearchDatabaseAsyncCacheTask& Other) = delete;
	FPoseSearchDatabaseAsyncCacheTask& operator=(FPoseSearchDatabaseAsyncCacheTask&& Other) = delete;

	void OnGetComplete(UE::DerivedData::FCacheGetResponse&& Response);
	void SetState(EState State) { ThreadSafeState.Set(int32(State)); }
	void ResetSearchIndex();

	TWeakObjectPtr<UPoseSearchDatabase> Database;
	FSearchIndex SearchIndex;

#if ENABLE_ANIM_DEBUG
	FSearchIndex SearchIndexCompare;
#endif //ENABLE_ANIM_DEBUG

	UE::DerivedData::FRequestOwner Owner;
	FIoHash DerivedDataKey = FIoHash::Zero;
	TSet<TWeakObjectPtr<const UObject>> DatabaseDependencies;
		
	FThreadSafeCounter ThreadSafeState = int32(EState::Notstarted);
	bool bBroadcastOnDerivedDataRebuild = false;
	bool bResidencyCleared = true;
};

class FPoseSearchDatabaseAsyncCacheTasks : public TArray<TUniquePtr<FPoseSearchDatabaseAsyncCacheTask>, TInlineAllocator<64>> {};

FPoseSearchDatabaseAsyncCacheTask::FPoseSearchDatabaseAsyncCacheTask(UPoseSearchDatabase* InDatabase, bool bPerformConditionalPostLoadIfRequired, FPartialKeyHashes& PartialKeyHashes)
	: Database(InDatabase)
	, Owner(UE::DerivedData::EPriority::Normal)
	, DerivedDataKey(FIoHash::Zero)
{
	if (IsInGameThread())
	{
		// it is safe to compose DDC key only on the game thread, since assets can modified in this thread execution
		StartNewRequestIfNeeded(bPerformConditionalPostLoadIfRequired, PartialKeyHashes);
	}
	else
	{
		UE_LOGF(LogPoseSearch, Log, "Delaying DDC until on the game thread    - %ls", *Database->GetName());
	}
}

void FPoseSearchDatabaseAsyncCacheTask::ClearAnimSequenceResidency()
{
	if (bResidencyCleared)
	{
		return;
	}
	
	const ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->GetRunningTargetPlatform();
	const uint32 DatabaseHash = GetTypeHash(Database.Get());
	for (TWeakObjectPtr<const UObject>& Dependency : DatabaseDependencies)
	{
		if (Dependency.IsValid())
		{
			if (UAnimSequence* AnimSequence = Cast<UAnimSequence>(const_cast<UObject*>(Dependency.Get())))
			{
				if (AnimSequence->HasResidency(DatabaseHash))
				{
					AnimSequence->ReleaseResidency(TargetPlatform, DatabaseHash);
				}
			}
		}
	}

	bResidencyCleared = true;
}

FPoseSearchDatabaseAsyncCacheTask::~FPoseSearchDatabaseAsyncCacheTask()
{
	// Owner.Cancel must be performed before SearchIndex.Reset() in case any task is flying (launched by Owner.LaunchTask)
	Owner.Cancel();

	Database = nullptr;

	ResetSearchIndex();

	DerivedDataKey = FIoHash::Zero;
	DatabaseDependencies.Reset();
}

void FPoseSearchDatabaseAsyncCacheTask::ResetSearchIndex()
{
	SearchIndex.Reset();
#if ENABLE_ANIM_DEBUG
	SearchIndexCompare.Reset();
#endif //ENABLE_ANIM_DEBUG
}

#if ENABLE_ANIM_DEBUG
void FPoseSearchDatabaseAsyncCacheTask::TestSynchronizeWithExternalDependencies()
{
	if (GetState() == EState::Ended && Database != nullptr)
	{
		Database->TestSynchronizeWithExternalDependencies();
	}
}
#endif //ENABLE_ANIM_DEBUG

void FPoseSearchDatabaseAsyncCacheTask::StartNewRequestIfNeeded(bool bPerformConditionalPostLoadIfRequired, FPartialKeyHashes& PartialKeyHashes)
{
	using namespace UE::DerivedData;

	check(IsInGameThread());

	// making sure there are no active requests
	// Owner.Cancel must be performed before SearchIndex.Reset() in case any task is flying (launched by Owner.LaunchTask)
	Owner.Cancel();

	// composing the key
	FKeyBuilder KeyBuilder(Database.Get(), true, bPerformConditionalPostLoadIfRequired, &PartialKeyHashes, !GVarMotionMatchUseRawAnimationData);
	if (KeyBuilder.AnyAssetNotFullyLoaded())
	{
		DerivedDataKey = FIoHash::Zero;
		SetState(EState::Notstarted);
	
		UE_LOGF(LogPoseSearch, Log, "Delaying DDC until dependents are fully loaded - %ls", *Database->GetName());
	}
	else
	{
		const FIoHash NewDerivedDataKey(KeyBuilder.Finalize());
		const bool bHasKeyChanged = NewDerivedDataKey != DerivedDataKey;
		if (bHasKeyChanged)
		{
			if (!KeyBuilder.AnyAssetNotReady())
			{
				DerivedDataKey = NewDerivedDataKey;

				DatabaseDependencies.Reset();
				DatabaseDependencies.Reserve(KeyBuilder.GetDependencies().Num());
				for (const UObject* Dependency : KeyBuilder.GetDependencies())
				{
					DatabaseDependencies.Add(Dependency);
				}

				SetState(EState::Prestarted);
				
				UE_LOGF(LogPoseSearch, Log, "%ls - %ls BeginCache", *LexToString(DerivedDataKey), *GetDatabaseAndOuterName(Database.Get()));

				const FCacheKey CacheKey{ Bucket, DerivedDataKey };
				const FCacheGetRequest CacheRequest = { { Database->GetPathName() }, CacheKey, ECachePolicy::Default };

				Owner = FRequestOwner(EPriority::Normal);
				GetCache().Get(MakeArrayView(&CacheRequest, 1), Owner, [this](FCacheGetResponse&& Response)
					{
						OnGetComplete(MoveTemp(Response));
					});
			}
			else
			{
				DerivedDataKey = FIoHash::Zero;
				SetState(EState::Notstarted);
				UE_LOGF(LogPoseSearch, Log, "Delaying DDC until dependents are ready - %ls", *Database->GetName());
			}
			
			bResidencyCleared = false;
		}
	}
}

// it cancels and waits for the task to be done and set the state to PreCancelled, so no other new requests can start until the task gets cancelled
void FPoseSearchDatabaseAsyncCacheTask::PreCancelIfDependsOn(const UObject* Object)
{
	ensure(IsInGameThread());

	// DatabaseDependencies is updated only in StartNewRequestIfNeeded when there are no active requests, so it's thread safe to access it 
	if (DatabaseDependencies.Contains(Object))
	{
		// Database can be null if the task was Ended/Failed and Database was already garbage collected, but Tick hasn't been called yet
		FString DatabaseName = IsValid() ? *GetDatabaseAndOuterName(Database.Get()) : TEXT("Garbage Collected Database");
		UE_LOGF(LogPoseSearch, Log, "%ls - %ls PreCancelled because of %ls", *LexToString(DerivedDataKey), *DatabaseName, *Object->GetName());

		// Owner.Cancel must be performed before SearchIndex.Reset() in case any task is flying (launched by Owner.LaunchTask)
		Owner.Cancel();

		SetState(EState::PreCancelled);
	}
}

// it cancels and waits for the task to be done and reset the local SearchIndex. SetState to Cancelled
void FPoseSearchDatabaseAsyncCacheTask::Cancel()
{
	check(IsInGameThread());

	FString DatabaseName = IsValid() ? *GetDatabaseAndOuterName(Database.Get()) : TEXT("Garbage Collected Database");
	UE_LOGF(LogPoseSearch, Log, "%ls - %ls Cancelled", *LexToString(DerivedDataKey), *DatabaseName);

	// Owner.Cancel must be performed before SearchIndex.Reset() in case any task is flying (launched by Owner.LaunchTask)
	Owner.Cancel();

	ResetSearchIndex();

	DerivedDataKey = FIoHash::Zero;
	SetState(EState::Cancelled);
	ClearAnimSequenceResidency();
}

void FPoseSearchDatabaseAsyncCacheTask::Update(FCriticalSection& OuterMutex, FPartialKeyHashes& PartialKeyHashes)
{
	check(IsInGameThread());

	check(GetState() != EState::Cancelled); // otherwise FPoseSearchDatabaseAsyncCacheTask should have been already removed

	if (GetState() == EState::Notstarted)
	{
		StartNewRequestIfNeeded(false, PartialKeyHashes);
	}

	if (GetState() == EState::Prestarted && Poll())
	{
		// task is done: we need to update the state form Prestarted to Ended/Failed
		Wait(OuterMutex);
	}

	if (GetState() != EState::PreCancelled)
	{
		if (bBroadcastOnDerivedDataRebuild)
		{
			Database->NotifyDerivedDataRebuild();
			bBroadcastOnDerivedDataRebuild = false;
		}
	}
}

// it waits for the task to be done and SetSearchIndex on the database. SetState to Ended/Failed
void FPoseSearchDatabaseAsyncCacheTask::Wait(FCriticalSection& OuterMutex)
{
	check(GetState() == EState::Prestarted);

	FScopeLock Lock(&OuterMutex);
	Owner.Wait();

	const bool bFailedIndexing = SearchIndex.IsEmpty();
	if (!bFailedIndexing)
	{
		Database->SetSearchIndex(SearchIndex);

		check(Database->Schema && !SearchIndex.IsEmpty() && SearchIndex.GetNumDimensions() == Database->Schema->SchemaCardinality);

		SetState(EState::Ended);
		ClearAnimSequenceResidency();
		bBroadcastOnDerivedDataRebuild = true;
	}
	else
	{
		check(!bBroadcastOnDerivedDataRebuild);
		ClearAnimSequenceResidency();
		SetState(EState::Failed);
	}
	ResetSearchIndex();
}

// true is the task is done executing
bool FPoseSearchDatabaseAsyncCacheTask::Poll() const
{
	return Owner.Poll();
}

bool FPoseSearchDatabaseAsyncCacheTask::IsValid() const
{
	return Database.IsValid();
}

// called once the task is done:
// if EStatus::Ok (data has been retrieved from DDC) we deserialize the payload into the local SearchIndex
// if EStatus::Error we BuildIndex and if that's successful we 'Put' it on DDC
void FPoseSearchDatabaseAsyncCacheTask::OnGetComplete(UE::DerivedData::FCacheGetResponse&& Response)
{
	using namespace UE::DerivedData;

	const FCacheKey& FullIndexKey = Response.Record.GetKey();

	// The database is part of the derived data cache and up to date, skip re-building it.
	bool bCacheCorrupted = false;
	if (Response.Status == EStatus::Ok)
	{
		COOK_STAT(auto Timer = UsageStats.TimeAsyncWait());

		// we found the cached data associated to the PendingDerivedDataKey: we'll deserialized into SearchIndex
		ResetSearchIndex();

		FSharedBuffer RawData = Response.Record.GetValue(Id).GetData().Decompress();
		FMemoryReaderView Reader(RawData);
		Reader << SearchIndex;

		check(Database != nullptr && Database->Schema);
		// cache can be corrupted in case the version of the derived data cache has not being updated while 
		// developing channels that changes their cardinality without impacting any asset properties
		// so to account for this, we just reindex the database and update the associated DDC 
		if (ValidateSearchInfoAgainstDatabase(SearchIndex, Database.Get(), FullIndexKey))
		{
			UE_LOGF(LogPoseSearch, Log, "%ls - %ls BuildIndex From Cache", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(Database.Get()));
		}
		else
		{
			bCacheCorrupted = true;
		}

		COOK_STAT(Timer.AddHit(RawData.GetSize()));
	}
	
	if (Response.Status == EStatus::Canceled)
	{
		ResetSearchIndex();
		UE_LOGF(LogPoseSearch, Log, "%ls - %ls BuildIndex Cancelled", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(Database.Get()));
	}
	
	bool bForceBuildIndex = false;
#if ENABLE_ANIM_DEBUG
	bForceBuildIndex = GVarMotionMatchTestForceIndexing;
#endif // ENABLE_ANIM_DEBUG

	if (Response.Status == EStatus::Error || bCacheCorrupted || bForceBuildIndex)
	{
		bool bCompareSearchIndex = false;
#if ENABLE_ANIM_DEBUG
		bCompareSearchIndex = Response.Status != EStatus::Error && !bCacheCorrupted && bForceBuildIndex;
		if (bCompareSearchIndex)
		{
			SearchIndexCompare = SearchIndex;
		}
#endif // ENABLE_ANIM_DEBUG

		// we didn't find the cached data associated to the PendingDerivedDataKey: we'll BuildIndex to update SearchIndex and "Put" the data over the DDC
		Owner.LaunchTask(TEXT("PoseSearchDatabaseBuild"), [this, FullIndexKey, bCompareSearchIndex]
			{
				COOK_STAT(auto Timer = UsageStats.TimeSyncWork());

				const UPoseSearchDatabase* MainDatabase = Database.Get();

				// collecting all the databases that need to be built to gather their FSearchIndexBase
				// the first one is always the main database (the one we're calculating the index on)
				TArray<const UPoseSearchDatabase*> IndexBaseDatabases;
				IndexBaseDatabases.Reserve(64);
				IndexBaseDatabases.Add(MainDatabase);
				if (!MainDatabase)
				{
					UE_LOGF(LogPoseSearch, Log, "%ls - BuildIndex Cancelled because associated Database weak pointer has been released.", *LexToString(FullIndexKey.Hash));
					ResetSearchIndex();
					return;
				}

				if (MainDatabase->NormalizationSet)
				{
					MainDatabase->NormalizationSet->AddUniqueDatabases(IndexBaseDatabases);
				}

				const UPoseSearchSchema* MainDatabaseSchema = MainDatabase->Schema;
				if (!MainDatabaseSchema || MainDatabaseSchema->SchemaCardinality <= 0)
				{
					UE_LOGF(LogPoseSearch, Error, "%ls - %ls BuildIndex Failed because of invalid Schema", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase));
					ResetSearchIndex();
					return;
				}

				const bool bNormalizeWithCommonSchema = MainDatabaseSchema->DataPreprocessor == EPoseSearchDataPreprocessor::NormalizeWithCommonSchema;

				// @todo: DDC or parallelize this code
				TArray<FSearchIndexBase> SearchIndexBases;
				TArray<const UPoseSearchSchema*> Schemas;
				SearchIndexBases.AddDefaulted(IndexBaseDatabases.Num());
				Schemas.AddDefaulted(IndexBaseDatabases.Num());
				for (int32 IndexBaseIdx = 0; IndexBaseIdx < IndexBaseDatabases.Num(); ++IndexBaseIdx)
				{
					const UPoseSearchDatabase* DependentDatabase = IndexBaseDatabases[IndexBaseIdx];
					check(DependentDatabase);

					Schemas[IndexBaseIdx] = bNormalizeWithCommonSchema ? MainDatabaseSchema : DependentDatabase->Schema.Get();

					if (!IndexDatabase(DependentDatabase, Schemas[IndexBaseIdx], SearchIndexBases[IndexBaseIdx], MainDatabase, FullIndexKey))
					{
						ResetSearchIndex();
						return;
					}
				}

				static_cast<FSearchIndexBase&>(SearchIndex) = SearchIndexBases[0];
				
#if ENABLE_ANIM_DEBUG
				// testing PruneDuplicateValues determinism
				if (GVarMotionMatchTestDeterminismPruneDuplicateValues)
				{
					const int32 NumIterations = GVarMotionMatchTestNumIterations;

					FSearchIndex TestSearchIndexA = SearchIndex;
					TestSearchIndexA.PruneDuplicateValues(MainDatabase->PosePruningSimilarityThreshold, MainDatabaseSchema->SchemaCardinality, false);
					for (int32 Iteration = 0; Iteration < NumIterations; ++Iteration)
					{
						FSearchIndex TestSearchIndexB = SearchIndex;
						TestSearchIndexB.PruneDuplicateValues(MainDatabase->PosePruningSimilarityThreshold, MainDatabaseSchema->SchemaCardinality, false);

						if (TestSearchIndexA.Values != TestSearchIndexB.Values)
						{
							UE_LOGF(LogPoseSearch, Warning, "OnGetComplete - PruneDuplicateValues is not deterministic");
						}

						if (TestSearchIndexA.ValuesVectorToPoseIndexes != TestSearchIndexB.ValuesVectorToPoseIndexes)
						{
							UE_LOGF(LogPoseSearch, Warning, "OnGetComplete - PruneDuplicateValues ValuesVectorToPoseIndexes generation is not deterministic");
						}
					}
				}
#endif // ENABLE_ANIM_DEBUG

				// VPTree requires ValuesVectorToPoseIndexes if there's any Values pruning
				const bool bDoNotGenerateValuesVectorToPoseIndexes = MainDatabase->PoseSearchMode != EPoseSearchMode::VPTree;
				SearchIndex.PruneDuplicateValues(MainDatabase->PosePruningSimilarityThreshold, MainDatabaseSchema->SchemaCardinality, bDoNotGenerateValuesVectorToPoseIndexes);

				const TArray<float> Deviation = FMeanDeviationCalculator::Calculate(SearchIndexBases, Schemas);

				// Building FSearchIndex
				if (!PreprocessSearchIndexWeights(SearchIndex, MainDatabaseSchema, Deviation))
				{
					UE_LOGF(LogPoseSearch, Error, "%ls - %ls BuildIndex Failed while preprocessing weights for schema '%ls'", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase), *MainDatabaseSchema->GetName());
					ResetSearchIndex();
					return;
				}

				if (Owner.IsCanceled())
				{
					UE_LOGF(LogPoseSearch, Log, "%ls - %ls BuildIndex Cancelled", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase));
					ResetSearchIndex();
					return;
				}

				const Eigen::ComputationInfo ComputationInfo = PreprocessSearchIndexPCAData(SearchIndex, MainDatabaseSchema->SchemaCardinality, MainDatabase->GetNumberOfPrincipalComponents(), MainDatabase->PoseSearchMode);
				if (ComputationInfo != Eigen::Success)
				{
					FString Reason;
					switch (ComputationInfo)
					{
					case Eigen::NumericalIssue: Reason = "Numerical Issues"; break;
					case Eigen::NoConvergence:	Reason = "No Convergence";   break;
					case Eigen::InvalidInput:	Reason = "Invalid Input"; 	 break;
					default:					Reason = "Unknown Reasons";  break;
					}
					UE_LOGF(LogPoseSearch, Error, "%ls - %ls BuildIndex Failed because of '%ls' while calculating PCA data. Try with a different dataset or change the database 'Pose Search Mode'", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase), *Reason);

					ResetSearchIndex();
					return;
				}

				if (Owner.IsCanceled())
				{
					UE_LOGF(LogPoseSearch, Log, "%ls - %ls BuildIndex Cancelled", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase));
					ResetSearchIndex();
					return;
				}

#if ENABLE_ANIM_DEBUG
				// testing PruneDuplicatePCAValues determinism
				if (GVarMotionMatchTestDeterminismPruneDuplicatePCAValues)
				{
					FSearchIndex TestSearchIndexA = SearchIndex;
					TestSearchIndexA.PruneDuplicatePCAValues(MainDatabase->PCAValuesPruningSimilarityThreshold, MainDatabase->GetNumberOfPrincipalComponents());
					for (int32 Iteration = 0; Iteration < GVarMotionMatchTestNumIterations; ++Iteration)
					{
						FSearchIndex TestSearchIndexB = SearchIndex;
						TestSearchIndexB.PruneDuplicatePCAValues(MainDatabase->PCAValuesPruningSimilarityThreshold, MainDatabase->GetNumberOfPrincipalComponents());

						if (TestSearchIndexA.PCAValues != TestSearchIndexB.PCAValues)
						{
							UE_LOGF(LogPoseSearch, Warning, "OnGetComplete - PruneDuplicatePCAValues is not deterministic");
						}

						if (TestSearchIndexA.PCAValuesVectorToPoseIndexes != TestSearchIndexB.PCAValuesVectorToPoseIndexes)
						{
							UE_LOGF(LogPoseSearch, Warning, "OnGetComplete - PruneDuplicatePCAValues PCAValuesVectorToPoseIndexes generation is not deterministic");
						}
					}
				}
#endif // ENABLE_ANIM_DEBUG

				SearchIndex.PruneDuplicatePCAValues(MainDatabase->PCAValuesPruningSimilarityThreshold, MainDatabase->GetNumberOfPrincipalComponents());
				if (Owner.IsCanceled())
				{
					UE_LOGF(LogPoseSearch, Log, "%ls - %ls BuildIndex Cancelled", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase));
					ResetSearchIndex();
					return;
				}

				// removing SearchIndex.Values and relying on FSearchIndex::GetReconstructedPoseValues to reconstruct the Values data from the PCAValues
				const bool bRemoveSearchIndexValues = MainDatabase->PoseSearchMode == EPoseSearchMode::PCAKDTree && MainDatabase->KDTreeQueryNumNeighbors <= 1;
				if (!bRemoveSearchIndexValues)
				{
					// we cannot prune any PCAValues if we need to reconstruct the Values from them!
					SearchIndex.PrunePCAValuesFromBlockTransitionPoses(MainDatabase->GetNumberOfPrincipalComponents());
				}

				PreprocessSearchIndexKDTree(SearchIndex, MainDatabase);
				if (Owner.IsCanceled())
				{
					UE_LOGF(LogPoseSearch, Log, "%ls - %ls BuildIndex Cancelled", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase));
					ResetSearchIndex();
					return;
				}

				if (bRemoveSearchIndexValues)
				{
					SearchIndex.ResetValues();
				}

				const int32 RandomSeed = GetTypeHash(FullIndexKey.Hash);
				PreprocessSearchIndexVPTree(SearchIndex, MainDatabase, RandomSeed);
				if (Owner.IsCanceled())
				{
					UE_LOGF(LogPoseSearch, Log, "%ls - %ls BuildIndex Cancelled", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase));
					ResetSearchIndex();
					return;
				}

				UE_LOGF(LogPoseSearch, Log, "%ls - %ls BuildIndex Succeeded", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase));

#if ENABLE_ANIM_DEBUG
				if (bCompareSearchIndex && SearchIndexCompare != SearchIndex)
				{
					TStringBuilder<0> Message;
					CompareSearchIndex(SearchIndexCompare, SearchIndex, MainDatabaseSchema, Message);
					UE_LOGF(LogPoseSearch, Warning, "%ls - %ls BuildIndex mismatch with DDC Index\n%ls", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase), *Message);
				}

				if (!ValidateSearchInfoAgainstDatabase(SearchIndex, Database.Get(), FullIndexKey))
				{
					UE_LOGF(LogPoseSearch, Error, "%ls - %ls BuildIndex Failed because of corrupted SearchIndex!", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(Database.Get()));
					ResetSearchIndex();
					return;
				}
#endif // ENABLE_ANIM_DEBUG

				// putting SearchIndex to DDC
				TArray<uint8> RawBytes;
				// reserving 20k as initial buffer to serialize the SearchIndex to avoid multiple reallocations
				RawBytes.Reserve(20 * 1024);
				FMemoryWriter Writer(RawBytes);
				Writer << SearchIndex;
				FSharedBuffer RawData = MakeSharedBufferFromArray(MoveTemp(RawBytes));
				const int32 BytesProcessed = RawData.GetSize();

				FCacheRecordBuilder Builder(FullIndexKey);
				Builder.AddValue(Id, RawData);

				GetCache().Put({ { { MainDatabase->GetPathName() }, Builder.Build() } }, Owner, [this, MainDatabaseSchema, MainDatabase, FullIndexKey](FCachePutResponse&& Response)
					{
						switch (Response.Status)
						{
						case EStatus::Error:
							UE_LOGF(LogPoseSearch, Log, "%ls - %ls Failed to store DDC", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase));
							break;
						case EStatus::Canceled:
							UE_LOGF(LogPoseSearch, Log, "%ls - %ls Canceled to store DDC", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase));
							break;
						case EStatus::Ok:
							UE_LOGF(LogPoseSearch, Log, "%ls - %ls BuildIndex stored to DDC", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase));

#if ENABLE_ANIM_DEBUG
							if (GVarMotionMatchTestDDCConsistency)
							{
								const FCacheKey CacheKey{ Bucket, DerivedDataKey };
								const FCacheGetRequest CacheRequest = { { Database->GetPathName() }, CacheKey, ECachePolicy::Default };
								GetCache().Get(MakeArrayView(&CacheRequest, 1), Owner, [this, MainDatabaseSchema, MainDatabase](FCacheGetResponse&& Response)
									{
										const FCacheKey& FullIndexKey = Response.Record.GetKey();
										check(FullIndexKey.Hash == DerivedDataKey);

										if (Response.Status == EStatus::Ok)
										{
											FSharedBuffer RawData = Response.Record.GetValue(Id).GetData().Decompress();
											FMemoryReaderView Reader(RawData);

											FSearchIndex TestSearchIndex;
											Reader << TestSearchIndex;

											if (TestSearchIndex != SearchIndex)
											{
												TStringBuilder<0> Message;
												CompareSearchIndex(TestSearchIndex, SearchIndex, MainDatabaseSchema, Message);
												UE_LOGF(LogPoseSearch, Warning, "%ls - %ls DDC Index mismatch with BuildIndex\n%ls", *LexToString(FullIndexKey.Hash), *GetDatabaseAndOuterName(MainDatabase), *Message);
											}
										}
									});
							}
#endif // ENABLE_ANIM_DEBUG
						}
					});

				COOK_STAT(Timer.AddMiss(BytesProcessed));
			});
	}
}

void FPoseSearchDatabaseAsyncCacheTask::AddReferencedObjects(FReferenceCollector& Collector)
{
	const EState State = GetState();
	if (State != EState::Ended && State != EState::Failed)
	{
		if (Database.IsValid())
		{
			// keeping around the assets for starting or in progress tasks
			Collector.AddReferencedObject(Database);
		}

		for (TWeakObjectPtr<const UObject>& Dependency : DatabaseDependencies)
		{
			if (Dependency.IsValid())
			{
				Collector.AddReferencedObject(Dependency);
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// FAsyncPoseSearchDatabasesManagement
FCriticalSection FAsyncPoseSearchDatabasesManagement::Mutex;

FAsyncPoseSearchDatabasesManagement& FAsyncPoseSearchDatabasesManagement::Get()
{
	FScopeLock Lock(&Mutex);

	static FAsyncPoseSearchDatabasesManagement SingletonInstance;
	return SingletonInstance;
}

FAsyncPoseSearchDatabasesManagement::FAsyncPoseSearchDatabasesManagement()
	: Tasks(*(new FPoseSearchDatabaseAsyncCacheTasks()))
{
	FScopeLock Lock(&Mutex);

	OnObjectModifiedHandle = FCoreUObjectDelegates::OnObjectModified.AddRaw(this, &FAsyncPoseSearchDatabasesManagement::OnObjectModified);
	OnObjectTransactedHandle = FCoreUObjectDelegates::OnObjectTransacted.AddRaw(this, &FAsyncPoseSearchDatabasesManagement::OnObjectTransacted);
	OnPackageReloadedHandle = FCoreUObjectDelegates::OnPackageReloaded.AddRaw(this, &FAsyncPoseSearchDatabasesManagement::OnPackageReloaded);
	OnPreObjectPropertyChangedHandle = FCoreUObjectDelegates::OnPreObjectPropertyChanged.AddRaw(this, &FAsyncPoseSearchDatabasesManagement::OnPreObjectPropertyChanged);
	OnObjectPropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FAsyncPoseSearchDatabasesManagement::OnObjectPropertyChanged);

	FCoreDelegates::OnPreExit.AddRaw(this, &FAsyncPoseSearchDatabasesManagement::Shutdown);
}

FAsyncPoseSearchDatabasesManagement::~FAsyncPoseSearchDatabasesManagement()
{
	FScopeLock Lock(&Mutex);

	FCoreDelegates::OnPreExit.RemoveAll(this);
	Shutdown();

	delete &Tasks;
}

// given Object it figures out a map of databases to UAnimSequenceBase(s) containing UAnimNotifyState_PoseSearchBranchIn
void FAsyncPoseSearchDatabasesManagement::CollectDatabasesToSynchronize(UObject* Object)
{
	if (!Object || !Object->GetClass())
	{
		return;
	}
	
	if (!Object->IsA<UAnimSequenceBase>() && !Object->IsA<UAnimNotifyState_PoseSearchBranchIn>())
	{
		return;
	}

	FScopeLock Lock(&Mutex);
	if (UAnimSequenceBase* SequenceBase = Cast<UAnimSequenceBase>(Object))
	{
		for (const FAnimNotifyEvent& NotifyEvent : SequenceBase->Notifies)
		{
			if (const UAnimNotifyState_PoseSearchBranchIn* BranchIn = Cast<UAnimNotifyState_PoseSearchBranchIn>(NotifyEvent.NotifyStateClass))
			{
				if (BranchIn->Database)
				{
					DatabasesToSynchronize.FindOrAdd(BranchIn->Database).AddUnique(SequenceBase);
				}
			}
		}
	}
	else if (UAnimNotifyState_PoseSearchBranchIn* BranchIn = Cast<UAnimNotifyState_PoseSearchBranchIn>(Object))
	{
		if (BranchIn->Database)
		{
			if (UAnimSequenceBase* OuterSequenceBase = Cast<UAnimSequenceBase>(BranchIn->GetOuter()))
			{
				DatabasesToSynchronize.FindOrAdd(BranchIn->Database).AddUnique(OuterSequenceBase);
			}
		}
	}
}

void FAsyncPoseSearchDatabasesManagement::SynchronizeDatabases()
{
	FScopeLock Lock(&Mutex);

	if (!DatabasesToSynchronize.IsEmpty())
	{
		// copying DatabasesToSynchronize because modifying the database will call OnObjectModified that could populate DatabasesToSynchronize again
		const TDatabasesToSynchronize DatabasesToSynchronizeCopy = DatabasesToSynchronize;
		DatabasesToSynchronize.Reset();

		TArray<UAnimSequenceBase*, TInlineAllocator<256>> SequencesBase;
		for (const TDatabasesToSynchronizePair& Pair : DatabasesToSynchronizeCopy)
		{
			if (Pair.Key.IsValid())
			{
				SequencesBase.Reset();
				for (const TWeakObjectPtr<UAnimSequenceBase>& SequenceBase : Pair.Value)
				{
					if (SequenceBase.IsValid())
					{
						SequencesBase.Add(SequenceBase.Get());
					}
				}

				Pair.Key->SynchronizeWithExternalDependencies(SequencesBase);
			}
		}
	}
}

// we're listening to OnObjectModified to cancel any pending Task indexing databases depending from Object to avoid multi threading issues
void FAsyncPoseSearchDatabasesManagement::OnObjectModified(UObject* Object)
{
	PreModified(Object, FKeyBuilder::ShouldIncludeFlagsToCheck);
}

void FAsyncPoseSearchDatabasesManagement::PreModified(UObject* Object, EObjectFlags ObjectFlagsToCheck)
{
	if (FKeyBuilder::ShouldInclude(Object, ObjectFlagsToCheck))
	{
		ensure(IsInGameThread());

		FScopeLock Lock(&Mutex);

		PartialKeyHashes.InvalidateKeysFor(Object);

		for (TUniquePtr<FPoseSearchDatabaseAsyncCacheTask>& TaskPtr : Tasks)
		{
			TaskPtr->PreCancelIfDependsOn(Object);
		}

		// collecting databases to synchronize prior modifying the Object
		CollectDatabasesToSynchronize(Object);
	}
}

void FAsyncPoseSearchDatabasesManagement::PostModified(UObject* Object, EObjectFlags ObjectFlagsToCheck)
{
	if (FKeyBuilder::ShouldInclude(Object, ObjectFlagsToCheck))
	{
		ensure(IsInGameThread());

		FScopeLock Lock(&Mutex);

		// collecting databases to synchronize, and merging the results with the PreModified collection
		CollectDatabasesToSynchronize(Object);

		SynchronizeDatabases();

		// promoting the EState::PreCancelled into EState::Cancelled
		for (TUniquePtr<FPoseSearchDatabaseAsyncCacheTask>& TaskPtr : Tasks)
		{
			if (TaskPtr->GetState() == FPoseSearchDatabaseAsyncCacheTask::EState::PreCancelled)
			{
				TaskPtr->Cancel();
			}
		}
	}
}

void FAsyncPoseSearchDatabasesManagement::OnObjectTransacted(UObject* Object, const FTransactionObjectEvent& TransactionObjectEvent)
{
	PostModified(Object, FKeyBuilder::ShouldIncludeFlagsToCheck);
}

void FAsyncPoseSearchDatabasesManagement::OnPackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent)
{
	check(IsInGameThread());

	if (!InPackageReloadedEvent)
	{
	}
	else if (InPackageReloadPhase == EPackageReloadPhase::PrePackageFixup)
	{
		const TMap<UObject*, UObject*>& RepointedObjects = InPackageReloadedEvent->GetRepointedObjects();
		for (const TPair<UObject*, UObject*>& RepointPair : RepointedObjects)
		{
			PreModified(RepointPair.Key, FKeyBuilder::ShouldIncludeFlagsToCheckOnReload);
		}
	}
	else if (InPackageReloadPhase == EPackageReloadPhase::PostPackageFixup)
	{
		const TMap<UObject*, UObject*>& RepointedObjects = InPackageReloadedEvent->GetRepointedObjects();
		for (const TPair<UObject*, UObject*>& RepointPair : RepointedObjects)
		{
			PostModified(RepointPair.Key, FKeyBuilder::ShouldIncludeFlagsToCheckOnReload);
		}
	}
}

void FAsyncPoseSearchDatabasesManagement::OnPreObjectPropertyChanged(UObject* InObject, const FEditPropertyChain& InPropertyChain)
{
	PreModified(InObject, FKeyBuilder::ShouldIncludeFlagsToCheck);
}

void FAsyncPoseSearchDatabasesManagement::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	PostModified(InObject, FKeyBuilder::ShouldIncludeFlagsToCheck);
}

void FAsyncPoseSearchDatabasesManagement::Shutdown()
{
	FScopeLock Lock(&Mutex);

	Tasks.Reset();

	FCoreUObjectDelegates::OnObjectModified.Remove(OnObjectModifiedHandle);
	OnObjectModifiedHandle.Reset();

	FCoreUObjectDelegates::OnObjectTransacted.Remove(OnObjectTransactedHandle);
	OnObjectTransactedHandle.Reset();

	FCoreUObjectDelegates::OnPackageReloaded.Remove(OnPackageReloadedHandle);
	OnPackageReloadedHandle.Reset();

	FCoreUObjectDelegates::OnPreObjectPropertyChanged.Remove(OnPreObjectPropertyChangedHandle);
	OnPreObjectPropertyChangedHandle.Reset();

	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnObjectPropertyChangedHandle);
	OnObjectPropertyChangedHandle.Reset();
}

void FAsyncPoseSearchDatabasesManagement::Tick(float DeltaTime)
{
	FScopeLock Lock(&Mutex);

	check(IsInGameThread());

#if ENABLE_ANIM_DEBUG
	// testing sampler determinism
	if (GVarMotionMatchTestDeterminismPartialKeyHashesSaveFile || GVarMotionMatchTestDeterminismPartialKeyHashes)
	{
		FSerializablePartialKeyHashes SerializablePartialKeyHashes(PartialKeyHashes);
		FString FileName = FString::Printf(TEXT("%s//TestPartialKeyHashesDeterminism//PartialKeyHashes.bin"), *FPaths::EngineDir());

		if (GVarMotionMatchTestDeterminismPartialKeyHashesSaveFile)
		{
			TArray<uint8> RawBytes;
			RawBytes.Reserve(1024 * 1024);
			FMemoryWriter Writer(RawBytes);
			Writer << SerializablePartialKeyHashes;
			FFileHelper::SaveArrayToFile(RawBytes, *FileName);

			GVarMotionMatchTestDeterminismPartialKeyHashesSaveFile = false;
			UE_LOGF(LogPoseSearch, Log, "FAsyncPoseSearchDatabasesManagement::Tick PartialKeyHashes serialized in '%ls'", *FileName);
		}

		if (GVarMotionMatchTestDeterminismPartialKeyHashes)
		{
			TArray<uint8> LoadedData;
			const bool bLoadedFile = FFileHelper::LoadFileToArray(LoadedData, *FileName, FILEREAD_Silent);
			if (bLoadedFile)
			{
				FMemoryReaderView Reader(LoadedData);
				FSerializablePartialKeyHashes SerializablePartialKeyHashesFromFile;
				Reader << SerializablePartialKeyHashesFromFile;
				if (SerializablePartialKeyHashes.LogDifferencesAgains(SerializablePartialKeyHashesFromFile))
				{
					UE_LOGF(LogPoseSearch, Log, "FAsyncPoseSearchDatabasesManagement::Tick PartialKeyHashes determinism validation succeeded");
				}
			}
			else
			{
				UE_LOGF(LogPoseSearch, Error, "FAsyncPoseSearchDatabasesManagement::Tick Failed to load '%ls' with serialized PartialKeyHashes", *FileName);
			}
			GVarMotionMatchTestDeterminismPartialKeyHashes = false;
		}
	}

	// testing sampler determinism
	const bool bTestAssetSamplerDeterminism = GVarMotionMatchTestDeterminismAssetSampler;
	const bool bTestAssetSamplerDeterminismFromPreviousExecution = GVarMotionMatchTestDeterminismAssetSamplerFromPreviousExecution;
	if (bTestAssetSamplerDeterminism || bTestAssetSamplerDeterminismFromPreviousExecution)
	{
		struct FTestSample
		{
			FBoneContainer BoneContainer;
			const UPoseSearchDatabase* Database = nullptr;
			int32 AnimationAssetIndex = INDEX_NONE;
			const UAnimationAsset* AnimationAsset = nullptr;
			FTransform RootTransformOrigin = FTransform::Identity;
			FVector BlendParameters = FVector::ZeroVector;
			int32 SampleIndex = INDEX_NONE;
			float SampleNormalizedTime = 0.f;

			TAlignedArray<uint8> SerializedData;

			FString GetFileName() const
			{
				return FString::Printf(TEXT("%s//TestAssetSamplerDeterminism//%s_%d_%s_%d.bin"), *FPaths::EngineDir(), *GetNameSafe(Database), AnimationAssetIndex, *GetNameSafe(AnimationAsset), SampleIndex);
			}
		};

		TArray<FTestSample> TestSamples;
		TestSamples.Reserve(256);
		for (const TUniquePtr<FPoseSearchDatabaseAsyncCacheTask>& TaskPtr : Tasks)
		{
			if (const UPoseSearchDatabase* Database = TaskPtr->GetDatabase())
			{
				if (Database->Schema)
				{
					TMap<FRole, FBoneContainer> RoledBoneContainers;
					InitRoledBoneContainers(RoledBoneContainers, Database, Database->Schema);

					for (int32 AnimationAssetIndex = 0; AnimationAssetIndex < Database->GetNumAnimationAssets(); ++AnimationAssetIndex)
					{
						if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAsset = Database->GetDatabaseAnimationAsset(AnimationAssetIndex))
						{
							if (DatabaseAsset->IsEnabled())
							{
								DatabaseAsset->IterateOverSamplingParameter([Database, DatabaseAsset, AnimationAssetIndex, &TestSamples, &RoledBoneContainers](const FVector& BlendParameters)
									{
										const int32 NumRoles = DatabaseAsset->GetNumRoles();
										for (int32 RoleIndex = 0; RoleIndex < NumRoles; ++RoleIndex)
										{
											const FRole& Role = DatabaseAsset->GetRole(RoleIndex);
											if (const FBoneContainer* BoneContainer = RoledBoneContainers.Find(Role))
											{
												const UAnimationAsset* AnimationAsset = DatabaseAsset->GetAnimationAssetForRole(Role);
												const FTransform RootTransformOrigin = DatabaseAsset->GetRootTransformOriginForRole(Role);

												constexpr int32 NUM_TESTING_SAMPLES_PER_DATABASE_ASSET = 100; // make sure it's greater than 1

												for (int32 SampleIndex = 0; SampleIndex < NUM_TESTING_SAMPLES_PER_DATABASE_ASSET; SampleIndex++)
												{
													FTestSample& TestSample = TestSamples.AddDefaulted_GetRef();
													TestSample.BoneContainer = *BoneContainer;
													TestSample.Database = Database;
													TestSample.AnimationAssetIndex = AnimationAssetIndex;
													TestSample.AnimationAsset = AnimationAsset;
													TestSample.RootTransformOrigin = RootTransformOrigin;
													TestSample.BlendParameters = BlendParameters;
													TestSample.SampleIndex = SampleIndex;
													TestSample.SampleNormalizedTime = SampleIndex / (NUM_TESTING_SAMPLES_PER_DATABASE_ASSET - 1);
												}
											}
										}
									});
							}
						}
					}
				}
			}
		}

		int32 NumIterations = GVarMotionMatchTestNumIterations;
		ParallelFor(TestSamples.Num(), [&TestSamples, NumIterations, bTestAssetSamplerDeterminism, bTestAssetSamplerDeterminismFromPreviousExecution](int32 TestSampleIndex)
			{
				FMemMark Mark(FMemStack::Get());

				FTestSample& TestSample = TestSamples[TestSampleIndex];

				const FAnimationAssetSampler AssetSampler(TestSample.AnimationAsset, FTransform::Identity, FVector::ZeroVector, FAnimationAssetSampler::DefaultRootTransformSamplingRate, true, !GVarMotionMatchUseRawAnimationData);
				const float SampleTime = AssetSampler.GetPlayLength() * TestSample.SampleNormalizedTime;

				FCompactPose Pose;
				Pose.SetBoneContainer(&TestSample.BoneContainer);
				AssetSampler.ExtractPose(SampleTime, Pose);
				const FTransform RootTransform = AssetSampler.ExtractRootTransform(SampleTime);

				const TConstArrayView<FTransform> Bones = Pose.GetBones();

				if (bTestAssetSamplerDeterminism)
				{
					for (int32 IterationIndex = 0; IterationIndex < NumIterations; ++IterationIndex)
					{
						FCompactPose TestPose;
						TestPose.SetBoneContainer(&TestSample.BoneContainer);
						AssetSampler.ExtractPose(SampleTime, TestPose);
						const FTransform TestRootTransform = AssetSampler.ExtractRootTransform(SampleTime);

						if (FMemory::Memcmp(&RootTransform, &TestRootTransform, sizeof(FTransform)) != 0)
						{
							UE_LOGF(LogPoseSearch, Error, "FAnimationAssetSampler - ExtractRootTransform is not deterministic");
						}

						const TConstArrayView<FTransform> TestBones = TestPose.GetBones();
						if (Bones.Num() != TestBones.Num())
						{
							UE_LOGF(LogPoseSearch, Error, "FAnimationAssetSampler - ExtractPose is not deterministic");
						}
						else
						{
							for (int32 BoneIndex = 0; BoneIndex < Bones.Num(); ++BoneIndex)
							{
								if (FMemory::Memcmp(&Bones[BoneIndex], &TestBones[BoneIndex], sizeof(FTransform)) != 0)
								{
									UE_LOGF(LogPoseSearch, Error, "FAnimationAssetSampler - ExtractPose is not deterministic");
								}
							}
						}
					}
				}

				if (bTestAssetSamplerDeterminismFromPreviousExecution)
				{
					const int32 RootTransformSize = sizeof(FTransform);
					const int32 BonesSize = sizeof(FTransform) * Bones.Num();

					TestSample.SerializedData.Reserve(RootTransformSize + BonesSize);
					TestSample.SerializedData.Append(reinterpret_cast<const uint8*>(&RootTransform), RootTransformSize);
					TestSample.SerializedData.Append(reinterpret_cast<const uint8*>(Bones.GetData()), BonesSize);
				}
			}, ParallelForFlags);

		if (bTestAssetSamplerDeterminismFromPreviousExecution)
		{
			for (const FTestSample& TestSample : TestSamples)
			{
				FString FileName = TestSample.GetFileName();
				TArray<uint8> LoadedData;
				const bool bLoadedFile = FFileHelper::LoadFileToArray(LoadedData, *FileName, FILEREAD_Silent);
				if (bLoadedFile)
				{
					if (LoadedData.Num() <= 0 || LoadedData.Num() % sizeof(FTransform) != 0)
					{
						UE_LOGF(LogPoseSearch, Error, "FAnimationAssetSampler - Loaded the wrong amount of data!");
					}
					else if (TestSample.SerializedData.Num() != LoadedData.Num())
					{
						UE_LOGF(LogPoseSearch, Error, "FAnimationAssetSampler - Loaded data mismatch expected amount of data!");
					}
					else if (FMemory::Memcmp(TestSample.SerializedData.GetData(), LoadedData.GetData(), TestSample.SerializedData.Num()) != 0)
					{
						const int32 NumTransforms = LoadedData.Num() / sizeof(FTransform);

						// copying the data into an aligned buffer, so we can cast it to FTransform
						TAlignedArray<uint8> LoadedDataAligned(LoadedData.GetData(), LoadedData.Num());
						TArrayView<const FTransform> LoadedTransforms(reinterpret_cast<const FTransform*>(LoadedDataAligned.GetData()), NumTransforms);
						TArrayView<const FTransform> SerializedTransforms(reinterpret_cast<const FTransform*>(TestSample.SerializedData.GetData()), NumTransforms);

						for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
						{
							if (FMemory::Memcmp(&LoadedTransforms[TransformIndex], &SerializedTransforms[TransformIndex], sizeof(FTransform)) != 0)
							{
								// NoTe:	TransformIndex == 0 for the root
								//			TransformIndex > 0 are the bones, where BoneIndex = TransformIndex - 1
								const int32 BoneIndex = TransformIndex - 1;
								UE_LOGF(LogPoseSearch, Error, "FAnimationAssetSampler - ExtractPose is not deterministic for %ls for Bone %d", *FileName, BoneIndex);
							}
						}
					}
				}
				else
				{
					bool bSavedFile = FFileHelper::SaveArrayToFile(TestSample.SerializedData, *FileName);
					if (!bSavedFile)
					{
						UE_LOGF(LogPoseSearch, Error, "FAnimationAssetSampler - Failed to save comparison file!");
					}
				}
			}
		}
	}

	if (GVarMotionMatchTestInvalidateCache)
	{
		if (GVarMotionMatchTestWaitForTaskCompletion)
		{
			// iterating backwards because of the possible RemoveAtSwap 
			for (int32 TaskIndex = Tasks.Num() - 1; TaskIndex >= 0; --TaskIndex)
			{
				if (Tasks[TaskIndex]->GetState() == FPoseSearchDatabaseAsyncCacheTask::EState::Ended ||
					Tasks[TaskIndex]->GetState() == FPoseSearchDatabaseAsyncCacheTask::EState::Failed)
				{
					UE_LOGF(LogPoseSearch, Log, "%ls - %ls Removed because of InvalidateCache with WaitForTaskCompletion", *LexToString(Tasks[TaskIndex]->GetDerivedDataKey()), *GetDatabaseAndOuterName(Tasks[TaskIndex]->GetDatabase()));
					Tasks.RemoveAtSwap(TaskIndex, EAllowShrinking::No);
				}
			}
		}
		else
		{
			for (TUniquePtr<FPoseSearchDatabaseAsyncCacheTask>& TaskPtr : Tasks)
			{
				UE_LOGF(LogPoseSearch, Log, "%ls - %ls Cancelled because of InvalidateCache", *LexToString(TaskPtr->GetDerivedDataKey()), *GetDatabaseAndOuterName(TaskPtr->GetDatabase()));
			}
			Tasks.Reset();
		}
	}

	if (GVarMotionMatchTestDeterminismSynchronizeWithExternalDependencies)
	{
		for (int32 TaskIndex = 0; TaskIndex < Tasks.Num(); ++TaskIndex)
		{
			Tasks[TaskIndex]->TestSynchronizeWithExternalDependencies();
		}
	}

#endif // ENABLE_ANIM_DEBUG
	
	// iterating backwards because of the possible RemoveAtSwap 
	for (int32 TaskIndex = Tasks.Num() - 1; TaskIndex >= 0; --TaskIndex)
	{
		if (!Tasks[TaskIndex]->IsValid())
		{
			UE_LOGF(LogPoseSearch, Log, "%ls - Removed because IsValid", *LexToString(Tasks[TaskIndex]->GetDerivedDataKey()));
			Tasks.RemoveAtSwap(TaskIndex, EAllowShrinking::No);
		}
		else if (Tasks[TaskIndex]->GetState() == FPoseSearchDatabaseAsyncCacheTask::EState::Cancelled)
		{
			const UPoseSearchDatabase* TaskDatabase = Tasks[TaskIndex]->GetDatabase();

			UE_LOGF(LogPoseSearch, Log, "%ls - %ls Removed because is Cancelled", *LexToString(Tasks[TaskIndex]->GetDerivedDataKey()), *GetDatabaseAndOuterName(TaskDatabase));
			Tasks.RemoveAtSwap(TaskIndex, EAllowShrinking::No);

#if ENABLE_ANIM_DEBUG
			if (GVarMotionMatchReindexCancelledDatabases)
			{
				RequestAsyncBuildIndex(TaskDatabase, ERequestAsyncBuildFlag::NewRequest);
			}
#endif // ENABLE_ANIM_DEBUG
		}
		else
		{
			Tasks[TaskIndex]->Update(Mutex, PartialKeyHashes);
		}
	}
	
#if ENABLE_ANIM_DEBUG
	if (GVarMotionMatchTestDeterminismDDCKey)
	{
		for (int32 TaskIndex = 0; TaskIndex < Tasks.Num(); ++TaskIndex)
		{
			if (const UPoseSearchDatabase* Database = Tasks[TaskIndex]->GetDatabase())
			{
				FKeyBuilder KeyBuilder(Database, false, false, nullptr, !GVarMotionMatchUseRawAnimationData);
				const FIoHash IoHash = KeyBuilder.Finalize();

				for (int32 IterationIndex = 0; IterationIndex < GVarMotionMatchTestNumIterations; ++IterationIndex)
				{
					FKeyBuilder TestKeyBuilder(Database, false, false, &PartialKeyHashes, !GVarMotionMatchUseRawAnimationData);
					const FIoHash TestIoHash = TestKeyBuilder.Finalize();

					if (!KeyBuilder.ValidateAgainst(TestKeyBuilder))
					{
						UE_LOGF(LogPoseSearch, Error, "FKeyBuilder - key generation is not deterministic: %ls / %ls for asset %ls", *LexToString(IoHash), *LexToString(TestIoHash), *Database->GetName());
					}

					if (IoHash != TestIoHash)
					{
						UE_LOGF(LogPoseSearch, Error, "FKeyBuilder - key generation is not deterministic: %ls / %ls for asset %ls", *LexToString(IoHash), *LexToString(TestIoHash), *Database->GetName());
					}
				}
			}
		}
	}
#endif // ENABLE_ANIM_DEBUG
}

void FAsyncPoseSearchDatabasesManagement::TickCook(float DeltaTime, bool bCookCompete)
{
	Tick(DeltaTime);
}

TStatId FAsyncPoseSearchDatabasesManagement::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncPoseSearchDatabasesManagement, STATGROUP_Tickables);
}

void FAsyncPoseSearchDatabasesManagement::AddReferencedObjects(FReferenceCollector& Collector)
{
	FScopeLock Lock(&Mutex);

	for (TUniquePtr<FPoseSearchDatabaseAsyncCacheTask>& TaskPtr : Tasks)
	{
		TaskPtr->AddReferencedObjects(Collector);
	}
}

EAsyncBuildIndexResult FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndexInternal(const UPoseSearchDatabase* Database, ERequestAsyncBuildFlag Flag)
{
	if (!IsValid(Database))
	{
		return EAsyncBuildIndexResult::Failed;
	}

	if (Database->GetPackage()->HasAnyPackageFlags(PKG_Cooked))
	{
		// Don't cache for cooked packages
		check(!Database->GetSearchIndex().IsEmpty());
		return EAsyncBuildIndexResult::Success;
	}

	FScopeLock Lock(&Mutex);

	check(EnumHasAnyFlags(Flag, ERequestAsyncBuildFlag::NewRequest | ERequestAsyncBuildFlag::ContinueRequest));

	FAsyncPoseSearchDatabasesManagement& This = FAsyncPoseSearchDatabasesManagement::Get();

	const bool bWaitForCompletion = EnumHasAnyFlags(Flag, ERequestAsyncBuildFlag::WaitForCompletion);

	FPoseSearchDatabaseAsyncCacheTask* Task = nullptr;
	for (TUniquePtr<FPoseSearchDatabaseAsyncCacheTask>& TaskPtr : This.Tasks)
	{
		if (TaskPtr->GetDatabase() == Database)
		{
			Task = TaskPtr.Get();

			if (EnumHasAnyFlags(Flag, ERequestAsyncBuildFlag::NewRequest))
			{
				if (Task->GetState() == FPoseSearchDatabaseAsyncCacheTask::EState::Prestarted)
				{
					Task->Cancel();
				}
				Task->StartNewRequestIfNeeded(bWaitForCompletion, This.PartialKeyHashes);
			}
			break;
		}
	}
		
	if (!Task)
	{
		// we didn't find the Task, so we Emplace a new one
		This.Tasks.Emplace(MakeUnique<FPoseSearchDatabaseAsyncCacheTask>(const_cast<UPoseSearchDatabase*>(Database), bWaitForCompletion, This.PartialKeyHashes));
		Task = This.Tasks.Last().Get();
	}

	if (bWaitForCompletion)
	{
		check(Task->GetState() != FPoseSearchDatabaseAsyncCacheTask::EState::Notstarted);
		if (Task->GetState() == FPoseSearchDatabaseAsyncCacheTask::EState::Prestarted)
		{
			Task->Wait(Mutex);
		}
	}

	if (Task->GetState() == FPoseSearchDatabaseAsyncCacheTask::EState::Ended)
	{
		check(!Database->GetSearchIndex().IsEmpty());
		return EAsyncBuildIndexResult::Success;
	}

	if (Task->GetState() == FPoseSearchDatabaseAsyncCacheTask::EState::Failed)
	{
		return EAsyncBuildIndexResult::Failed;
	}

	return EAsyncBuildIndexResult::InProgress;
}

EAsyncBuildIndexResult FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(const UPoseSearchDatabase* Database, ERequestAsyncBuildFlag Flag)
{
	if (GVarMotionMatchReindexAllReferencedDatabases)
	{
		FDatabaseSet DatabaseSet;
		RecursivePopulateDependentDatabases(Database, DatabaseSet);

		for (FDatabaseSet::TConstIterator Iter = DatabaseSet.CreateConstIterator(); Iter; ++Iter)
		{
			const UPoseSearchDatabase* DependentDatabase = *Iter;
			if (DependentDatabase != Database)
			{
				RequestAsyncBuildIndexInternal(DependentDatabase, ERequestAsyncBuildFlag::ContinueRequest);
			}
		}
	}

	return RequestAsyncBuildIndexInternal(Database, Flag);
}

void FAsyncPoseSearchDatabasesManagement::Clear()
{
	FScopeLock Lock(&Mutex);

	FAsyncPoseSearchDatabasesManagement& This = FAsyncPoseSearchDatabasesManagement::Get();

	for (TUniquePtr<FPoseSearchDatabaseAsyncCacheTask>& TaskPtr : This.Tasks)
	{
		UE_LOGF(LogPoseSearch, Log, "%ls - %ls Cancelled because of Clear", *LexToString(TaskPtr->GetDerivedDataKey()), *GetDatabaseAndOuterName(TaskPtr->GetDatabase()));
	}
	This.Tasks.Reset();
}

//////////////////////////////////////////////////////////////////////////
// FAsyncPoseSearchDatabasesManagementCache
EAsyncBuildIndexResult FAsyncPoseSearchDatabasesManagementCache::operator()(const UPoseSearchDatabase* Database, ERequestAsyncBuildFlag RequestAsyncBuildFlag)
{
	if (const EAsyncBuildIndexResult* AsyncBuildIndexResult = CachedRequests.Find(Database))
	{
		return *AsyncBuildIndexResult;
	}

	const EAsyncBuildIndexResult NewAsyncBuildIndexResult = FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(Database, RequestAsyncBuildFlag);
	CachedRequests.Add(Database) = NewAsyncBuildIndexResult;
	return NewAsyncBuildIndexResult;
}

} // namespace UE::PoseSearch

#undef LOCTEXT_NAMESPACE

#endif // WITH_EDITOR
