// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "PoseSearch/PoseSearchDerivedDataKey.h"
#include "Animation/AnimBoneCompressionSettings.h"
#include "Animation/AnimCompress.h"
#include "Animation/AnimCurveCompressionSettings.h"
#include "Animation/AnimData/IAnimationDataModel.h"
#include "Animation/IAnimationSequenceCompiler.h"
#include "AnimationModifier.h"
#include "Components/SkeletalMeshComponent.h"
#include "EdGraph/EdGraphNode.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAsset.h"
#include "Engine/StreamableRenderAsset.h"
#include "Factories/FbxAnimSequenceImportData.h"
#include "PoseSearch/PoseSearchAnimNotifies.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "RigVMCore/RigVMGraphFunctionHost.h"
#include "RigVMHost.h"
#include "RigVMModel/RigVMClient.h"
#include "UObject/DevObjectVersion.h"

namespace UE::PoseSearch
{
// log FKeyBuilder profiling
#ifndef ENABLE_KEY_PROFILING
	#define ENABLE_KEY_PROFILING 0
#endif
	
// log properties and UObjects names
#ifndef UE_POSE_SEARCH_DERIVED_DATA_LOGGING
	#define UE_POSE_SEARCH_DERIVED_DATA_LOGGING 0
#endif

// log properties data
#ifndef UE_POSE_SEARCH_DERIVED_DATA_LOGGING_VERBOSE
	#define UE_POSE_SEARCH_DERIVED_DATA_LOGGING_VERBOSE 0
#endif

FKeyBuilder::FKeyBuilder()
{
	check(IsInGameThread());

	ArIgnoreOuterRef = true;

	// Set FDerivedDataKeyBuilder to be a saving archive instead of a reference collector.
	// Reference collection causes FSoftObjectPtrs to be serialized by their weak pointer,
	// which doesn't give a stable hash.  Serializing these to a saving archive will
	// use a string reference instead, which is a more meaningful hash value.
	SetIsSaving(true);
}

FKeyBuilder::FKeyBuilder(const UObject* Object, bool bUseDataVer, bool bPerformConditionalPostLoadIfRequired, FPartialKeyHashes* InPartialKeyHashes, bool bInPerformSequenceResidencyRequests)
: FKeyBuilder()
{
#if ENABLE_KEY_PROFILING
	const double StartTime(FPlatformTime::Seconds());
#endif // ENABLE_KEY_PROFILING

	check(Object);
	check(IsInGameThread());

	// FKeyBuilder is a saving only archiver, and since it doesn't modify the input Object it's safe to do a const_cast 
	KeyOwner = const_cast<UObject*>(Object);

	// preallocating a reasonable amount of memory to avoid multiple reallocations
	ObjectsToSerialize.Reserve(256);
	ObjectBeingSerializedDependencies.Reserve(256);

	bPerformConditionalPostLoad = bPerformConditionalPostLoadIfRequired;
	bPerformSequenceResidencyRequests = bInPerformSequenceResidencyRequests;

	ExternalPartialKeyHashes = InPartialKeyHashes;

	UObject* NonPackageOuter = GetNonPackageOuter(KeyOwner);
	
	if (NonPackageOuter)
	{
		// if Object has a non UPackage outer we serialize that as well to collect all the referenced objects to KeyBuilder.GetDependencies() 
		// to be able to reindex databases owned by other objects. via operator << the Outer will ba added to ObjectsToSerialize as first item
		// and Object as second item, so Object will be serialized first. This will make unique keys for different Outer(s) with the same Outer
		// thing that wouldn't happen if Object and Outer gets added in the opposite order
		*this << NonPackageOuter;
	}
	
	// we still need to serialize the KeyOwner in case it has newly created or duplicated and not yet referenced by Outer
	*this << KeyOwner;

	while (!ObjectsToSerialize.IsEmpty() && !bAnyAssetNotFullyLoaded)
	{
		SerializeObjectInternal(ObjectsToSerialize.Pop(EAllowShrinking::No));
	}

#if ENABLE_KEY_PROFILING
	const double EndTime(FPlatformTime::Seconds());
	const double DeltaTimeInMilliseconds = (EndTime - StartTime) * 1000.0;
	UE_LOGF(LogPoseSearch, Log, "Profiling: FKeyBuilder::FKeyBuilder %f ms", DeltaTimeInMilliseconds);
#endif // ENABLE_KEY_PROFILING

}

void FKeyBuilder::Seek(int64 InPos)
{
	checkf(InPos == Tell(), TEXT("A hash cannot be computed when serialization relies on seeking."));
	FArchiveUObject::Seek(InPos);
}

bool FKeyBuilder::ShouldSkipProperty(const FProperty* InProperty) const
{
	if (InProperty == nullptr)
	{
		return false;
	}

	if (FArchiveUObject::ShouldSkipProperty(InProperty))
	{
		#if UE_POSE_SEARCH_DERIVED_DATA_LOGGING
		UE_LOGF(LogPoseSearch, Log, "  x %ls (ShouldSkipProperty)", *InProperty->GetFullName());
		#endif
		return true;
	}

	if (InProperty->HasAllPropertyFlags(CPF_Transient))
	{
		#if UE_POSE_SEARCH_DERIVED_DATA_LOGGING
		UE_LOGF(LogPoseSearch, Log, "  x %ls (Transient)", *InProperty->GetFullName());
		#endif
		return true;
	}

	if (InProperty->HasMetaData(ExcludeFromHashName))
	{
		#if UE_POSE_SEARCH_DERIVED_DATA_LOGGING
		UE_LOGF(LogPoseSearch, Log, "  x %ls (ExcludeFromHash)", *InProperty->GetFullName());
		#endif
		return true;
	}
		
	if (InProperty->HasMetaData(IgnoreForMemberInitializationTestName))
	{
		#if UE_POSE_SEARCH_DERIVED_DATA_LOGGING
		UE_LOGF(LogPoseSearch, Log, "  x %ls (IgnoreForMemberInitializationTest)", *InProperty->GetFullName());
		#endif
		return true;
	}

	check(!InProperty->HasMetaData(NeverInHashName));

	#if UE_POSE_SEARCH_DERIVED_DATA_LOGGING
	UE_LOGF(LogPoseSearch, Log, "  - %ls", *InProperty->GetFullName());
	#endif

	return false;
}

void FKeyBuilder::Serialize(void* Data, int64 Length)
{
	#if UE_POSE_SEARCH_DERIVED_DATA_LOGGING_VERBOSE
	const uint8* HasherData = reinterpret_cast<uint8*>(Data);
	FString RawBytesString = BytesToString(HasherData, Length);
	UE_LOGF(LogPoseSearch, Log, "  > %ls", *RawBytesString);
	#endif

	Hasher.Update(Data, Length);
}

FArchive& FKeyBuilder::operator<<(FStringBuilderBase& StringBuilderBase)
{
	// this is an encoding only archiver (no decoding will ever happen), so we don't need to serialize the length of the StringBuilderBase
	Serialize(StringBuilderBase.GetData(), sizeof(FStringBuilderBase::ElementType) * StringBuilderBase.Len());
	return *this;
}

FArchive& FKeyBuilder::operator<<(FName& Name)
{
	// non FStringBuilderBase version
	//FString NameString = Name.ToString();
	//*this << NameString;
	//return *this;
	
	// we cannot use GetTypeHash(Name) since it's bound to be non deterministic between editor restarts, so we convert the 
	// name into an FString and let the Serialize(void* Data, int64 Length) deal with it
	
	// using FStringBuilderBase instead of FAnsiStringBuilderBase to avoid Name TCHAR to ANSICHAR conversion 
	// (at the cost of having more data to pass into FKeyBuilder::Serialize), but with the optimization of not having to serialize the length of the string
	StringBuilder.Reset();
	StringBuilder << Name;
	*this << StringBuilder;
	return *this;
}

FArchive& FKeyBuilder::operator<<(FText& Value)
{
	// non FStringBuilderBase version
	//return FArchiveUObject::operator<<(Value);

	StringBuilder.Reset();
	StringBuilder << Value.ToString();
	*this << StringBuilder;
	return *this;
}

FArchive& FKeyBuilder::TryAddDependency(UObject* Object)
{
	check(Object);

	// @todo: add RF_NeedPostLoadSubobjects, RF_NeedInitialization, RF_NeedLoad, RF_WillBeLoaded? 
	if (Object->HasAnyFlags(RF_NeedPostLoad))
	{
		if (bPerformConditionalPostLoad)
		{
			Object->ConditionalPostLoad();
		}
		else
		{
			bAnyAssetNotFullyLoaded = true;
			return *this;
		}
	}

	if (Object->IsA<UAnimSequence>())
	{
		UAnimSequence* AnimSequence = Cast<UAnimSequence>(Object);
		check(AnimSequence);
		if (!AnimSequence->CanBeCompressed())
		{
			bAnyAssetNotFullyLoaded = true;
			return *this;
		}
	}

	// collecting ALL the dependencies of the object being serialized, so we can then cache it in PartialKeyHashes
	ObjectBeingSerializedDependencies.Add(Object);

	bool bAlreadyProcessed = false;
	Dependencies.Add(Object, &bAlreadyProcessed);

	// If we haven't already serialized this object
	if (bAlreadyProcessed)
	{
#if UE_POSE_SEARCH_DERIVED_DATA_LOGGING
		UE_LOGF(LogPoseSearch, Log, "AlreadyProcessed '%ls' (%ls)", *Object->GetName(), *Object->GetClass()->GetName());
#endif
		return *this;
	}

	ObjectsToSerialize.Add(Object);
	return *this;
}

FArchive& FKeyBuilder::operator<<(class UObject*& Object)
{
	if (ShouldInclude(Object, ShouldIncludeFlagsToCheck))
	{
		return TryAddDependency(Object);
	}

#if UE_POSE_SEARCH_DERIVED_DATA_LOGGING
	if (Object)
	{
		if (Object->HasAnyFlags(RF_Transient))
		{
			UE_LOGF(LogPoseSearch, Log, "Transient '%ls' (%ls)", *Object->GetName(), *Object->GetClass()->GetName());
		}
		if (Object->HasAnyFlags(RF_MirroredGarbage))
		{
			UE_LOGF(LogPoseSearch, Log, "MirroredGarbage '%ls' (%ls)", *Object->GetName(), *Object->GetClass()->GetName());
		}
		if (Object->HasAnyFlags(RF_NewerVersionExists))
		{
			UE_LOGF(LogPoseSearch, Log, "NewerVersionExists '%ls' (%ls)", *Object->GetName(), *Object->GetClass()->GetName());
		}
		if (Object->HasAnyFlags(RF_BeginDestroyed))
		{
			UE_LOGF(LogPoseSearch, Log, "BeginDestroyed '%ls' (%ls)", *Object->GetName(), *Object->GetClass()->GetName());
		}
		if (Object->HasAnyFlags(RF_FinishDestroyed))
		{
			UE_LOGF(LogPoseSearch, Log, "FinishDestroyed '%ls' (%ls)", *Object->GetName(), *Object->GetClass()->GetName());
		}
		if (Object->HasAnyFlags(RF_DuplicateTransient))
		{
			UE_LOGF(LogPoseSearch, Log, "DuplicateTransient '%ls' (%ls)", *Object->GetName(), *Object->GetClass()->GetName());
		}
		if (Object->HasAnyFlags(RF_NonPIEDuplicateTransient))
		{
			UE_LOGF(LogPoseSearch, Log, "DuplicateTransient '%ls' (%ls)", *Object->GetName(), *Object->GetClass()->GetName());
		}
		if (IsExcludedType(Object))
		{
			UE_LOGF(LogPoseSearch, Log, "Excluded '%ls' (%ls)", *Object->GetName(), *Object->GetClass()->GetName());
		}
	}
#endif // UE_POSE_SEARCH_DERIVED_DATA_LOGGING
	return *this;
}

void FKeyBuilder::SerializeObjectInternal(UObject* Object)
{
	// resetting Hasher, now used to hash all the input Object properties, except referencesto other UObject(s), that will be added to 	
	// ObjectBeingSerializedDependencies and Dependencies, and processed after this Object processing
	Hasher.Reset();

	check(!bAnyAssetNotFullyLoaded && Object);

	FPartialKeyHashes& PartialKeyHashes = GetPartialKeyHashes();

	FObjectKey ObjectKey(Object);
	if (const FPartialKeyHashes::FEntry* Entry = PartialKeyHashes.FindAndRemoveResolved(ObjectKey))
	{
		for (const FObjectKey& DependencyPtr : Entry->Dependencies)
		{
			if (UObject* Dependency = DependencyPtr.ResolveObjectPtr())
			{
				TryAddDependency(Dependency);
			}
		}
	}
	else
	{
		ObjectBeingSerializedDependencies.Reset();

		if (IsAddNameOnlyType(Object))
		{
			// for specific types we only add their names to the hash
#if UE_POSE_SEARCH_DERIVED_DATA_LOGGING
			UE_LOGF(LogPoseSearch, Log, "AddingNameOnly '%ls' (%ls)", *Object->GetName(), *Object->GetClass()->GetName());
#endif
			// non FStringBuilderBase version
			//FString FullName = Object->GetFullName();
			//*this << FullName;

			StringBuilder.Reset();
			Object->GetFullName(StringBuilder);
			*this << StringBuilder;
		}
		else
		{
#if UE_POSE_SEARCH_DERIVED_DATA_LOGGING
			UE_LOGF(LogPoseSearch, Log, "Begin '%ls' (%ls)", *Object->GetName(), *Object->GetClass()->GetName());
#endif
			Object->Serialize(*this);

#if UE_POSE_SEARCH_DERIVED_DATA_LOGGING
			UE_LOGF(LogPoseSearch, Log, "End '%ls' (%ls)", *Object->GetName(), *Object->GetClass()->GetName());
#endif
		}

		if (!bAnyAssetNotFullyLoaded)
		{
			PartialKeyHashes.Add(ObjectKey, Hasher.Finalize(), ObjectBeingSerializedDependencies);
		}
	}
}

FString FKeyBuilder::GetArchiveName() const
{
	return TEXT("FDerivedDataKeyBuilder");
}

bool FKeyBuilder::AnyAssetNotFullyLoaded() const
{
	return bAnyAssetNotFullyLoaded;
}

bool FKeyBuilder::AnyAssetNotReady() const
{
	if (bPerformSequenceResidencyRequests)
	{
		TArray<UAnimSequence*, TInlineAllocator<64>> SequencesToWaitFor;
		const ITargetPlatform* TargetPlatform = nullptr;
		for (const UObject* Dependency : Dependencies)
		{
			if (Dependency->IsA<UAnimSequence>())
			{
				UAnimSequence* AnimSequence = Cast<UAnimSequence>(const_cast<UObject*>(Dependency));
				check(AnimSequence);

				// initializing TargetPlatform lazily when the first UAnimSequence has been found
				if (SequencesToWaitFor.IsEmpty())
				{
					TargetPlatform = GetTargetPlatformManager()->GetRunningTargetPlatform();
				}

				AnimSequence->RequestResidency(TargetPlatform, GetTypeHash(KeyOwner));
				SequencesToWaitFor.Add(AnimSequence);
			}
		}

		if (!SequencesToWaitFor.IsEmpty())
		{
			Anim::IAnimSequenceCompilingManager::FinishCompilation(MakeArrayView(SequencesToWaitFor));

			for (UAnimSequence* AnimSequence : SequencesToWaitFor)
			{
				if (!AnimSequence->HasCompressedDataForPlatform(TargetPlatform))
				{
					return true;
				}
			}
		}
	}
	return false;
}

FIoHash FKeyBuilder::Finalize()
{
#if ENABLE_KEY_PROFILING
	const double StartTime(FPlatformTime::Seconds());
#endif // ENABLE_KEY_PROFILING

	check(!bAnyAssetNotFullyLoaded); // otherwise key can be non deterministic
	check(KeyOwner);

	FPartialKeyHashes& PartialKeyHashes = GetPartialKeyHashes();

	Hasher.Reset();

	TSet<FObjectKey> VisitedObjectKeys;
	VisitedObjectKeys.Reserve(8192);

	TArray<FObjectKey> ObjectKeysToVisit;
	ObjectKeysToVisit.Reserve(1024);

	if (UObject* NonPackageOuter = GetNonPackageOuter(KeyOwner))
	{
		ObjectKeysToVisit.Add(NonPackageOuter);
		
		// non FStringBuilderBase version
		//FString FullName = KeyOwner->GetFullName();
		//*this << FullName;

		// since Object is contained in Outer we need to serialize its name (otherwise skipped)
		// we cannot use GetTypeHash(Name) since it's bound to be non deterministic between editor restarts, so we convert the name into an FString and let the Serialize(void* Data, int64 Length) deal with it
		StringBuilder.Reset(); 
		KeyOwner->GetFullName(StringBuilder);
		*this << StringBuilder;
	}

	ObjectKeysToVisit.Add(KeyOwner);

	// used to invalidate the key without having to change POSESEARCHDB_DERIVEDDATA_VER all the times
	int32 NonConstDatabaseIndexDerivedDataCacheKeyVersion = DatabaseIndexDerivedDataCacheKeyVersion;
	FGuid VersionGuid = FDevSystemGuids::GetSystemGuid(FDevSystemGuids::Get().POSESEARCHDB_DERIVEDDATA_VER);
	FString AnimationCompressionVersionString = UE::Anim::Compression::AnimationCompressionVersionString;

	*this << VersionGuid;
	*this << AnimationCompressionVersionString;
	*this << NonConstDatabaseIndexDerivedDataCacheKeyVersion;
	*this << bPerformSequenceResidencyRequests;
		
	// iterating over all ObjectKeysToVisit to finalize the composition the final hash while continuing adding their unvisited dependencies
	while (!ObjectKeysToVisit.IsEmpty())
	{
		const FObjectKey CurrentObjectKey = ObjectKeysToVisit.Pop(EAllowShrinking::No);

		const FPartialKeyHashes::FEntry* Entry = PartialKeyHashes.Find(CurrentObjectKey);
		if (ensure(Entry))
		{
			const HashDigestType::ByteArray& LocalHashData = Entry->Hash.GetBytes();
			Hasher.Update(LocalHashData, sizeof(HashDigestType::ByteArray));
	
			bool bAlreadyVisited = false;
			VisitedObjectKeys.Add(CurrentObjectKey, &bAlreadyVisited);
			if (!bAlreadyVisited)
			{
				ObjectKeysToVisit.Append(Entry->Dependencies);
			}
		}
	}

	// Stores a BLAKE3-160 hash, taken from the first 20 bytes of a BLAKE3-256 hash
	const FIoHash IoHash(Hasher.Finalize());

#if ENABLE_KEY_PROFILING
	const double EndTime(FPlatformTime::Seconds());
	const double DeltaTimeInMilliseconds = (EndTime - StartTime) * 1000.0;
	UE_LOGF(LogPoseSearch, Log, "Profiling: FKeyBuilder::Finalize %f ms", DeltaTimeInMilliseconds);
#endif // ENABLE_KEY_PROFILING

	return IoHash;
}

const TSet<const UObject*>& FKeyBuilder::GetDependencies() const
{
	return Dependencies;
}

// to keep the key generation lightweight, we don't hash these types
bool FKeyBuilder::IsExcludedType(const UObject* Object)
{
	if (Object->IsA<UAnimationModifier>())
	{
		return true;
	}
		
	// excluding ALL the UAnimNotifyState(s) except the PoseSearch ones
	if (Object->IsA<UAnimNotifyState>() && !Object->IsA<UAnimNotifyState_PoseSearchBase>())
	{
		return true;
	}

	// excluding ALL the UAnimNotify(s) except the PoseSearch ones
	if (Object->IsA<UAnimNotify>() && !Object->IsA<UAnimNotify_PoseSearchBase>())
	{
		return true;
	}

	if (Object->IsA<UEdGraphNode>())
	{
		return true;
	}
	
	if (Object->IsA<URigVMHost>() || Object->IsA<URigVM>())
	{
		return true;
	}

	if (Cast<IRigVMClientHost>(Object) || Cast<IRigVMGraphFunctionHost>(Object) || Cast<IRigVMClientExternalModelHost>(Object))
	{
		return true;
	}

	return false;
}

bool FKeyBuilder::ShouldInclude(const UObject* Object, EObjectFlags ObjectFlagsToCheck)
{
	if (!Object)
	{
		return false;
	}

	if (Object->IsTemplate())
	{
		return false;
	}

	if (Object->HasAnyFlags(ObjectFlagsToCheck))
	{
		return false;
	}

	if (IsExcludedType(Object))
	{
		return false;
	}

	return true;
}

// to keep the key generation lightweight, we hash only the full names for these types. Object(s) will be added to Dependencies
bool FKeyBuilder::IsAddNameOnlyType(const UObject* Object)
{
	check(Object);

	return
		Object->IsA<UActorComponent>() ||
		Object->IsA<UAnimBoneCompressionSettings>() ||
		Object->IsA<UAnimCurveCompressionSettings>() ||
		Object->IsA<UAssetImportData>() ||
		Object->IsA<UFunction>() ||
		Object->IsA<USkeletalMesh>() ||
		Object->IsA<UStreamableRenderAsset>() ||
		nullptr != Cast<IAnimationDataModel>(Object);
}

bool FKeyBuilder::ValidateAgainst(const FKeyBuilder& Other) const
{
	if (bAnyAssetNotFullyLoaded != Other.bAnyAssetNotFullyLoaded)
	{
		return false;
	}

	if (Dependencies.Num() != Other.Dependencies.Num())
	{
		return false;
	}

	for (TSet<const UObject*>::TConstIterator Iter = Dependencies.CreateConstIterator(); Iter; ++Iter)
	{
		if (!Other.Dependencies.Contains(*Iter))
		{
			return false;
		}
	}

	if (ObjectsToSerialize.Num() != Other.ObjectsToSerialize.Num())
	{
		return false;
	}
	
	for (int32 Index = 0; Index < ObjectsToSerialize.Num(); ++Index)
	{
		if (ObjectsToSerialize[Index] != Other.ObjectsToSerialize[Index])
		{
			return false;
		}
	}

	return true;
}

} // namespace UE::PoseSearch	

#endif // WITH_EDITOR