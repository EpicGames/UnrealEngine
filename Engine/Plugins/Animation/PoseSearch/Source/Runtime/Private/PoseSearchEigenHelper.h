// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"

#if UE_BUILD_DEBUG && WITH_EDITOR
#define UE_POSE_SEARCH_EIGEN_DEBUG 1
#endif

#ifndef UE_POSE_SEARCH_EIGEN_DEBUG
#define UE_POSE_SEARCH_EIGEN_DEBUG 0
#endif

// Eigen debugging configuration
#if UE_POSE_SEARCH_EIGEN_DEBUG
//#define EIGEN_INITIALIZE_MATRICES_BY_NAN
#define EIGEN_NO_AUTOMATIC_RESIZING
#ifdef EIGEN_NO_DEBUG
#undef EIGEN_NO_DEBUG
#endif
#ifdef EIGEN_NO_STATIC_ASSERT
#undef EIGEN_NO_STATIC_ASSERT
#endif
#endif // UE_POSE_SEARCH_EIGEN_DEBUG

// Disbable static analysis warnings
#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(push)
#pragma warning(disable:6294) // Ill-defined for-loop:  initial condition does not satisfy test.  Loop body not executed.
#endif

#ifndef EIGEN_MPL2_ONLY
#define EIGEN_MPL2_ONLY
#endif

PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
#include <Eigen/Core>
#include <Eigen/Eigenvalues>
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END

// Restore static analysis warnings
#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(pop)
#endif

#if UE_POSE_SEARCH_EIGEN_DEBUG
#include <sstream>
#endif

namespace UE::PoseSearch
{
	using RowMajorVector = Eigen::Matrix<float, 1, Eigen::Dynamic, Eigen::RowMajor>;
	using RowMajorVectorMap = Eigen::Map<RowMajorVector, Eigen::RowMajor>;
	using RowMajorVectorMapConst = Eigen::Map<const RowMajorVector, Eigen::RowMajor>;

	using RowMajorMatrix = Eigen::Matrix<float, Eigen::Dynamic, Eigen::Dynamic, Eigen::RowMajor>;
	using RowMajorMatrixMap = Eigen::Map<RowMajorMatrix, Eigen::RowMajor>;
	using RowMajorMatrixMapConst = Eigen::Map<const RowMajorMatrix, Eigen::RowMajor>;

	using ColMajorMatrix = Eigen::Matrix<float, Eigen::Dynamic, Eigen::Dynamic, Eigen::ColMajor>;
	using ColMajorMatrixMap = Eigen::Map<ColMajorMatrix, Eigen::ColMajor>;
	using ColMajorMatrixMapConst = Eigen::Map<const ColMajorMatrix, Eigen::ColMajor>;

#if UE_POSE_SEARCH_EIGEN_DEBUG
	template<typename EigenDenseBaseDerivedType>
	FString EigenMatrixToString(const Eigen::DenseBase<EigenDenseBaseDerivedType>& Matrix)
	{
		std::stringstream StringStream;
		StringStream << Matrix;
		return StringStream.str().c_str();
	}
#endif

} // namespace UE::PoseSearch