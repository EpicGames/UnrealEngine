// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchEvent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchEvent)

bool FPoseSearchEvent::IsValid() const
{
	// maybe we should check if TimeToEvent >= 0?
	return EventTag.IsValid();
}

void FPoseSearchEvent::Reset()
{
	EventTag = FGameplayTag();
	TimeToEvent = 0.f;
}

FPoseSearchEvent FPoseSearchEvent::GetPlayRateOverriddenEvent(const FFloatInterval& PlayRateRangeBase) const
{
	FPoseSearchEvent PlayRateOverriddenEventToSearch = *this;

	if (!bUsePlayRateRangeOverride)
	{
		PlayRateOverriddenEventToSearch.PlayRateRangeOverride = PlayRateRangeBase;
	}

	return PlayRateOverriddenEventToSearch;
}

void UPoseSearchEventLibrary::UpdatePoseSearchEvent(const FPoseSearchEvent& InNewEvent, bool bIsNewEventValid, float DeltaSeconds, FPoseSearchEvent& InOutCurrentEvent)
{
	if (bIsNewEventValid && InNewEvent.IsValid())
	{
		InOutCurrentEvent = InNewEvent;
	}
	else
	{
		InOutCurrentEvent.TimeToEvent -= DeltaSeconds;
		if (InOutCurrentEvent.TimeToEvent < 0.f)
		{
			InOutCurrentEvent.TimeToEvent = 0.f;
			InOutCurrentEvent.EventTag = FGameplayTag();
		}
	}
}
