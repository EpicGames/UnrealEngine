// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchFeatureChannel.h"
#include "PoseSearch/PoseSearchDerivedDataKey.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "Animation/Skeleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchFeatureChannel)

//////////////////////////////////////////////////////////////////////////
// UPoseSearchFeatureChannel
void UPoseSearchFeatureChannel::GetPermutationTimeOffsets(EPermutationTimeType PermutationTimeType, float DesiredPermutationTimeOffset, float& OutPermutationSampleTimeOffset, float& OutPermutationOriginTimeOffset)
{
	switch (PermutationTimeType)
	{
	case EPermutationTimeType::UseSampleTime:
		OutPermutationSampleTimeOffset = 0.f;
		OutPermutationOriginTimeOffset = 0.f;
		break;
	case EPermutationTimeType::UsePermutationTime:
		OutPermutationSampleTimeOffset = DesiredPermutationTimeOffset;
		OutPermutationOriginTimeOffset = DesiredPermutationTimeOffset;
		break;
	case EPermutationTimeType::UseSampleToPermutationTime:
		OutPermutationSampleTimeOffset = DesiredPermutationTimeOffset;
		OutPermutationOriginTimeOffset = 0.f;
		break;
	default:
		checkNoEntry();
		OutPermutationSampleTimeOffset = 0.f;
		OutPermutationOriginTimeOffset = 0.f;
		break;
	}
}

#if ENABLE_DRAW_DEBUG
void UPoseSearchFeatureChannel::DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseVector) const
{
	for (int32 FilterIndex = 0; FilterIndex < GetNumFilters(); ++FilterIndex)
	{
		if (const FPoseSearchFilter* Filter = GetFilter(FilterIndex))
		{
			Filter->DebugDraw(DrawParams, PoseVector, this);
		}
	}
}
#endif //ENABLE_DRAW_DEBUG

#if WITH_EDITOR
void UPoseSearchFeatureChannel::GetOuterLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat) const
{
	if (LabelFormat != UE::PoseSearch::ELabelFormat::Compact_Horizontal)
	{
		if (const UPoseSearchFeatureChannel* OuterChannel = Cast<UPoseSearchFeatureChannel>(GetOuter()))
		{
			if (LabelBuilder.Len() > 0)
			{
				LabelBuilder.Append(TEXT("_"));
			}
			OuterChannel->GetLabel(LabelBuilder, UE::PoseSearch::ELabelFormat::Full_Horizontal);
		}
	}
}

void UPoseSearchFeatureChannel::AppendLabelSeparator(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat, bool bTryUsingSpace)
{
	if (LabelBuilder.Len() == 0)
	{
		// do nothing
	}
	else if (LabelFormat == UE::PoseSearch::ELabelFormat::Full_Vertical)
	{
		LabelBuilder.Append(TEXT("\n"));
	}
	else if (bTryUsingSpace)
	{
		LabelBuilder.Append(TEXT(" "));
	}
	else
	{
		LabelBuilder.Append(TEXT("_"));
	}
}

UE::PoseSearch::TLabelBuilder& UPoseSearchFeatureChannel::GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat) const
{
	GetOuterLabel(LabelBuilder, LabelFormat);
	AppendLabelSeparator(LabelBuilder, LabelFormat);
	LabelBuilder.Append(GetName());
	return LabelBuilder;
}

bool UPoseSearchFeatureChannel::CanBeNormalizedWith(const UPoseSearchFeatureChannel* Other) const
{
	using namespace UE::PoseSearch;

	if (this == Other)
	{
		return true;
	}

	if (ChannelCardinality != Other->ChannelCardinality)
	{
		return false;
	}

	if (GetClass() != Other->GetClass())
	{
		return false;
	}

	if (!GetSchema()->AreSkeletonsCompatible(Other->GetSchema()))
	{
		return false;
	}

	if (!GetNormalizationGroup().IsNone() && GetNormalizationGroup() == Other->GetNormalizationGroup())
	{
		return true;
	}

	TLabelBuilder ThisLabelBuilder, OtherLabelBuilder;
	if (FCString::Strcmp(GetLabel(ThisLabelBuilder).ToString(), Other->GetLabel(OtherLabelBuilder).ToString()) != 0)
	{
		return false;
	}

	return true;
}

const UPoseSearchSchema* UPoseSearchFeatureChannel::GetSchema() const
{
	UObject* Outer = GetOuter();
	while (Outer != nullptr)
	{
		if (const UPoseSearchSchema* Schema = Cast<UPoseSearchSchema>(Outer))
		{
			return Schema;
		}
		Outer = Outer->GetOuter();
	}
	return nullptr;
}

const UE::PoseSearch::FRole UPoseSearchFeatureChannel::GetDefaultRole() const
{
	if (const UPoseSearchSchema* Schema = GetSchema())
	{
		return Schema->GetDefaultRole();
	}
	return UE::PoseSearch::DefaultRole;
}

#endif // WITH_EDITOR

USkeleton* UPoseSearchFeatureChannel::GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle)
{
	bInvalidSkeletonIsError = false;
#if WITH_EDITOR
	// blueprint generated classes don't have a schema, until they're instanced by the schema
	if (const UPoseSearchSchema* Schema = GetSchema())
	{
		return Schema->GetSkeleton(GetDefaultRole());
	}
#else
	checkNoEntry();
#endif
	return nullptr;
}


