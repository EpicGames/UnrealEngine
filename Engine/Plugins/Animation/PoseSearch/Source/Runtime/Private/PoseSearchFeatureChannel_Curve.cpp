// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchFeatureChannel_Curve.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "PoseSearch/PoseSearchAssetIndexer.h"
#include "PoseSearch/PoseSearchAssetSampler.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchHistory.h"
#include "PoseSearch/PoseSearchSchema.h"

#if WITH_EDITOR
#include "PropertyHandle.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchFeatureChannel_Curve)

UPoseSearchFeatureChannel_Curve::UPoseSearchFeatureChannel_Curve()
{
	bUseBlueprintQueryOverride = Cast<UBlueprintGeneratedClass>(GetClass()) != nullptr;
}

bool UPoseSearchFeatureChannel_Curve::Finalize(UPoseSearchSchema* Schema)
{
	ChannelDataOffset = Schema->SchemaCardinality;
	ChannelCardinality = 1;
	Schema->SchemaCardinality += ChannelCardinality;

	CurveIdx = Schema->AddCurveReference(CurveName, SampleRole);

	return true;
}

float UPoseSearchFeatureChannel_Curve::BP_GetCurveValueFromContext_Implementation(FChooserEvaluationContext& Context) const
{
	using namespace UE::PoseSearch;

	if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(GetAnimContext(&Context)))
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return BP_GetCurveValue(AnimInstance);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	UE_LOGF(LogPoseSearch, Warning, "UPoseSearchFeatureChannel_Position::BP_GetCurveValueFromContext_Implementation - unsupported null UAnimInstance: WIP support for AnimNext!");
	return 0.f;
}


void UPoseSearchFeatureChannel_Curve::BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const
{
	using namespace UE::PoseSearch;

	if (bUseBlueprintQueryOverride)
	{
		FChooserEvaluationContext* ChooserContext = SearchContext.GetContext(SampleRole);
		const float CurveValue = ensure(ChooserContext) ? BP_GetCurveValueFromContext(*ChooserContext) : 0.f;
		FFeatureVectorHelper::EncodeFloat(SearchContext.EditFeatureVector(), ChannelDataOffset, CurveValue);
		return;
	}
	
	// trying to get the BuildQuery data from another schema UPoseSearchFeatureChannel_Curve already cached in the SearchContext
	if (SearchContext.IsUseCachedChannelData())
	{
		// composing a unique identifier to specify this channel with all the required properties to be able to share the query data with other channels of the same type
		uint32 UniqueIdentifier = GetClass()->GetUniqueID();
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(CurveName));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SampleRole));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(CurveIdx));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SampleTimeOffset));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(InputQueryPose));

		TConstArrayView<float> CachedChannelData;
		if (const UPoseSearchFeatureChannel* CachedChannel = SearchContext.GetCachedChannelData(UniqueIdentifier, this, CachedChannelData))
		{
#if DO_CHECK
			const UPoseSearchFeatureChannel_Curve* CachedCurveChannel = Cast<UPoseSearchFeatureChannel_Curve>(CachedChannel);
			check(CachedCurveChannel);
			check(CachedCurveChannel->GetChannelCardinality() == ChannelCardinality);
			check(CachedChannelData.Num() == ChannelCardinality);

			// making sure there were no hash collisions
			check(CachedCurveChannel->CurveName == CurveName);
			check(CachedCurveChannel->SampleRole == SampleRole);
			check(CachedCurveChannel->CurveIdx == CurveIdx);
			check(CachedCurveChannel->SampleTimeOffset == SampleTimeOffset);
			check(CachedCurveChannel->InputQueryPose == InputQueryPose);
#endif //DO_CHECK

			// copying the CachedChannelData into this channel portion of the FeatureVectorBuilder
			FFeatureVectorHelper::Copy(SearchContext.EditFeatureVector().Slice(ChannelDataOffset, ChannelCardinality), 0, ChannelCardinality, CachedChannelData);
			return;
		}
	}

	const bool bCanUseContinuingPoseValues = SearchContext.CanUseContinuingPoseValues();
	const bool bSkip = InputQueryPose != EInputQueryPose::UseCharacterPose && bCanUseContinuingPoseValues;
	if (bSkip)
	{
		if (bCanUseContinuingPoseValues)
		{
			FFeatureVectorHelper::Copy(SearchContext.EditFeatureVector(), ChannelDataOffset, ChannelCardinality, SearchContext.GetContinuingPoseValues());
		}
		return;
	}
	
	const float CurveValue = SearchContext.GetSampleCurveValue(SampleTimeOffset, CurveName, SampleRole);
	FFeatureVectorHelper::EncodeFloat(SearchContext.EditFeatureVector(), ChannelDataOffset, CurveValue);
}

#if WITH_EDITOR
void UPoseSearchFeatureChannel_Curve::FillWeights(TArrayView<float> Weights) const
{
	for (int32 i = 0; i < ChannelCardinality; ++i)
	{
		Weights[ChannelDataOffset + i] = Weight;
	}
}

bool UPoseSearchFeatureChannel_Curve::IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const
{
	using namespace UE::PoseSearch;

	for (int32 SampleIdx = Indexer.GetBeginSampleIdx(); SampleIdx != Indexer.GetEndSampleIdx(); ++SampleIdx)
	{
		float CurveValue;

		Indexer.GetSampleCurveValue(CurveValue, SampleTimeOffset, SampleIdx, CurveName, SampleRole);
		FFeatureVectorHelper::EncodeFloat(Indexer.GetPoseVector(SampleIdx), ChannelDataOffset, CurveValue);
	}

	return true;
}

UE::PoseSearch::TLabelBuilder& UPoseSearchFeatureChannel_Curve::GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat) const
{
	using namespace UE::PoseSearch;

	GetOuterLabel(LabelBuilder, LabelFormat);
	AppendLabelSeparator(LabelBuilder, LabelFormat);

	LabelBuilder.Append(TEXT("Curve_"));
	LabelBuilder.Append(CurveName.ToString());
	AppendLabelSeparator(LabelBuilder, LabelFormat, true);
	LabelBuilder.Appendf(TEXT("%.2f"), SampleTimeOffset);
	return LabelBuilder;
}
#endif
