// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchFeatureChannel_Distance.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "PoseSearch/PoseSearchAssetIndexer.h"
#include "PoseSearch/PoseSearchAssetSampler.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchFeatureChannel_Position.h"
#include "PoseSearch/PoseSearchSchema.h"

#if WITH_EDITOR
#include "PropertyHandle.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchFeatureChannel_Distance)

UPoseSearchFeatureChannel_Distance::UPoseSearchFeatureChannel_Distance()
{
	bUseBlueprintQueryOverride = Cast<UBlueprintGeneratedClass>(GetClass()) != nullptr;
}

bool UPoseSearchFeatureChannel_Distance::Finalize(UPoseSearchSchema* Schema)
{
	using namespace UE::PoseSearch;

	ChannelDataOffset = Schema->SchemaCardinality;
	ChannelCardinality = 1;
	Schema->SchemaCardinality += ChannelCardinality;

	SchemaBoneIdx = Schema->AddBoneReference(Bone, SampleRole, bDefaultWithRootBone);
	SchemaOriginBoneIdx = Schema->AddBoneReference(OriginBone, OriginRole, bDefaultWithRootBone);
	
	return SchemaBoneIdx != InvalidSchemaBoneIdx && SchemaOriginBoneIdx != InvalidSchemaBoneIdx;
}

void UPoseSearchFeatureChannel_Distance::AddDependentChannels(UPoseSearchSchema* Schema) const
{
	using namespace UE::PoseSearch;

	if (Schema->bInjectAdditionalDebugChannels)
	{
		if (SchemaOriginBoneIdx != RootSchemaBoneIdx || PermutationTimeType == EPermutationTimeType::UsePermutationTime)
		{
			const EPermutationTimeType DependentChannelsPermutationTimeType = PermutationTimeType == EPermutationTimeType::UsePermutationTime ? EPermutationTimeType::UseSampleToPermutationTime : EPermutationTimeType::UseSampleTime;
			UPoseSearchFeatureChannel_Position::FindOrAddToSchema(Schema, 0.f, OriginBone.BoneName, OriginRole, DependentChannelsPermutationTimeType, bDefaultWithRootBone);
		}
	}
}

bool UPoseSearchFeatureChannel_Distance::IsFilterValid(TConstArrayView<float> PoseValues, TConstArrayView<float> QueryValues, int32 PoseIdx, const UE::PoseSearch::FPoseMetadata& Metadata) const
{
	using namespace UE::PoseSearch;
	const float Pose = FFeatureVectorHelper::DecodeFloat(PoseValues, ChannelDataOffset);
	const float Query = FFeatureVectorHelper::DecodeFloat(QueryValues, ChannelDataOffset);

	check(MaxDistance > 0.f);

	return FMath::Abs(Pose - Query) <= MaxDistance;
}

void UPoseSearchFeatureChannel_Distance::BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const
{
	using namespace UE::PoseSearch;

	if (bUseBlueprintQueryOverride)
	{
		if (FChooserEvaluationContext* ChooserContext = SearchContext.GetContext(SampleRole))
		{
			const float Distance = BP_GetDistance(*ChooserContext);
			FFeatureVectorHelper::EncodeFloat(SearchContext.EditFeatureVector(), ChannelDataOffset, Distance);
		}
		return;
	}

	// trying to get the BuildQuery data from another schema UPoseSearchFeatureChannel_Distance already cached in the SearchContext
	if (SearchContext.IsUseCachedChannelData())
	{
		// composing a unique identifier to specify this channel with all the required properties to be able to share the query data with other channels of the same type
		uint32 UniqueIdentifier = GetClass()->GetUniqueID();
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SampleRole));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(OriginRole));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SamplingAttributeId));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SampleTimeOffset));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(OriginTimeOffset));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SchemaBoneIdx));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SchemaOriginBoneIdx));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(InputQueryPose));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(PermutationTimeType));

		TConstArrayView<float> CachedChannelData;
		if (const UPoseSearchFeatureChannel* CachedChannel = SearchContext.GetCachedChannelData(UniqueIdentifier, this, CachedChannelData))
		{
#if DO_CHECK
			const UPoseSearchFeatureChannel_Distance* CachedDistanceChannel = Cast<UPoseSearchFeatureChannel_Distance>(CachedChannel);
			check(CachedDistanceChannel);
			check(CachedDistanceChannel->GetChannelCardinality() == ChannelCardinality);
			check(CachedChannelData.Num() == ChannelCardinality);

			// making sure there were no hash collisions
			check(CachedDistanceChannel->SampleRole == SampleRole);
			check(CachedDistanceChannel->OriginRole == OriginRole);
			check(CachedDistanceChannel->SamplingAttributeId == SamplingAttributeId);
			check(CachedDistanceChannel->SampleTimeOffset == SampleTimeOffset);
			check(CachedDistanceChannel->OriginTimeOffset == OriginTimeOffset);
			check(CachedDistanceChannel->SchemaBoneIdx == SchemaBoneIdx);
			check(CachedDistanceChannel->SchemaOriginBoneIdx == SchemaOriginBoneIdx);
			check(CachedDistanceChannel->InputQueryPose == InputQueryPose);
			check(CachedDistanceChannel->PermutationTimeType == PermutationTimeType);
#endif //DO_CHECK

			// copying the CachedChannelData into this channel portion of the FeatureVectorBuilder
			FFeatureVectorHelper::Copy(SearchContext.EditFeatureVector().Slice(ChannelDataOffset, ChannelCardinality), 0, ChannelCardinality, CachedChannelData);
			return;
		}
	}

	const bool bCanUseContinuingPoseValues = SearchContext.CanUseContinuingPoseValues();
	const bool bSkip = InputQueryPose != EInputQueryPose::UseCharacterPose && bCanUseContinuingPoseValues && SampleRole == OriginRole;
	if (bSkip)
	{
		if (bCanUseContinuingPoseValues)
		{
			FFeatureVectorHelper::Copy(SearchContext.EditFeatureVector(), ChannelDataOffset, ChannelCardinality, SearchContext.GetContinuingPoseValues());
		}
		return;
	}
	
	// calculating the distance
	const FVector BonePosition = SearchContext.GetSamplePosition(SampleTimeOffset, OriginTimeOffset, SchemaBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole, PermutationTimeType);
	const float Distance = BonePosition.Length();
	FFeatureVectorHelper::EncodeFloat(SearchContext.EditFeatureVector(), ChannelDataOffset, Distance);
}

#if ENABLE_DRAW_DEBUG
void UPoseSearchFeatureChannel_Distance::DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseVector) const
{
	using namespace UE::PoseSearch;

	bool bDrawInjectAdditionalDebugChannels = false;
#if WITH_EDITORONLY_DATA
	if (const UPoseSearchSchema* Schema = GetSchema())
	{
		bDrawInjectAdditionalDebugChannels = Schema->bDrawInjectAdditionalDebugChannels;
	}
#endif // WITH_EDITORONLY_DATA

	if (bDrawInjectAdditionalDebugChannels || DrawParams.IsAnyWeightRelevant(this))
	{
		Super::DebugDraw(DrawParams, PoseVector);

		FColor Color;
#if WITH_EDITORONLY_DATA
		Color = DebugColor.ToFColor(true);
#else // WITH_EDITORONLY_DATA
		Color = FColor::Blue;
#endif // WITH_EDITORONLY_DATA

		float PermutationSampleTimeOffset = 0.f;
		float PermutationOriginTimeOffset = 0.f;
		UPoseSearchFeatureChannel::GetPermutationTimeOffsets(PermutationTimeType, DrawParams.ExtractPermutationTime(PoseVector), PermutationSampleTimeOffset, PermutationOriginTimeOffset);
		const EPermutationTimeType OriginPermutationTimeType = PermutationTimeType == EPermutationTimeType::UsePermutationTime ? EPermutationTimeType::UseSampleToPermutationTime : EPermutationTimeType::UseSampleTime;

		const float Distance = FFeatureVectorHelper::DecodeFloat(PoseVector, ChannelDataOffset);
		const FVector OriginBonePos = DrawParams.ExtractPosition(PoseVector, OriginTimeOffset, SchemaOriginBoneIdx, OriginRole, OriginPermutationTimeType, INDEX_NONE, PermutationOriginTimeOffset);

		static const int32 Segments = 32;
		DrawParams.DrawSphere(OriginBonePos, Distance, Segments, Color);
	}
}
#endif // ENABLE_DRAW_DEBUG

#if WITH_EDITOR
void UPoseSearchFeatureChannel_Distance::FillWeights(TArrayView<float> Weights) const
{
	for (int32 i = 0; i < ChannelCardinality; ++i)
	{
		Weights[ChannelDataOffset + i] = Weight;
	}
}

bool UPoseSearchFeatureChannel_Distance::IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const
{
	using namespace UE::PoseSearch;

	FVector BonePosition;
	for (int32 SampleIdx = Indexer.GetBeginSampleIdx(); SampleIdx != Indexer.GetEndSampleIdx(); ++SampleIdx)
	{
		if (Indexer.GetSamplePosition(BonePosition, SampleTimeOffset, OriginTimeOffset, SampleIdx, SchemaBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole, PermutationTimeType, SamplingAttributeId))
		{
			FFeatureVectorHelper::EncodeFloat(Indexer.GetPoseVector(SampleIdx), ChannelDataOffset, BonePosition.Length());
		}
		else
		{
			return false;
		}
	}
	return true;
}

UE::PoseSearch::TLabelBuilder& UPoseSearchFeatureChannel_Distance::GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat) const
{
	using namespace UE::PoseSearch;

	GetOuterLabel(LabelBuilder, LabelFormat);
	AppendLabelSeparator(LabelBuilder, LabelFormat);

	LabelBuilder.Append(TEXT("Dist"));

	const UPoseSearchSchema* Schema = GetSchema();
	check(Schema);
	if (SchemaBoneIdx > RootSchemaBoneIdx)
	{
		LabelBuilder.Append(TEXT("_"));
		LabelBuilder.Append(Schema->GetBoneReferences(SampleRole)[SchemaBoneIdx].BoneName.ToString());
	}
	else if (SchemaBoneIdx == TrajectorySchemaBoneIdx)
	{
		LabelBuilder.Append(TEXT("_Trj"));
	}
	
	if (SampleRole != DefaultRole)
	{
		LabelBuilder.Append(TEXT("["));
		LabelBuilder.Append(SampleRole.ToString());
		LabelBuilder.Append(TEXT("]"));
	}

	if (SchemaOriginBoneIdx > RootSchemaBoneIdx)
	{
		LabelBuilder.Append(TEXT("_"));
		LabelBuilder.Append(Schema->GetBoneReferences(OriginRole)[SchemaOriginBoneIdx].BoneName.ToString());
	}
	else if (SchemaOriginBoneIdx == TrajectorySchemaBoneIdx)
	{
		LabelBuilder.Append(TEXT("_Trj"));
	}
	
	if (OriginRole != DefaultRole)
	{
		LabelBuilder.Append(TEXT("["));
		LabelBuilder.Append(OriginRole.ToString());
		LabelBuilder.Append(TEXT("]"));
	}

	if (PermutationTimeType == EPermutationTimeType::UsePermutationTime)
	{
		LabelBuilder.Append(TEXT("_PT"));
	}
	else if (PermutationTimeType == EPermutationTimeType::UseSampleToPermutationTime)
	{
		LabelBuilder.Append(TEXT("_SPT"));
	}
	
	AppendLabelSeparator(LabelBuilder, LabelFormat, true);

	LabelBuilder.Appendf(TEXT("%.2f"), SampleTimeOffset);

	if (!FMath::IsNearlyZero(OriginTimeOffset))
	{
		LabelBuilder.Appendf(TEXT("-%.2f"), OriginTimeOffset);
	}

	return LabelBuilder;
}

USkeleton* UPoseSearchFeatureChannel_Distance::GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle)
{
	// blueprint generated classes don't have a schema, until they're instanced by the schema
	if (const UPoseSearchSchema* Schema = GetSchema())
	{
		bInvalidSkeletonIsError = false;
		if (PropertyHandle)
		{
			if (PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UPoseSearchFeatureChannel_Distance, Bone))
			{
				return Schema->GetSkeleton(SampleRole);
			}
			if (PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UPoseSearchFeatureChannel_Distance, OriginBone))
			{
				return Schema->GetSkeleton(OriginRole);
			}
		}
	}

	return Super::GetSkeleton(bInvalidSkeletonIsError, PropertyHandle);
}
#endif
