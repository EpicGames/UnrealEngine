// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchFeatureChannel_FilterCrashingLegs.h"
#include "PoseSearch/PoseSearchAssetIndexer.h"
#include "PoseSearch/PoseSearchAssetSampler.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearch/PoseSearchFeatureChannel_Position.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchFeatureChannel_FilterCrashingLegs)

static void ComputeThighsSideAndForward(const FVector& RightThighPos, const FVector& LeftThighPos, FVector& ThighsSide, FVector& ThighsForward)
{
	ThighsSide = (RightThighPos - LeftThighPos).GetSafeNormal2D(UE_KINDA_SMALL_NUMBER, FVector::LeftVector);
	ThighsForward = (ThighsSide ^ FVector::UpVector).GetSafeNormal2D(UE_KINDA_SMALL_NUMBER, FVector::ForwardVector);
}

static float ComputeCrashingLegsValue(const FVector& RightThighPos, const FVector& LeftThighPos, const FVector& RightFootPos, const FVector& LeftFootPos)
{
	FVector ThighsSide;
	FVector ThighsForward;
	ComputeThighsSideAndForward(RightThighPos, LeftThighPos, ThighsSide, ThighsForward);

	const FVector FeetDir = (RightFootPos - LeftFootPos).GetSafeNormal2D(UE_KINDA_SMALL_NUMBER, FVector::LeftVector);
	const float SideDot = FeetDir | ThighsSide;
	const float ForwardDot = FeetDir | ThighsForward;
	const float CrashingLegsValue = FMath::Atan2(ForwardDot, SideDot) / UE_PI;
	return CrashingLegsValue;
}

bool UPoseSearchFeatureChannel_FilterCrashingLegs::Finalize(UPoseSearchSchema* Schema)
{
	using namespace UE::PoseSearch;

	ChannelDataOffset = Schema->SchemaCardinality;
	ChannelCardinality = 1;
	Schema->SchemaCardinality += ChannelCardinality;

	LeftThighIdx = Schema->AddBoneReference(LeftThigh, SampleRole);
	RightThighIdx = Schema->AddBoneReference(RightThigh, SampleRole);
	LeftFootIdx = Schema->AddBoneReference(LeftFoot, SampleRole);
	RightFootIdx = Schema->AddBoneReference(RightFoot, SampleRole);

	return LeftThighIdx != InvalidSchemaBoneIdx && RightThighIdx != InvalidSchemaBoneIdx && LeftFootIdx != InvalidSchemaBoneIdx && RightFootIdx != InvalidSchemaBoneIdx;
}

void UPoseSearchFeatureChannel_FilterCrashingLegs::AddDependentChannels(UPoseSearchSchema* Schema) const
{
	if (Schema->bInjectAdditionalDebugChannels)
	{
		UPoseSearchFeatureChannel_Position::FindOrAddToSchema(Schema, 0.f, LeftThigh.BoneName, SampleRole);
		UPoseSearchFeatureChannel_Position::FindOrAddToSchema(Schema, 0.f, RightThigh.BoneName, SampleRole);
		UPoseSearchFeatureChannel_Position::FindOrAddToSchema(Schema, 0.f, LeftFoot.BoneName, SampleRole);
		UPoseSearchFeatureChannel_Position::FindOrAddToSchema(Schema, 0.f, RightFoot.BoneName, SampleRole);
	}
}

void UPoseSearchFeatureChannel_FilterCrashingLegs::BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const
{
	using namespace UE::PoseSearch;

	// trying to get the BuildQuery data from another schema UPoseSearchFeatureChannel_FilterCrashingLegs already cached in the SearchContext
	if (SearchContext.IsUseCachedChannelData())
	{
		// composing a unique identifier to specify this channel with all the required properties to be able to share the query data with other channels of the same type
		uint32 UniqueIdentifier = GetClass()->GetUniqueID();
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SampleRole));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(LeftThighIdx));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(RightThighIdx));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(LeftFootIdx));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(RightFootIdx));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(InputQueryPose));

		TConstArrayView<float> CachedChannelData;
		if (const UPoseSearchFeatureChannel* CachedChannel = SearchContext.GetCachedChannelData(UniqueIdentifier, this, CachedChannelData))
		{
#if DO_CHECK
			const UPoseSearchFeatureChannel_FilterCrashingLegs* CachedFilterCrashingLegs = Cast<UPoseSearchFeatureChannel_FilterCrashingLegs>(CachedChannel);
			check(CachedFilterCrashingLegs);
			check(CachedFilterCrashingLegs->GetChannelCardinality() == ChannelCardinality);
			check(CachedChannelData.Num() == ChannelCardinality);

			// making sure there were no hash collisions
			check(CachedFilterCrashingLegs->SampleRole == SampleRole);
			check(CachedFilterCrashingLegs->LeftThighIdx == LeftThighIdx);
			check(CachedFilterCrashingLegs->RightThighIdx == RightThighIdx);
			check(CachedFilterCrashingLegs->LeftFootIdx == LeftFootIdx);
			check(CachedFilterCrashingLegs->RightFootIdx == RightFootIdx);
			check(CachedFilterCrashingLegs->InputQueryPose == InputQueryPose);
#endif //DO_CHECK

			// copying the CachedChannelData into this channel portion of the FeatureVectorBuilder
			FFeatureVectorHelper::Copy(SearchContext.EditFeatureVector().Slice(ChannelDataOffset, ChannelCardinality), 0, ChannelCardinality, CachedChannelData);
			return;
		}
	}

	// trying to get the BuildQuery data from the continuing pose
	const bool bCanUseContinuingPoseValues = SearchContext.CanUseContinuingPoseValues();
	const bool bSkip = InputQueryPose != EInputQueryPose::UseCharacterPose && bCanUseContinuingPoseValues;
	if (bSkip)
	{
		if (bCanUseContinuingPoseValues)
		{
			FFeatureVectorHelper::Copy(SearchContext.EditFeatureVector(), ChannelDataOffset, ChannelCardinality, SearchContext.GetContinuingPoseValues());
		}
		return;
	}

	// composing the BuildQuery data from the bones
	const FVector RightThighPosition = SearchContext.GetSamplePosition(0.f, 0.f, RightThighIdx, RootSchemaBoneIdx, SampleRole, SampleRole);
	const FVector LeftThighPosition = SearchContext.GetSamplePosition(0.f, 0.f, LeftThighIdx, RootSchemaBoneIdx, SampleRole, SampleRole);
	const FVector RightFootPosition = SearchContext.GetSamplePosition(0.f, 0.f, RightFootIdx, RootSchemaBoneIdx, SampleRole, SampleRole);
	const FVector LeftFootPosition = SearchContext.GetSamplePosition(0.f, 0.f, LeftFootIdx, RootSchemaBoneIdx, SampleRole, SampleRole);

	const float CrashingLegsValue = ComputeCrashingLegsValue(RightThighPosition, LeftThighPosition, RightFootPosition, LeftFootPosition);

	FFeatureVectorHelper::EncodeFloat(SearchContext.EditFeatureVector(), ChannelDataOffset, CrashingLegsValue);
}

#if ENABLE_DRAW_DEBUG
void UPoseSearchFeatureChannel_FilterCrashingLegs::DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseVector) const
{
	using namespace UE::PoseSearch;

	if (DrawParams.IsAnyWeightRelevant(this))
	{
		Super::DebugDraw(DrawParams, PoseVector);

		const float CrashingLegsValue = FFeatureVectorHelper::DecodeFloat(PoseVector, ChannelDataOffset);

		const FVector LeftThighPosition = DrawParams.ExtractPosition(PoseVector, 0.f, LeftThighIdx, SampleRole);
		const FVector RightThighPosition = DrawParams.ExtractPosition(PoseVector, 0.f, RightThighIdx, SampleRole);
		const FVector LeftFootPosition = DrawParams.ExtractPosition(PoseVector, 0.f, LeftFootIdx, SampleRole);
		const FVector RightFootPosition = DrawParams.ExtractPosition(PoseVector, 0.f, RightFootIdx, SampleRole);

		const float FeetDistance = (RightFootPosition - LeftFootPosition).Length();

		FVector ThighsSide;
		FVector ThighsForward;
		ComputeThighsSideAndForward(RightThighPosition, LeftThighPosition, ThighsSide, ThighsForward);
		const FVector CrossingLegsVector = (FMath::Cos(CrashingLegsValue * UE_PI) * ThighsSide + FMath::Sin(CrashingLegsValue * UE_PI) * ThighsForward) * FeetDistance;

		auto LerpColor = [](FColor A, FColor B, float T) -> FColor
			{
				return FColor(
					FMath::RoundToInt(float(A.R) * (1.f - T) + float(B.R) * T),
					FMath::RoundToInt(float(A.G) * (1.f - T) + float(B.G) * T),
					FMath::RoundToInt(float(A.B) * (1.f - T) + float(B.B) * T),
					FMath::RoundToInt(float(A.A) * (1.f - T) + float(B.A) * T));
			};

		// when CrossingLegsValue is greater than .5 or lesser than -.5 we draw in Red or Orange
		const float LerpValue = FMath::Min(CrashingLegsValue * 2.f, 1.f);
		const FColor Color = LerpValue >= 0.f ? LerpColor(FColor::Green, FColor::Red, LerpValue) : LerpColor(FColor::Green, FColor::Orange, -LerpValue);

		DrawParams.DrawLine(LeftFootPosition, LeftFootPosition + CrossingLegsVector, Color);
		DrawParams.DrawLine(RightFootPosition, RightFootPosition - CrossingLegsVector, Color);
	}
}
#endif // ENABLE_DRAW_DEBUG

// @todo: reimplement this using the new FPoseSearchFilter
// IPoseSearchFilter interface
bool UPoseSearchFeatureChannel_FilterCrashingLegs::IsFilterActive() const
{
	return AllowedTolerance != 0.f;
}

bool UPoseSearchFeatureChannel_FilterCrashingLegs::IsFilterValid(TConstArrayView<float> PoseValues, TConstArrayView<float> QueryValues, int32 PoseIdx, const UE::PoseSearch::FPoseMetadata& Metadata) const
{
	using namespace UE::PoseSearch;

	const float PoseValue = FFeatureVectorHelper::DecodeFloat(PoseValues, ChannelDataOffset);
	const float QueryValue = FFeatureVectorHelper::DecodeFloat(QueryValues, ChannelDataOffset);

	bool bIsPoseValid = true;
	if (FMath::Abs(QueryValue - PoseValue) > AllowedTolerance)
	{
		bIsPoseValid = false;
	}

	return bIsPoseValid;
}

#if WITH_EDITOR
void UPoseSearchFeatureChannel_FilterCrashingLegs::FillWeights(TArrayView<float> Weights) const
{
	for (int32 i = 0; i < ChannelCardinality; ++i)
	{
		Weights[ChannelDataOffset + i] = Weight;
	}
}

bool UPoseSearchFeatureChannel_FilterCrashingLegs::IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const
{
	using namespace UE::PoseSearch;
	FVector RightThighPosition, LeftThighPosition, RightFootPosition, LeftFootPosition;
	for (int32 SampleIdx = Indexer.GetBeginSampleIdx(); SampleIdx != Indexer.GetEndSampleIdx(); ++SampleIdx)
	{
		if (Indexer.GetSamplePosition(RightThighPosition, 0.f, 0.f, SampleIdx, RightThighIdx, RootSchemaBoneIdx, SampleRole, SampleRole) &&
			Indexer.GetSamplePosition(LeftThighPosition, 0.f, 0.f, SampleIdx, LeftThighIdx, RootSchemaBoneIdx, SampleRole, SampleRole) &&
			Indexer.GetSamplePosition(RightFootPosition, 0.f, 0.f, SampleIdx, RightFootIdx, RootSchemaBoneIdx, SampleRole, SampleRole) &&
			Indexer.GetSamplePosition(LeftFootPosition, 0.f, 0.f, SampleIdx, LeftFootIdx, RootSchemaBoneIdx, SampleRole, SampleRole))
		{
			const float CrashingLegsValue = ComputeCrashingLegsValue(RightThighPosition, LeftThighPosition, RightFootPosition, LeftFootPosition);
			FFeatureVectorHelper::EncodeFloat(Indexer.GetPoseVector(SampleIdx), ChannelDataOffset, CrashingLegsValue);
		}
		else
		{
			return false;
		}
	}
	return true;
}

#endif