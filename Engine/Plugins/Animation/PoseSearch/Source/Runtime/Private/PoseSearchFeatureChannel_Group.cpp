// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchFeatureChannel_Group.h"
#include "PoseSearch/PoseSearchSchema.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchFeatureChannel_Group)

bool UPoseSearchFeatureChannel_GroupBase::Finalize(UPoseSearchSchema* Schema)
{
	ChannelDataOffset = Schema->SchemaCardinality;
	for (TObjectPtr<UPoseSearchFeatureChannel>& SubChannelPtr : GetSubChannels())
	{
		if (UPoseSearchFeatureChannel* SubChannel = SubChannelPtr.Get())
		{
			if (!SubChannel->Finalize(Schema))
			{
				return false;
			}
		}
	}
	ChannelCardinality = Schema->SchemaCardinality - ChannelDataOffset;
	return true;
}

void UPoseSearchFeatureChannel_GroupBase::AddDependentChannels(UPoseSearchSchema* Schema) const
{
	for (const TObjectPtr<UPoseSearchFeatureChannel>& SubChannelPtr : GetSubChannels())
	{
		if (SubChannelPtr)
		{
			SubChannelPtr->AddDependentChannels(Schema);
		}
	}
}

#if WITH_EDITOR
void UPoseSearchFeatureChannel_GroupBase::FillWeights(TArrayView<float> Weights) const
{
	for (const TObjectPtr<UPoseSearchFeatureChannel>& SubChannelPtr : GetSubChannels())
	{
		if (const UPoseSearchFeatureChannel* SubChannel = SubChannelPtr.Get())
		{
			SubChannel->FillWeights(Weights);
		}
	}
}

bool UPoseSearchFeatureChannel_GroupBase::IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const
{
	for (const TObjectPtr<UPoseSearchFeatureChannel>& SubChannelPtr : GetSubChannels())
	{
		if (const UPoseSearchFeatureChannel* SubChannel = SubChannelPtr.Get())
		{
			if (!SubChannel->IndexAsset(Indexer))
			{
				return false;
			}
		}
	}
	return true;
}
#endif // WITH_EDITOR

void UPoseSearchFeatureChannel_GroupBase::BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const
{
	for (const TObjectPtr<UPoseSearchFeatureChannel>& SubChannelPtr : GetSubChannels())
	{
		if (const UPoseSearchFeatureChannel* SubChannel = SubChannelPtr.Get())
		{
			SubChannel->BuildQuery(SearchContext);
		}
	}
}

#if ENABLE_DRAW_DEBUG
void UPoseSearchFeatureChannel_GroupBase::DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseVector) const
{
	Super::DebugDraw(DrawParams, PoseVector);

	for (const TObjectPtr<UPoseSearchFeatureChannel>& SubChannelPtr : GetSubChannels())
	{
		if (const UPoseSearchFeatureChannel* SubChannel = SubChannelPtr.Get())
		{
			SubChannel->DebugDraw(DrawParams, PoseVector);
		}
	}
}
#endif // ENABLE_DRAW_DEBUG

// IPoseSearchFilter interface
bool UPoseSearchFeatureChannel_GroupBase::IsFilterActive() const
{
	for (const TObjectPtr<UPoseSearchFeatureChannel>& SubChannelPtr : GetSubChannels())
	{
		if (const UPoseSearchFeatureChannel* SubChannel = SubChannelPtr.Get())
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (SubChannel->IsFilterActive())
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				return true;
			}
		}
	}

	return false;
}

bool UPoseSearchFeatureChannel_GroupBase::IsFilterValid(TConstArrayView<float> PoseValues, TConstArrayView<float> QueryValues, int32 PoseIdx, const UE::PoseSearch::FPoseMetadata& Metadata) const
{
	for (const TObjectPtr<UPoseSearchFeatureChannel>& SubChannelPtr : GetSubChannels())
	{
		if (const UPoseSearchFeatureChannel* SubChannel = SubChannelPtr.Get())
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (!SubChannel->IsFilterValid(PoseValues, QueryValues, PoseIdx, Metadata))
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				return false;
			}
		}
	}

	return true;
}

#if WITH_EDITOR
UE::PoseSearch::TLabelBuilder& UPoseSearchFeatureChannel_Group::GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat) const
{
	GetOuterLabel(LabelBuilder, LabelFormat);
	AppendLabelSeparator(LabelBuilder, UE::PoseSearch::ELabelFormat::Full_Horizontal);
	LabelBuilder.Append(TEXT("Group"));
	if (DebugWeightGroupID != INDEX_NONE)
	{
		LabelBuilder.Appendf(TEXT("_%d"), DebugWeightGroupID);
	}
	return LabelBuilder;
}
#endif // WITH_EDITOR
