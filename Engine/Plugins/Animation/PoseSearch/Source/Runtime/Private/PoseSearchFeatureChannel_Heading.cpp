// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchFeatureChannel_Heading.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "PoseSearch/PoseSearchAssetIndexer.h"
#include "PoseSearch/PoseSearchAssetSampler.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearch/PoseSearchFeatureChannel_Position.h"

#if WITH_EDITOR
#include "PropertyHandle.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchFeatureChannel_Heading)

UPoseSearchFeatureChannel_Heading::UPoseSearchFeatureChannel_Heading()
{
	bUseBlueprintQueryOverride = Cast<UBlueprintGeneratedClass>(GetClass()) != nullptr;
}

void UPoseSearchFeatureChannel_Heading::FindOrAddToSchema(UPoseSearchSchema* Schema, float SampleTimeOffset, const FName& BoneName, const UE::PoseSearch::FRole& Role, EHeadingAxis HeadingAxis, EPermutationTimeType PermutationTimeType, bool bDefaultWithRootBone)
{
	using namespace UE::PoseSearch;

	check(Schema);
	const int8 SchemaBoneIdx = Schema->FindSchemaBoneIdx(BoneName, Role, bDefaultWithRootBone);
	if (SchemaBoneIdx == InvalidSchemaBoneIdx)
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchFeatureChannel_Heading::FindOrAddToSchema: couldn't find SchemaBoneIdx for Bone '%ls' with Role '%ls' in UPoseSearchSchema '%ls'", *BoneName.ToString(), *Role.ToString(), *GetNameSafe(Schema));
		return;
	}

	if (!Schema->FindChannel([SchemaBoneIdx, SampleTimeOffset, Role, HeadingAxis, PermutationTimeType](const UPoseSearchFeatureChannel* Channel) -> const UPoseSearchFeatureChannel_Heading*
		{
			if (const UPoseSearchFeatureChannel_Heading* Heading = Cast<UPoseSearchFeatureChannel_Heading>(Channel))
			{
				if (Heading->SchemaBoneIdx == SchemaBoneIdx &&
					Heading->SchemaOriginBoneIdx == RootSchemaBoneIdx &&
					Heading->SampleTimeOffset == SampleTimeOffset &&
					Heading->OriginTimeOffset == 0.f &&
					Heading->HeadingAxis == HeadingAxis &&
					Heading->PermutationTimeType == PermutationTimeType &&
					Heading->SampleRole == Role &&
					Heading->OriginRole == Role &&
					!Heading->bDefaultWithRootBone)
				{
					return Heading;
				}
			}
			return nullptr;
		}))
	{
		const TConstArrayView<FBoneReference> BoneReferences = Schema->GetBoneReferences(Role);

		UPoseSearchFeatureChannel_Heading* Heading = NewObject<UPoseSearchFeatureChannel_Heading>(Schema, NAME_None, RF_Transient);
		Heading->Bone.BoneName = BoneReferences.IsValidIndex(SchemaBoneIdx) ? BoneReferences[SchemaBoneIdx].BoneName : NAME_None;
		Heading->SampleRole = Role;
		Heading->OriginBone.BoneName = BoneReferences.IsValidIndex(RootSchemaBoneIdx) ? BoneReferences[RootSchemaBoneIdx].BoneName : NAME_None;
		Heading->OriginRole = Role;
#if WITH_EDITORONLY_DATA
		Heading->Weight = 0.f;
		Heading->DebugColor = FLinearColor::Gray;
#endif // WITH_EDITORONLY_DATA
		Heading->SampleTimeOffset = SampleTimeOffset;
		Heading->HeadingAxis = HeadingAxis;
		Heading->PermutationTimeType = PermutationTimeType;
		Heading->bDefaultWithRootBone = false;

		Schema->AddTemporaryChannel(Heading);
	}
}

bool UPoseSearchFeatureChannel_Heading::Finalize(UPoseSearchSchema* Schema)
{
	using namespace UE::PoseSearch;

	ChannelDataOffset = Schema->SchemaCardinality;
	ChannelCardinality = UE::PoseSearch::FFeatureVectorHelper::GetVectorCardinality(ComponentStripping);
	Schema->SchemaCardinality += ChannelCardinality;

	SchemaBoneIdx = Schema->AddBoneReference(Bone, SampleRole, bDefaultWithRootBone);
	SchemaOriginBoneIdx = Schema->AddBoneReference(OriginBone, OriginRole, bDefaultWithRootBone);

	return SchemaBoneIdx != InvalidSchemaBoneIdx && SchemaOriginBoneIdx != InvalidSchemaBoneIdx;
}

void UPoseSearchFeatureChannel_Heading::AddDependentChannels(UPoseSearchSchema* Schema) const
{
	if (Schema->bInjectAdditionalDebugChannels)
	{
		const EPermutationTimeType SamplePermutationTimeType = PermutationTimeType != EPermutationTimeType::UseSampleTime ? EPermutationTimeType::UseSampleToPermutationTime : EPermutationTimeType::UseSampleTime;
		UPoseSearchFeatureChannel_Position::FindOrAddToSchema(Schema, SampleTimeOffset, Bone.BoneName, SampleRole, SamplePermutationTimeType, bDefaultWithRootBone);
		
		// injecting 2 Heading channels to be able to reconstruct the Bone rotation
		const EPermutationTimeType OriginPermutationTimeType = PermutationTimeType == EPermutationTimeType::UsePermutationTime ? EPermutationTimeType::UseSampleToPermutationTime : EPermutationTimeType::UseSampleTime;
		UPoseSearchFeatureChannel_Heading::FindOrAddToSchema(Schema, OriginTimeOffset, Bone.BoneName, OriginRole, EHeadingAxis::X, OriginPermutationTimeType, bDefaultWithRootBone);
		UPoseSearchFeatureChannel_Heading::FindOrAddToSchema(Schema, OriginTimeOffset, Bone.BoneName, OriginRole, EHeadingAxis::Y, OriginPermutationTimeType, bDefaultWithRootBone);
	}
}

FVector UPoseSearchFeatureChannel_Heading::GetAxis(const FQuat& Rotation) const
{
	switch (HeadingAxis)
	{
	case EHeadingAxis::X:
		return Rotation.GetAxisX();
	case EHeadingAxis::Y:
		return Rotation.GetAxisY();
	case EHeadingAxis::Z:
		return Rotation.GetAxisZ();
	}

	checkNoEntry();
	return FVector::XAxisVector;
}

FQuat UPoseSearchFeatureChannel_Heading::BP_GetWorldRotationFromContext_Implementation(FChooserEvaluationContext& Context) const
{
	using namespace UE::PoseSearch;

	if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(GetAnimContext(&Context)))
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return BP_GetWorldRotation(AnimInstance);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	UE_LOGF(LogPoseSearch, Warning, "UPoseSearchFeatureChannel_Heading::BP_GetWorldRotationFromContext_Implementation - unsupported null UAnimInstance: WIP support for AnimNext!");
	return FQuat::Identity;
}

void UPoseSearchFeatureChannel_Heading::BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const
{
	using namespace UE::PoseSearch;

	if (bUseBlueprintQueryOverride)
	{
		FChooserEvaluationContext* ChooserContext = SearchContext.GetContext(SampleRole);
		const FQuat BoneRotationWorld = ensure(ChooserContext) ? BP_GetWorldRotationFromContext(*ChooserContext) : FQuat::Identity;
		const FQuat BoneRotation = SearchContext.GetSampleRotation(SampleTimeOffset, OriginTimeOffset, SchemaBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole, EPermutationTimeType::UseSampleTime, &BoneRotationWorld);
		FFeatureVectorHelper::EncodeVector(SearchContext.EditFeatureVector(), ChannelDataOffset, GetAxis(BoneRotation), ComponentStripping, true);
		return;
	}
	
	// trying to get the BuildQuery data from another schema UPoseSearchFeatureChannel_Heading already cached in the SearchContext
	if (SearchContext.IsUseCachedChannelData())
	{
		// composing a unique identifier to specify this channel with all the required properties to be able to share the query data with other channels of the same type
		uint32 UniqueIdentifier = GetClass()->GetUniqueID();
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SampleRole));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(OriginRole));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SamplingAttributeId));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SampleTimeOffset));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(OriginTimeOffset));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(HeadingAxis));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SchemaBoneIdx));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SchemaOriginBoneIdx));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(InputQueryPose));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(ComponentStripping));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(PermutationTimeType));

		TConstArrayView<float> CachedChannelData;
		if (const UPoseSearchFeatureChannel* CachedChannel = SearchContext.GetCachedChannelData(UniqueIdentifier, this, CachedChannelData))
		{
#if DO_CHECK
			const UPoseSearchFeatureChannel_Heading* CachedHeadingChannel = Cast<UPoseSearchFeatureChannel_Heading>(CachedChannel);
			check(CachedHeadingChannel);
			check(CachedHeadingChannel->GetChannelCardinality() == ChannelCardinality);
			check(CachedChannelData.Num() == ChannelCardinality);

			// making sure there were no hash collisions
			check(CachedHeadingChannel->SampleRole == SampleRole);
			check(CachedHeadingChannel->OriginRole == OriginRole);
			check(CachedHeadingChannel->SamplingAttributeId == SamplingAttributeId);
			check(CachedHeadingChannel->SampleTimeOffset == SampleTimeOffset);
			check(CachedHeadingChannel->OriginTimeOffset == OriginTimeOffset);
			check(CachedHeadingChannel->HeadingAxis == HeadingAxis);
			check(CachedHeadingChannel->SchemaBoneIdx == SchemaBoneIdx);
			check(CachedHeadingChannel->SchemaOriginBoneIdx == SchemaOriginBoneIdx);
			check(CachedHeadingChannel->InputQueryPose == InputQueryPose);
			check(CachedHeadingChannel->ComponentStripping == ComponentStripping);
			check(CachedHeadingChannel->PermutationTimeType == PermutationTimeType);
#endif //DO_CHECK

			// copying the CachedChannelData into this channel portion of the FeatureVectorBuilder
			FFeatureVectorHelper::Copy(SearchContext.EditFeatureVector().Slice(ChannelDataOffset, ChannelCardinality), 0, ChannelCardinality, CachedChannelData);
			return;
		}
	}

	const bool bCanUseContinuingPoseValues = SearchContext.CanUseContinuingPoseValues();
	const bool bSkip = InputQueryPose != EInputQueryPose::UseCharacterPose && bCanUseContinuingPoseValues && SampleRole == OriginRole;
	if (bSkip)
	{
		if (bCanUseContinuingPoseValues)
		{
			FFeatureVectorHelper::Copy(SearchContext.EditFeatureVector(), ChannelDataOffset, ChannelCardinality, SearchContext.GetContinuingPoseValues());
		}
		return;
	}
	
	// calculating the BoneRotation in component space for the bone indexed by SchemaBoneIdx
	const FQuat BoneRotation = SearchContext.GetSampleRotation(SampleTimeOffset, OriginTimeOffset, SchemaBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole, PermutationTimeType);
	FFeatureVectorHelper::EncodeVector(SearchContext.EditFeatureVector(), ChannelDataOffset, GetAxis(BoneRotation), ComponentStripping,true);
}

#if ENABLE_DRAW_DEBUG
void UPoseSearchFeatureChannel_Heading::DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseVector) const
{
	using namespace UE::PoseSearch;

	bool bDrawInjectAdditionalDebugChannels = false;
#if WITH_EDITORONLY_DATA
	if (const UPoseSearchSchema* Schema = GetSchema())
	{
		bDrawInjectAdditionalDebugChannels = Schema->bDrawInjectAdditionalDebugChannels;
	}
#endif // WITH_EDITORONLY_DATA

	if (bDrawInjectAdditionalDebugChannels || DrawParams.IsAnyWeightRelevant(this))
	{
		Super::DebugDraw(DrawParams, PoseVector);

		FColor Color;
#if WITH_EDITORONLY_DATA
		Color = DebugColor.ToFColor(true);
#else // WITH_EDITORONLY_DATA
		Color = FColor::White;
#endif // WITH_EDITORONLY_DATA

		float PermutationSampleTimeOffset = 0.f;
		float PermutationOriginTimeOffset = 0.f;
		UPoseSearchFeatureChannel::GetPermutationTimeOffsets(PermutationTimeType, DrawParams.ExtractPermutationTime(PoseVector), PermutationSampleTimeOffset, PermutationOriginTimeOffset);
		const EPermutationTimeType SamplePermutationTimeType = PermutationTimeType != EPermutationTimeType::UseSampleTime ? EPermutationTimeType::UseSampleToPermutationTime : EPermutationTimeType::UseSampleTime;
		const EPermutationTimeType OriginPermutationTimeType = PermutationTimeType == EPermutationTimeType::UsePermutationTime ? EPermutationTimeType::UseSampleToPermutationTime : EPermutationTimeType::UseSampleTime;

		const FVector FeaturesVector = FFeatureVectorHelper::DecodeVector(PoseVector, ChannelDataOffset, ComponentStripping);
		const FVector BoneHeading = DrawParams.ExtractRotation(PoseVector, OriginTimeOffset, SchemaOriginBoneIdx, OriginRole, OriginPermutationTimeType, INDEX_NONE, PermutationOriginTimeOffset).RotateVector(FeaturesVector);
		const FVector BonePos = DrawParams.ExtractPosition(PoseVector, SampleTimeOffset, SchemaBoneIdx, SampleRole, SamplePermutationTimeType, SamplingAttributeId, PermutationSampleTimeOffset);

		DrawParams.DrawPoint(BonePos, Color, 3.f);
		DrawParams.DrawLine(BonePos + BoneHeading * 4.f, BonePos + BoneHeading * 15.f, Color);
	}
}
#endif // ENABLE_DRAW_DEBUG

#if WITH_EDITOR
void UPoseSearchFeatureChannel_Heading::FillWeights(TArrayView<float> Weights) const
{
	for (int32 i = 0; i < ChannelCardinality; ++i)
	{
		Weights[ChannelDataOffset + i] = Weight;
	}
}

bool UPoseSearchFeatureChannel_Heading::IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const
{
	using namespace UE::PoseSearch;

	FQuat SampleRotation = FQuat::Identity;
	for (int32 SampleIdx = Indexer.GetBeginSampleIdx(); SampleIdx != Indexer.GetEndSampleIdx(); ++SampleIdx)
	{
		if (Indexer.GetSampleRotation(SampleRotation, SampleTimeOffset, OriginTimeOffset, SampleIdx, SchemaBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole, PermutationTimeType, SamplingAttributeId))
		{
			FFeatureVectorHelper::EncodeVector(Indexer.GetPoseVector(SampleIdx), ChannelDataOffset, GetAxis(SampleRotation), ComponentStripping, true);
		}
		else
		{
			return false;
		}
	}
	return true;
}

UE::PoseSearch::TLabelBuilder& UPoseSearchFeatureChannel_Heading::GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat) const
{
	using namespace UE::PoseSearch;

	GetOuterLabel(LabelBuilder, LabelFormat);
	AppendLabelSeparator(LabelBuilder, LabelFormat);

	LabelBuilder.Append(TEXT("Head"));
	switch (HeadingAxis)
	{
	case EHeadingAxis::X:
		LabelBuilder.Append(TEXT("X"));
		break;
	case EHeadingAxis::Y:
		LabelBuilder.Append(TEXT("Y"));
		break;
	case EHeadingAxis::Z:
		LabelBuilder.Append(TEXT("Z"));
		break;
	}

	if (ComponentStripping == EComponentStrippingVector::StripXY)
	{
		LabelBuilder.Append(TEXT("_z"));
	}
	else if (ComponentStripping == EComponentStrippingVector::StripZ)
	{
		LabelBuilder.Append(TEXT("_xy"));
	}

	const UPoseSearchSchema* Schema = GetSchema();
	check(Schema);
	if (SchemaBoneIdx > RootSchemaBoneIdx)
	{
		LabelBuilder.Append(TEXT("_"));
		LabelBuilder.Append(Schema->GetBoneReferences(SampleRole)[SchemaBoneIdx].BoneName.ToString());
	}
	else if (SchemaBoneIdx == TrajectorySchemaBoneIdx)
	{
		LabelBuilder.Append(TEXT("_Trj"));
	}

	if (SampleRole != DefaultRole)
	{
		LabelBuilder.Append(TEXT("["));
		LabelBuilder.Append(SampleRole.ToString());
		LabelBuilder.Append(TEXT("]"));
	}

	if (SchemaOriginBoneIdx > RootSchemaBoneIdx)
	{
		LabelBuilder.Append(TEXT("_"));
		LabelBuilder.Append(Schema->GetBoneReferences(OriginRole)[SchemaOriginBoneIdx].BoneName.ToString());
	}
	else if (SchemaOriginBoneIdx == TrajectorySchemaBoneIdx)
	{
		LabelBuilder.Append(TEXT("_Trj"));
	}

	if (OriginRole != DefaultRole)
	{
		LabelBuilder.Append(TEXT("["));
		LabelBuilder.Append(OriginRole.ToString());
		LabelBuilder.Append(TEXT("]"));
	}

	if (PermutationTimeType == EPermutationTimeType::UsePermutationTime)
	{
		LabelBuilder.Append(TEXT("_PT"));
	}
	else if (PermutationTimeType == EPermutationTimeType::UseSampleToPermutationTime)
	{
		LabelBuilder.Append(TEXT("_SPT"));
	}

	AppendLabelSeparator(LabelBuilder, LabelFormat, true);

	LabelBuilder.Appendf(TEXT("%.2f"), SampleTimeOffset);

	if (!FMath::IsNearlyZero(OriginTimeOffset))
	{
		LabelBuilder.Appendf(TEXT("-%.2f"), OriginTimeOffset);
	}

	return LabelBuilder;
}

USkeleton* UPoseSearchFeatureChannel_Heading::GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle)
{
	// blueprint generated classes don't have a schema, until they're instanced by the schema
	if (const UPoseSearchSchema* Schema = GetSchema())
	{
		bInvalidSkeletonIsError = false;
		if (PropertyHandle)
		{
			if (PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UPoseSearchFeatureChannel_Heading, Bone))
			{
				return Schema->GetSkeleton(SampleRole);
			}
			if (PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UPoseSearchFeatureChannel_Heading, OriginBone))
			{
				return Schema->GetSkeleton(OriginRole);
			}
		}
	}

	return Super::GetSkeleton(bInvalidSkeletonIsError, PropertyHandle);
}
#endif
