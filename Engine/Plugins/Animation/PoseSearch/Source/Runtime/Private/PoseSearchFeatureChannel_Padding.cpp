// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchFeatureChannel_Padding.h"
#include "PoseSearch/PoseSearchAssetIndexer.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchSchema.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchFeatureChannel_Padding)

void UPoseSearchFeatureChannel_Padding::AddToSchema(UPoseSearchSchema* Schema, int32 PaddingSize)
{
	UPoseSearchFeatureChannel_Padding* Padding = NewObject<UPoseSearchFeatureChannel_Padding>(Schema, NAME_None, RF_Transient);
	Padding->PaddingSize = PaddingSize;
	Schema->AddTemporaryChannel(Padding);
}

bool UPoseSearchFeatureChannel_Padding::Finalize(UPoseSearchSchema* Schema)
{
	ChannelDataOffset = Schema->SchemaCardinality;
	ChannelCardinality = PaddingSize;
	Schema->SchemaCardinality += ChannelCardinality;
	return true;
}

void UPoseSearchFeatureChannel_Padding::BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const
{
	// nothing to do here
}

#if WITH_EDITOR
void UPoseSearchFeatureChannel_Padding::FillWeights(TArrayView<float> Weights) const
{
	for (int32 i = 0; i < ChannelCardinality; ++i)
	{
		Weights[ChannelDataOffset + i] = 0.f;
	}
}

bool UPoseSearchFeatureChannel_Padding::IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const
{
	UE_LOGF(LogPoseSearch, Log, "UPoseSearchFeatureChannel_Padding::IndexAsset: padding data with '%d' additional floats per pose in UPoseSearchSchema '%ls'", PaddingSize, *GetNameSafe(GetSchema()));
	return true;
}

UE::PoseSearch::TLabelBuilder& UPoseSearchFeatureChannel_Padding::GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat) const
{
	GetOuterLabel(LabelBuilder, LabelFormat);
	AppendLabelSeparator(LabelBuilder, LabelFormat);
	LabelBuilder.Appendf(TEXT("Pad_%d_%d"), ChannelDataOffset, ChannelDataOffset + ChannelCardinality);
	return LabelBuilder;
}
#endif
