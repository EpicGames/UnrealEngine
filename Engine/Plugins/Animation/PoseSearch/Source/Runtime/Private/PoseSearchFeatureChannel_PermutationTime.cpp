// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchFeatureChannel_PermutationTime.h"
#include "PoseSearch/PoseSearchAssetIndexer.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchSchema.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchFeatureChannel_PermutationTime)

void UPoseSearchFeatureChannel_PermutationTime::FindOrAddToSchema(UPoseSearchSchema* Schema)
{
	if (Schema->FindFirstChannelOfType<UPoseSearchFeatureChannel_PermutationTime>() == nullptr)
	{
		UPoseSearchFeatureChannel_PermutationTime* PermutationTime = NewObject<UPoseSearchFeatureChannel_PermutationTime>(Schema, NAME_None, RF_Transient);
#if WITH_EDITORONLY_DATA
		PermutationTime->Weight = 0.f;
#endif // WITH_EDITORONLY_DATA
		Schema->AddTemporaryChannel(PermutationTime);

		UE_LOGF(LogPoseSearch, Warning, "required UPoseSearchFeatureChannel_PermutationTime has been added by the Schema '%ls' with Weight of zero. Explicitly add one of this channel to the Schema to be able to tune its weight", *Schema->GetName());
	}
}

bool UPoseSearchFeatureChannel_PermutationTime::Finalize(UPoseSearchSchema* Schema)
{
	ChannelDataOffset = Schema->SchemaCardinality;
	ChannelCardinality = 1;
	Schema->SchemaCardinality += ChannelCardinality;
	return true;
}

void UPoseSearchFeatureChannel_PermutationTime::BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const
{
	using namespace UE::PoseSearch;
	FFeatureVectorHelper::EncodeFloat(SearchContext.EditFeatureVector(), ChannelDataOffset, SearchContext.GetDesiredPermutationTimeOffset());
}

float UPoseSearchFeatureChannel_PermutationTime::GetPermutationTime(TConstArrayView<float> PoseVector) const
{
	using namespace UE::PoseSearch;
	return FFeatureVectorHelper::DecodeFloat(PoseVector, ChannelDataOffset);
}

#if WITH_EDITOR
void UPoseSearchFeatureChannel_PermutationTime::FillWeights(TArrayView<float> Weights) const
{
	for (int32 i = 0; i < ChannelCardinality; ++i)
	{
		Weights[ChannelDataOffset + i] = Weight;
	}
}

bool UPoseSearchFeatureChannel_PermutationTime::IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const
{
	using namespace UE::PoseSearch;

	const float PermutationTimeOffset = Indexer.CalculatePermutationTimeOffset();
	for (int32 SampleIdx = Indexer.GetBeginSampleIdx(); SampleIdx != Indexer.GetEndSampleIdx(); ++SampleIdx)
	{
		FFeatureVectorHelper::EncodeFloat(Indexer.GetPoseVector(SampleIdx), ChannelDataOffset, PermutationTimeOffset);
	}
	return true;
}

UE::PoseSearch::TLabelBuilder& UPoseSearchFeatureChannel_PermutationTime::GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat) const
{
	GetOuterLabel(LabelBuilder, LabelFormat);
	AppendLabelSeparator(LabelBuilder, LabelFormat);
	LabelBuilder.Append(TEXT("PermTime"));
	return LabelBuilder;
}
#endif
