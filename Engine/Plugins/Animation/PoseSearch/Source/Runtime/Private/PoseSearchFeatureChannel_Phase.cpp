// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchFeatureChannel_Phase.h"
#include "PoseSearch/PoseSearchAssetIndexer.h"
#include "PoseSearch/PoseSearchAssetSampler.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearch/PoseSearchFeatureChannel_Position.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchFeatureChannel_Phase)

#if WITH_EDITOR
namespace UE::PoseSearch
{
	struct LocalMinMax
	{
		enum
		{
			Min,
			Max
		} Type = Min;
		int32 Index = 0;
		float SignalValue = 0.f;
	};

	template <typename T>
	T GetValueAtIndex(int32 Sample, const TArray<T>& Values)
	{
		const int32 Num = Values.Num();
		check(Num > 1);

		if (Sample < 0)
		{
			return (Values[1] - Values[0]) * Sample + Values[0];
		}

		if (Sample < Num)
		{
			return Values[Sample];
		}

		return (Values[Num - 1] - Values[Num - 2]) * (Sample - (Num - 1)) + Values[Num - 1];
	}

	static bool CollectBonePositions(TArray<FVector>& BonePositions, FAssetIndexer& Indexer, int8 SchemaBoneIdx, const FRole& Role)
	{
		const int32 NumSamples = Indexer.GetEndSampleIdx() - Indexer.GetBeginSampleIdx();

		// collecting all the bone transforms
		BonePositions.Reset();
		BonePositions.AddDefaulted(NumSamples);
		for (int32 SampleIdx = 0; SampleIdx != NumSamples; ++SampleIdx)
		{
			if (!Indexer.GetSamplePosition(BonePositions[SampleIdx], 0.f, 0.f, SampleIdx, SchemaBoneIdx, RootSchemaBoneIdx, Role, Role))
			{
				return false;
			}
		}
		return true;
	}

	static void CalculateSignal(const TArray<FVector>& BonePositions, TArray<float>& Signal, int32 offset = 1)
	{
		Signal.Reset();
		Signal.AddDefaulted(BonePositions.Num());

		for (int32 SampleIdx = 0; SampleIdx != BonePositions.Num(); ++SampleIdx)
		{
			Signal[SampleIdx] = (GetValueAtIndex(SampleIdx + offset, BonePositions) - GetValueAtIndex(SampleIdx - offset, BonePositions)).Length();
		}
	}

	static void SmoothSignal(const TArray<float>& Signal, TArray<float>& SmoothedSignal, int32 offset = 1)
	{
		SmoothedSignal.Reset();
		SmoothedSignal.AddDefaulted(Signal.Num());

		for (int32 SampleIdx = -offset; SampleIdx != offset; ++SampleIdx)
		{
			SmoothedSignal[0] += GetValueAtIndex(SampleIdx, Signal);
		}

		for (int32 SampleIdx = 1; SampleIdx != Signal.Num(); ++SampleIdx)
		{
			SmoothedSignal[SampleIdx] = SmoothedSignal[SampleIdx - 1] - GetValueAtIndex(SampleIdx - offset - 1, Signal) + GetValueAtIndex(SampleIdx + offset, Signal);
		}

		for (int32 SampleIdx = 0; SampleIdx != Signal.Num(); ++SampleIdx)
		{
			SmoothedSignal[SampleIdx] /= 2 * offset + 1;
		}
	}

	static void FindLocalMinMax(const TArray<float>& Signal, TArray<LocalMinMax>& MinMax)
	{
		enum SignalState
		{
			Flat,
			Ascending,
			Descending
		};

		MinMax.Reset();
		if (Signal.Num() > 1)
		{
			SignalState State = SignalState::Flat;
			for (int32 SignalIndex = 1; SignalIndex < Signal.Num(); ++SignalIndex)
			{
				const int32 PrevSignalIndex = SignalIndex - 1;
				const float PrevSignalValue = Signal[PrevSignalIndex];
				const float SignalValue = Signal[SignalIndex];

				if (State == SignalState::Flat)
				{
					if (SignalValue > PrevSignalValue)
					{
						State = SignalState::Ascending;
					}
					else if (SignalValue < PrevSignalValue)
					{
						State = SignalState::Descending;
					}
				}
				else if (State == SignalState::Ascending)
				{
					if (SignalValue < PrevSignalValue)
					{
						State = SignalState::Descending;

						LocalMinMax LocalMinMax;
						LocalMinMax.Type = LocalMinMax::Max;
						LocalMinMax.Index = PrevSignalIndex;
						LocalMinMax.SignalValue = Signal[LocalMinMax.Index];

						check(MinMax.IsEmpty() || MinMax.Last().Type != LocalMinMax.Type);
						MinMax.Add(LocalMinMax);
					}
				}
				else // if (State == SignalState::Descending)
				{
					if (SignalValue > PrevSignalValue)
					{
						State = SignalState::Ascending;

						LocalMinMax LocalMinMax;
						LocalMinMax.Type = LocalMinMax::Min;
						LocalMinMax.Index = PrevSignalIndex;
						LocalMinMax.SignalValue = Signal[LocalMinMax.Index];

						check(MinMax.IsEmpty() || MinMax.Last().Type != LocalMinMax.Type);
						MinMax.Add(LocalMinMax);
					}
				}
			}
		}
	}

	static void ExtrapolateLocalMinMaxBoundaries(TArray<LocalMinMax>& MinMax, const TArray<float>& Signal)
	{
		const int32 Num = MinMax.Num();

		check(Signal.Num() > 0);

		LocalMinMax InitialMinMax;
		LocalMinMax FinalMinMax;

		if (Num == 0)
		{
			const bool IsInitialMax = Signal[0] > Signal[Signal.Num() - 1];

			InitialMinMax.Index = 0;
			InitialMinMax.SignalValue = Signal[0];
			InitialMinMax.Type = IsInitialMax ? LocalMinMax::Max : LocalMinMax::Min;

			FinalMinMax.Index = Signal.Num() - 1;
			FinalMinMax.SignalValue = Signal[Signal.Num() - 1];
			FinalMinMax.Type = IsInitialMax ? LocalMinMax::Min : LocalMinMax::Max;

			MinMax.Add(InitialMinMax);
			MinMax.Add(FinalMinMax);
		}
		else
		{
			int32 InitialDelta = 0;
			int32 FinalDelta = 0;
			if (Num > 2)
			{
				InitialDelta = MinMax[2].Index - MinMax[1].Index;
				FinalDelta = MinMax[Num - 2].Index - MinMax[Num - 3].Index;
			}
			else if (Num > 1)
			{
				InitialDelta = MinMax[1].Index - MinMax[0].Index;
				FinalDelta = MinMax[Num - 1].Index - MinMax[Num - 2].Index;
			}
			else
			{
				InitialDelta = MinMax[0].Index;
				FinalDelta = (Signal.Num() - 1) - MinMax[0].Index;
			}

			InitialMinMax.SignalValue = Num > 1 ? MinMax[1].SignalValue : Signal[0];
			InitialMinMax.Type = MinMax[0].Type == LocalMinMax::Min ? LocalMinMax::Max : LocalMinMax::Min;
			InitialMinMax.Index = FMath::Min(MinMax[0].Index - InitialDelta, 0);

			FinalMinMax.SignalValue = Num > 1 ? MinMax[Num - 2].SignalValue : Signal[Signal.Num() - 1];
			FinalMinMax.Type = MinMax[Num - 1].Type == LocalMinMax::Min ? LocalMinMax::Max : LocalMinMax::Min;
			FinalMinMax.Index = FMath::Max(MinMax[Num - 1].Index + FinalDelta, Signal.Num() - 1);

			// there's no point in adding an InitialMinMax if the first MinMax is at the first frame of the signal
			if (MinMax[0].Index > 0)
			{
				MinMax.Insert(InitialMinMax, 0);
			}

			// there's no point in adding a FinalMinMax if the last MinMax is at the last frame of the signal
			if (MinMax[Num - 1].Index < Signal.Num() - 1)
			{
				MinMax.Add(FinalMinMax);
			}
		}
	}

	static void ValidateLocalMinMax(const TArray<LocalMinMax>& MinMax)
	{
		for (int32 i = 1; i < MinMax.Num(); ++i)
		{
			check(MinMax[i].Type != MinMax[i - 1].Type);
			check(MinMax[i].Index > MinMax[i - 1].Index);
			if (MinMax[i].Type == LocalMinMax::Min)
			{
				check(MinMax[i].SignalValue <= MinMax[i - 1].SignalValue);
			}
			else
			{
				check(MinMax[i].SignalValue >= MinMax[i - 1].SignalValue);
			}
		}
	}

	static void CalculatePhaseAndCertainty(int32 Index, const TArray<LocalMinMax>& MinMax, int32 SignalSize, float& Phase, float& Certainty)
	{
		// @todo: expose them via UI
		static float CertaintyMin = 1.f;
		static float CertaintyMult = 0.1f;

		const int32 LastIndex = MinMax.Num() - 1;
		for (int32 i = 1; i < MinMax.Num(); ++i)
		{
			const int32 MinMaxIndex = MinMax[i].Index;
			if (Index < MinMaxIndex)
			{
				const int32 PrevMinMaxIndex = MinMax[i - 1].Index;
				check(MinMaxIndex > PrevMinMaxIndex);
				const float Ratio = static_cast<float>((Index - PrevMinMaxIndex)) / static_cast<float>((MinMaxIndex - PrevMinMaxIndex));
				const float PhaseOffset = MinMax[i - 1].Type == LocalMinMax::Min ? 0.f : 0.5f;
				Phase = PhaseOffset + Ratio * 0.5f;

				const float DeltaSignalValue = FMath::Abs(MinMax[i - 1].SignalValue - MinMax[i].SignalValue);
				const float NextDeltaSignalValue = i < LastIndex ? FMath::Abs(MinMax[i].SignalValue - MinMax[i + 1].SignalValue) : DeltaSignalValue;
				Certainty = CertaintyMin + (DeltaSignalValue * (1.f - Ratio) + NextDeltaSignalValue * Ratio) * CertaintyMult;
				return;
			}
		}

		Phase = MinMax[LastIndex].Type == LocalMinMax::Min ? 0.f : 0.5f;
		Certainty = CertaintyMin + (LastIndex > 0 ? FMath::Abs(MinMax[LastIndex].SignalValue - MinMax[LastIndex - 1].SignalValue) : 0.f) * CertaintyMult;
	}

	static void CalculatePhasesFromLocalMinMax(const TArray<LocalMinMax>& MinMax, TArray<FVector2D>& Phases, int32 SignalSize)
	{
		Phases.Reset();
		Phases.AddDefaulted(SignalSize);

		float Certainty = 1.f;
		float Phase = 0.f;
		for (int32 i = 0; i < SignalSize; ++i)
		{
			CalculatePhaseAndCertainty(i, MinMax, SignalSize, Phase, Certainty);
			FMath::SinCos(&Phases[i].X, &Phases[i].Y, Phase * TWO_PI);
			Phases[i] *= Certainty;
		}
	}

} // namespace UE::PoseSearch
#endif // WITH_EDITOR

bool UPoseSearchFeatureChannel_Phase::Finalize(UPoseSearchSchema* Schema)
{
	using namespace UE::PoseSearch;

	ChannelDataOffset = Schema->SchemaCardinality;
	ChannelCardinality = 2;
	Schema->SchemaCardinality += ChannelCardinality;

	SchemaBoneIdx = Schema->AddBoneReference(Bone, SampleRole);

	return SchemaBoneIdx != InvalidSchemaBoneIdx;
}

void UPoseSearchFeatureChannel_Phase::AddDependentChannels(UPoseSearchSchema* Schema) const
{
	if (Schema->bInjectAdditionalDebugChannels)
	{
		UPoseSearchFeatureChannel_Position::FindOrAddToSchema(Schema, 0.f, Bone.BoneName, SampleRole);
	}
}

void UPoseSearchFeatureChannel_Phase::BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const
{
	using namespace UE::PoseSearch;

	// trying to get the BuildQuery data from another schema UPoseSearchFeatureChannel_Phase already cached in the SearchContext
	if (SearchContext.IsUseCachedChannelData())
	{
		// composing a unique identifier to specify this channel with all the required properties to be able to share the query data with other channels of the same type
		uint32 UniqueIdentifier = GetClass()->GetUniqueID();
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SchemaBoneIdx));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(InputQueryPose));

		TConstArrayView<float> CachedChannelData;
		if (const UPoseSearchFeatureChannel* CachedChannel = SearchContext.GetCachedChannelData(UniqueIdentifier, this, CachedChannelData))
		{
#if DO_CHECK
			const UPoseSearchFeatureChannel_Phase* CachedPhaseChannel = Cast<UPoseSearchFeatureChannel_Phase>(CachedChannel);
			check(CachedPhaseChannel);
			check(CachedPhaseChannel->GetChannelCardinality() == ChannelCardinality);
			check(CachedChannelData.Num() == ChannelCardinality);

			// making sure there were no hash collisions
			check(CachedPhaseChannel->SchemaBoneIdx == SchemaBoneIdx);
			check(CachedPhaseChannel->InputQueryPose == InputQueryPose);
#endif //DO_CHECK

			// copying the CachedChannelData into this channel portion of the FeatureVectorBuilder
			FFeatureVectorHelper::Copy(SearchContext.EditFeatureVector().Slice(ChannelDataOffset, ChannelCardinality), 0, ChannelCardinality, CachedChannelData);
			return;
		}
	}

	const bool bCanUseContinuingPoseValues = SearchContext.CanUseContinuingPoseValues();
	const bool bSkip = InputQueryPose != EInputQueryPose::UseCharacterPose && bCanUseContinuingPoseValues;
	if (bSkip)
	{
		if (bCanUseContinuingPoseValues)
		{
			FFeatureVectorHelper::Copy(SearchContext.EditFeatureVector(), ChannelDataOffset, ChannelCardinality, SearchContext.GetContinuingPoseValues());
		}
	}
	
	// @todo: Support phase in BuildQuery
	// FFeatureVectorHelper::EncodeVector2D(SearchContext.EditFeatureVector(), DataOffset, ???);
}

#if ENABLE_DRAW_DEBUG
void UPoseSearchFeatureChannel_Phase::DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseVector) const
{
	using namespace UE::PoseSearch;

	static float ScaleFactor = 1.f;
	static int32 Segments = 32;

	if (DrawParams.IsAnyWeightRelevant(this))
	{
		Super::DebugDraw(DrawParams, PoseVector);

		FColor Color;
#if WITH_EDITORONLY_DATA
		Color = DebugColor.ToFColor(true);
#else // WITH_EDITORONLY_DATA
		Color = FColor::Yellow;
#endif // WITH_EDITORONLY_DATA

		const FVector2D Phase = FFeatureVectorHelper::DecodeVector2D(PoseVector, ChannelDataOffset);
		const FVector BonePos = DrawParams.ExtractPosition(PoseVector, 0.f, SchemaBoneIdx, SampleRole);

		const FTransform RootBoneTransform = DrawParams.GetRootBoneTransform(SampleRole);
		const FVector TransformXAxisVector = RootBoneTransform.TransformVector(FVector::XAxisVector);
		const FVector TransformYAxisVector = RootBoneTransform.TransformVector(FVector::YAxisVector);
		const FVector TransformZAxisVector = RootBoneTransform.TransformVector(FVector::ZAxisVector);

		const FVector PhaseVector = (TransformZAxisVector * Phase.X + TransformYAxisVector * Phase.Y) * ScaleFactor;
		DrawParams.DrawLine(BonePos, BonePos + PhaseVector, Color);
		DrawParams.DrawCircle(BonePos, TransformXAxisVector, PhaseVector.Length(), Segments, Color);
	}
}
#endif // ENABLE_DRAW_DEBUG

#if WITH_EDITOR
void UPoseSearchFeatureChannel_Phase::FillWeights(TArrayView<float> Weights) const
{
	for (int32 i = 0; i < ChannelCardinality; ++i)
	{
		Weights[ChannelDataOffset + i] = Weight;
	}
}

bool UPoseSearchFeatureChannel_Phase::IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const
{
	using namespace UE::PoseSearch;

	if (Indexer.GetNumIndexedPoses() <= 1)
	{
		// We must have at least 2 poses in order to index phase properly.
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchFeatureChannel_Phase::IndexAsset - Attempt to index animation asset with single pose is not supported. Asset name: %ls", *Indexer.GetAssetName());
		return false;
	}

	// @todo: do we really need to use double(s) in all this math?
	// @todo: expose them via UI
	static float BoneSamplingCentralDifferencesTime = 0.2f; // seconds
	static float SmoothingWindowTime = 0.3f; // seconds

	const UPoseSearchSchema* Schema = Indexer.GetSchema();

	TArray<FVector2D> Phases;
	TArray<float> Signal;
	TArray<float> SmoothedSignal;
	TArray<LocalMinMax> LocalMinMax;
	TArray<FVector> BonePositions;

	if (!CollectBonePositions(BonePositions, Indexer, SchemaBoneIdx, SampleRole))
	{
		return false;
	}

	// @todo: have different way of calculating signals, for example: height of the bone transform, acceleration, etc?
	const int32 BoneSamplingCentralDifferencesOffset = FMath::Max(FMath::CeilToInt(BoneSamplingCentralDifferencesTime * Schema->SampleRate), 1);
	CalculateSignal(BonePositions, Signal, BoneSamplingCentralDifferencesOffset);

	const int32 SmoothingWindowOffset = FMath::Max(FMath::CeilToInt(SmoothingWindowTime * Schema->SampleRate), 1);
	SmoothSignal(Signal, SmoothedSignal, SmoothingWindowOffset);

	FindLocalMinMax(SmoothedSignal, LocalMinMax);
	ValidateLocalMinMax(LocalMinMax);

	ExtrapolateLocalMinMaxBoundaries(LocalMinMax, SmoothedSignal);
	ValidateLocalMinMax(LocalMinMax);
	CalculatePhasesFromLocalMinMax(LocalMinMax, Phases, SmoothedSignal.Num());

	for (int32 SampleIdx = Indexer.GetBeginSampleIdx(); SampleIdx != Indexer.GetEndSampleIdx(); ++SampleIdx)
	{
		FFeatureVectorHelper::EncodeVector2D(Indexer.GetPoseVector(SampleIdx), ChannelDataOffset, Phases[SampleIdx - Indexer.GetBeginSampleIdx()]);
	}
	return true;
}

UE::PoseSearch::TLabelBuilder& UPoseSearchFeatureChannel_Phase::GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat) const
{
	using namespace UE::PoseSearch;

	GetOuterLabel(LabelBuilder, LabelFormat);
	AppendLabelSeparator(LabelBuilder, LabelFormat);

	LabelBuilder.Append(TEXT("Pha"));

	const UPoseSearchSchema* Schema = GetSchema();
	check(Schema);
	if (SchemaBoneIdx > RootSchemaBoneIdx)
	{
		LabelBuilder.Append(TEXT("_"));
		LabelBuilder.Append(Schema->GetBoneReferences(SampleRole)[SchemaBoneIdx].BoneName.ToString());
	}

	return LabelBuilder;
}
#endif
