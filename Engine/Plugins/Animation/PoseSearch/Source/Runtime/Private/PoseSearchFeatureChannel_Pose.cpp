// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchFeatureChannel_Pose.h"
#include "Animation/Skeleton.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearch/PoseSearchDerivedDataKey.h"
#include "PoseSearch/PoseSearchFeatureChannel_Heading.h"
#include "PoseSearch/PoseSearchFeatureChannel_Phase.h"
#include "PoseSearch/PoseSearchFeatureChannel_Position.h"
#include "PoseSearch/PoseSearchFeatureChannel_Velocity.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchFeatureChannel_Pose)

#define LOCTEXT_NAMESPACE "PoseSearchFeatureChannels"

UPoseSearchFeatureChannel_Pose::UPoseSearchFeatureChannel_Pose()
{
	// defaulting UPoseSearchFeatureChannel_Pose for a meaningful locomotion setup
#if WITH_EDITORONLY_DATA
	Weight = 1.f;
#endif // WITH_EDITORONLY_DATA

	SampledBones.Add(FPoseSearchBone({ {"foot_l"}, int32(EPoseSearchBoneFlags::Position | EPoseSearchBoneFlags::Velocity)
#if WITH_EDITORONLY_DATA
		, FName(), 1.f, FLinearColor::Green
#endif // WITH_EDITORONLY_DATA
		}));

	SampledBones.Add(FPoseSearchBone({ {"foot_r"}, int32(EPoseSearchBoneFlags::Position | EPoseSearchBoneFlags::Velocity)
#if WITH_EDITORONLY_DATA
		, FName(), 1.f, FLinearColor::Green
#endif // WITH_EDITORONLY_DATA
		}));
}

bool UPoseSearchFeatureChannel_Pose::Finalize(UPoseSearchSchema* Schema)
{
	SubChannels.Reset();

	for (int32 ChannelBoneIdx = 0; ChannelBoneIdx != SampledBones.Num(); ++ChannelBoneIdx)
	{
		const FPoseSearchBone& SampledBone = SampledBones[ChannelBoneIdx];
		if (EnumHasAnyFlags(SampledBone.Flags, EPoseSearchBoneFlags::Position))
		{
			UPoseSearchFeatureChannel_Position* Position = NewObject<UPoseSearchFeatureChannel_Position>(this, NAME_None, RF_Transient);
			Position->Bone = SampledBone.Reference;
			Position->SampleRole = SampleRole;
			Position->OriginRole = SampleRole;
#if WITH_EDITORONLY_DATA
			Position->NormalizationGroup = SampledBone.NormalizationGroup;
			Position->Weight = SampledBone.Weight * Weight;
			Position->DebugColor = SampledBone.DebugColor;
#endif // WITH_EDITORONLY_DATA
			Position->SampleTimeOffset = 0.f;
			Position->InputQueryPose = InputQueryPose;
			Position->PermutationTimeType = PermutationTimeType;
			SubChannels.Add(Position);
		}

		if (EnumHasAnyFlags(SampledBone.Flags, EPoseSearchBoneFlags::Rotation))
		{
			UPoseSearchFeatureChannel_Heading* HeadingX = NewObject<UPoseSearchFeatureChannel_Heading>(this, NAME_None, RF_Transient);
			HeadingX->Bone = SampledBone.Reference;
			HeadingX->SampleRole = SampleRole;
			HeadingX->OriginRole = SampleRole;
#if WITH_EDITORONLY_DATA
			HeadingX->NormalizationGroup = SampledBone.NormalizationGroup;
			HeadingX->Weight = SampledBone.Weight * Weight;
			HeadingX->DebugColor = SampledBone.DebugColor;
#endif // WITH_EDITORONLY_DATA
			HeadingX->SampleTimeOffset = 0.f;
			HeadingX->HeadingAxis = EHeadingAxis::X;
			HeadingX->InputQueryPose = InputQueryPose;
			HeadingX->PermutationTimeType = PermutationTimeType;
			SubChannels.Add(HeadingX);

			UPoseSearchFeatureChannel_Heading* HeadingY = NewObject<UPoseSearchFeatureChannel_Heading>(this, NAME_None, RF_Transient);
			HeadingY->Bone = SampledBone.Reference;
			HeadingY->SampleRole = SampleRole;
			HeadingY->OriginRole = SampleRole;
#if WITH_EDITORONLY_DATA
			HeadingY->NormalizationGroup = SampledBone.NormalizationGroup;
			HeadingY->Weight = SampledBone.Weight * Weight;
			HeadingY->DebugColor = SampledBone.DebugColor;
#endif // WITH_EDITORONLY_DATA
			HeadingY->SampleTimeOffset = 0.f;
			HeadingY->HeadingAxis = EHeadingAxis::Y;
			HeadingY->InputQueryPose = InputQueryPose;
			HeadingY->PermutationTimeType = PermutationTimeType;
			SubChannels.Add(HeadingY);
		}

		if (EnumHasAnyFlags(SampledBone.Flags, EPoseSearchBoneFlags::Velocity))
		{
			UPoseSearchFeatureChannel_Velocity* Velocity = NewObject<UPoseSearchFeatureChannel_Velocity>(this, NAME_None, RF_Transient);
			Velocity->Bone = SampledBone.Reference;
			Velocity->SampleRole = SampleRole;
			Velocity->OriginRole = SampleRole;
#if WITH_EDITORONLY_DATA
			Velocity->NormalizationGroup = SampledBone.NormalizationGroup;
			Velocity->Weight = SampledBone.Weight * Weight;
			Velocity->DebugColor = SampledBone.DebugColor;
#endif // WITH_EDITORONLY_DATA
			Velocity->SampleTimeOffset = 0.f;
			Velocity->InputQueryPose = InputQueryPose;
			Velocity->bUseCharacterSpaceVelocities = bUseCharacterSpaceVelocities;
			Velocity->PermutationTimeType = PermutationTimeType;
			SubChannels.Add(Velocity);
		}

		if (EnumHasAnyFlags(SampledBone.Flags, EPoseSearchBoneFlags::Phase))
		{
			UPoseSearchFeatureChannel_Phase* Phase = NewObject<UPoseSearchFeatureChannel_Phase>(this, NAME_None, RF_Transient);
			Phase->Bone = SampledBone.Reference;
			Phase->SampleRole = SampleRole;
#if WITH_EDITORONLY_DATA
			Phase->NormalizationGroup = SampledBone.NormalizationGroup;
			Phase->Weight = SampledBone.Weight * Weight;
			Phase->DebugColor = SampledBone.DebugColor;
#endif // WITH_EDITORONLY_DATA
			Phase->InputQueryPose = InputQueryPose;

			if (PermutationTimeType != EPermutationTimeType::UseSampleTime)
			{
				UE_LOGF(LogPoseSearch, Error, "UPoseSearchFeatureChannel_Pose::Finalize - unsupported PermutationTimeType %d for EPoseSearchBoneFlags::Phase in Schema '%ls'", (int32)PermutationTimeType, *Schema->GetName());
				return false;
			}
			SubChannels.Add(Phase);
		}
	}

	return Super::Finalize(Schema);
}

#if WITH_EDITOR
UE::PoseSearch::TLabelBuilder& UPoseSearchFeatureChannel_Pose::GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat) const
{
	using namespace UE::PoseSearch;

	GetOuterLabel(LabelBuilder, LabelFormat);
	AppendLabelSeparator(LabelBuilder, UE::PoseSearch::ELabelFormat::Full_Horizontal);
	LabelBuilder.Append(TEXT("Pose"));

	if (SampleRole != DefaultRole)
	{
		LabelBuilder.Append(TEXT("["));
		LabelBuilder.Append(SampleRole.ToString());
		LabelBuilder.Append(TEXT("]"));
	}

	if (PermutationTimeType == EPermutationTimeType::UsePermutationTime)
	{
		LabelBuilder.Append(TEXT("_PT"));
	}
	else if (PermutationTimeType == EPermutationTimeType::UseSampleToPermutationTime)
	{
		LabelBuilder.Append(TEXT("_SPT"));
	}

	if (DebugWeightGroupID != INDEX_NONE)
	{
		LabelBuilder.Appendf(TEXT("_%d"), DebugWeightGroupID);
	}
	return LabelBuilder;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
