// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchFeatureChannel_Position.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "PoseSearch/PoseSearchAssetIndexer.h"
#include "PoseSearch/PoseSearchAssetSampler.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchSchema.h"

#if WITH_EDITOR
#include "PropertyHandle.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchFeatureChannel_Position)

UPoseSearchFeatureChannel_Position::UPoseSearchFeatureChannel_Position()
{
	bUseBlueprintQueryOverride = Cast<UBlueprintGeneratedClass>(GetClass()) != nullptr;
}

void UPoseSearchFeatureChannel_Position::FindOrAddToSchema(UPoseSearchSchema* Schema, float SampleTimeOffset, const FName& BoneName, const UE::PoseSearch::FRole& Role, EPermutationTimeType PermutationTimeType, bool bDefaultWithRootBone)
{
	using namespace UE::PoseSearch;

	check(Schema);
	const int8 SchemaBoneIdx = Schema->FindSchemaBoneIdx(BoneName, Role, bDefaultWithRootBone);
	if (SchemaBoneIdx == InvalidSchemaBoneIdx)
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchFeatureChannel_Position::FindOrAddToSchema: couldn't find SchemaBoneIdx for Bone '%ls' with Role '%ls' in UPoseSearchSchema '%ls'", *BoneName.ToString(), *Role.ToString(), *GetNameSafe(Schema));
		return;
	}

	if (!Schema->FindChannel([SchemaBoneIdx, SampleTimeOffset, &Role, PermutationTimeType](const UPoseSearchFeatureChannel* Channel) -> const UPoseSearchFeatureChannel_Position*
		{
			if (const UPoseSearchFeatureChannel_Position* Position = Cast<UPoseSearchFeatureChannel_Position>(Channel))
			{
				if (Position->SchemaBoneIdx == SchemaBoneIdx &&
					Position->SchemaOriginBoneIdx == RootSchemaBoneIdx &&
					Position->SampleTimeOffset == SampleTimeOffset &&
					Position->OriginTimeOffset == 0.f &&
					Position->PermutationTimeType == PermutationTimeType &&
					Position->SampleRole == Role &&
					Position->OriginRole == Role &&
					!Position->bDefaultWithRootBone &&
					!Position->bNormalizeDisplacement)
				{
					return Position;
				}
			}
			return nullptr;
		}))
	{
		const TConstArrayView<FBoneReference> BoneReferences = Schema->GetBoneReferences(Role);

		UPoseSearchFeatureChannel_Position* Position = NewObject<UPoseSearchFeatureChannel_Position>(Schema, NAME_None, RF_Transient);
		Position->Bone.BoneName = BoneReferences.IsValidIndex(SchemaBoneIdx) ? BoneReferences[SchemaBoneIdx].BoneName : NAME_None;
		Position->SampleRole = Role;
		Position->OriginBone.BoneName = BoneReferences.IsValidIndex(RootSchemaBoneIdx) ? BoneReferences[RootSchemaBoneIdx].BoneName : NAME_None;
		Position->OriginRole = Role;
#if WITH_EDITORONLY_DATA
		Position->Weight = 0.f;
		Position->DebugColor = FLinearColor::Gray;
#endif // WITH_EDITORONLY_DATA
		Position->SampleTimeOffset = SampleTimeOffset;
		Position->PermutationTimeType = PermutationTimeType;
		Position->bDefaultWithRootBone = false;

		Schema->AddTemporaryChannel(Position);
	}
}

bool UPoseSearchFeatureChannel_Position::Finalize(UPoseSearchSchema* Schema)
{
	using namespace UE::PoseSearch;

	ChannelDataOffset = Schema->SchemaCardinality;
	ChannelCardinality = UE::PoseSearch::FFeatureVectorHelper::GetVectorCardinality(ComponentStripping);
	Schema->SchemaCardinality += ChannelCardinality;

	SchemaBoneIdx = Schema->AddBoneReference(Bone, SampleRole, bDefaultWithRootBone);
	SchemaOriginBoneIdx = Schema->AddBoneReference(OriginBone, OriginRole, bDefaultWithRootBone);
	
	return SchemaBoneIdx != InvalidSchemaBoneIdx && SchemaOriginBoneIdx != InvalidSchemaBoneIdx;
}

void UPoseSearchFeatureChannel_Position::AddDependentChannels(UPoseSearchSchema* Schema) const
{
	using namespace UE::PoseSearch;

	if (Schema->bInjectAdditionalDebugChannels && (SchemaOriginBoneIdx != RootSchemaBoneIdx || PermutationTimeType == EPermutationTimeType::UsePermutationTime || OriginTimeOffset != 0.f))
	{
		const EPermutationTimeType DependentChannelsPermutationTimeType = PermutationTimeType == EPermutationTimeType::UsePermutationTime ? EPermutationTimeType::UseSampleToPermutationTime : EPermutationTimeType::UseSampleTime;
		UPoseSearchFeatureChannel_Position::FindOrAddToSchema(Schema, OriginTimeOffset, OriginBone.BoneName, OriginRole, DependentChannelsPermutationTimeType, bDefaultWithRootBone);
	}
}

FVector UPoseSearchFeatureChannel_Position::BP_GetWorldPositionFromContext_Implementation(FChooserEvaluationContext& Context) const
{
	using namespace UE::PoseSearch;

	if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(GetAnimContext(&Context)))
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return BP_GetWorldPosition(AnimInstance);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	UE_LOGF(LogPoseSearch, Warning, "UPoseSearchFeatureChannel_Position::BP_GetWorldPositionFromContext_Implementation - unsupported null UAnimInstance: WIP support for AnimNext!");
	return FVector::ZeroVector;
}

void UPoseSearchFeatureChannel_Position::BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const
{
	using namespace UE::PoseSearch;

	if (bUseBlueprintQueryOverride)
	{
		FChooserEvaluationContext* ChooserContext = SearchContext.GetContext(SampleRole);
		const FVector BonePositionWorld = ensure(ChooserContext) ? BP_GetWorldPositionFromContext(*ChooserContext) : FVector::ZeroVector;
		const FVector BonePosition = SearchContext.GetSamplePosition(SampleTimeOffset, OriginTimeOffset, SchemaBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole, PermutationTimeType, &BonePositionWorld);
  		FFeatureVectorHelper::EncodeVector(SearchContext.EditFeatureVector(), ChannelDataOffset, BonePosition, ComponentStripping, false);
		return;
	}

	// trying to get the BuildQuery data from another schema UPoseSearchFeatureChannel_Position already cached in the SearchContext
	if (SearchContext.IsUseCachedChannelData())
	{
		// composing a unique identifier to specify this channel with all the required properties to be able to share the query data with other channels of the same type
		uint32 UniqueIdentifier = GetClass()->GetUniqueID();
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SampleRole));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(OriginRole));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SamplingAttributeId));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SampleTimeOffset));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(OriginTimeOffset));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SchemaBoneIdx));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SchemaOriginBoneIdx));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(InputQueryPose));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(ComponentStripping));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(PermutationTimeType));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(bNormalizeDisplacement));

		TConstArrayView<float> CachedChannelData;
		if (const UPoseSearchFeatureChannel* CachedChannel = SearchContext.GetCachedChannelData(UniqueIdentifier, this, CachedChannelData))
		{
#if DO_CHECK
			const UPoseSearchFeatureChannel_Position* CachedPositionChannel = Cast<UPoseSearchFeatureChannel_Position>(CachedChannel);
			check(CachedPositionChannel);
			check(CachedPositionChannel->GetChannelCardinality() == ChannelCardinality);
			check(CachedChannelData.Num() == ChannelCardinality);

			// making sure there were no hash collisions
			check(CachedPositionChannel->SampleRole == SampleRole);
			check(CachedPositionChannel->OriginRole == OriginRole);
			check(CachedPositionChannel->SamplingAttributeId == SamplingAttributeId);
			check(CachedPositionChannel->SampleTimeOffset == SampleTimeOffset);
			check(CachedPositionChannel->OriginTimeOffset == OriginTimeOffset);
			check(CachedPositionChannel->SchemaBoneIdx == SchemaBoneIdx);
			check(CachedPositionChannel->SchemaOriginBoneIdx == SchemaOriginBoneIdx);
			check(CachedPositionChannel->InputQueryPose == InputQueryPose);
			check(CachedPositionChannel->ComponentStripping == ComponentStripping);
			check(CachedPositionChannel->PermutationTimeType == PermutationTimeType);
			check(CachedPositionChannel->bNormalizeDisplacement == bNormalizeDisplacement);
#endif //DO_CHECK

			// copying the CachedChannelData into this channel portion of the FeatureVectorBuilder
			FFeatureVectorHelper::Copy(SearchContext.EditFeatureVector().Slice(ChannelDataOffset, ChannelCardinality), 0, ChannelCardinality, CachedChannelData);
			return;
		}
	}

	const bool bCanUseContinuingPoseValues = SearchContext.CanUseContinuingPoseValues();
	const bool bSkip = InputQueryPose != EInputQueryPose::UseCharacterPose && bCanUseContinuingPoseValues && SampleRole == OriginRole;
	if (bSkip )
	{
		if (bCanUseContinuingPoseValues)
		{
			FFeatureVectorHelper::Copy(SearchContext.EditFeatureVector(), ChannelDataOffset, ChannelCardinality, SearchContext.GetContinuingPoseValues());
		}
		return;
	}
	
	// calculating the BonePosition in root bone space for the bone indexed by SchemaBoneIdx
	FVector BonePosition = SearchContext.GetSamplePosition(SampleTimeOffset, OriginTimeOffset, SchemaBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole, PermutationTimeType);
	if (bNormalizeDisplacement)
	{
		BonePosition = BonePosition.GetClampedToMaxSize(1.f);
	}
	FFeatureVectorHelper::EncodeVector(SearchContext.EditFeatureVector(), ChannelDataOffset, BonePosition, ComponentStripping, false);
}

#if ENABLE_DRAW_DEBUG
void UPoseSearchFeatureChannel_Position::DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseVector) const
{
	using namespace UE::PoseSearch;

	bool bDrawInjectAdditionalDebugChannels = false;
#if WITH_EDITORONLY_DATA
	if (const UPoseSearchSchema* Schema = GetSchema())
	{
		bDrawInjectAdditionalDebugChannels = Schema->bDrawInjectAdditionalDebugChannels;
	}
#endif // WITH_EDITORONLY_DATA

	if (bDrawInjectAdditionalDebugChannels || DrawParams.IsAnyWeightRelevant(this))
	{
		Super::DebugDraw(DrawParams, PoseVector);

		FColor Color;
#if WITH_EDITORONLY_DATA
		Color = DebugColor.ToFColor(true);
#else // WITH_EDITORONLY_DATA
		Color = FColor::Blue;
#endif // WITH_EDITORONLY_DATA

		float PermutationSampleTimeOffset = 0.f;
		float PermutationOriginTimeOffset = 0.f;
		UPoseSearchFeatureChannel::GetPermutationTimeOffsets(PermutationTimeType, DrawParams.ExtractPermutationTime(PoseVector), PermutationSampleTimeOffset, PermutationOriginTimeOffset);
		const EPermutationTimeType OriginPermutationTimeType = PermutationTimeType == EPermutationTimeType::UsePermutationTime ? EPermutationTimeType::UseSampleToPermutationTime : EPermutationTimeType::UseSampleTime;

		const FVector FeaturesVector = FFeatureVectorHelper::DecodeVector(PoseVector, ChannelDataOffset, ComponentStripping);
		const FVector OriginBonePos = DrawParams.ExtractPosition(PoseVector, OriginTimeOffset, SchemaOriginBoneIdx, OriginRole, OriginPermutationTimeType, INDEX_NONE, PermutationOriginTimeOffset);
		const FVector DeltaPos = DrawParams.ExtractRotation(PoseVector, OriginTimeOffset, RootSchemaBoneIdx, OriginRole, OriginPermutationTimeType, INDEX_NONE, PermutationOriginTimeOffset).RotateVector(FeaturesVector);
		const FVector BonePos = OriginBonePos + DeltaPos;

		if (bNormalizeDisplacement)
		{
			static const float NormalizeDisplacementLength = 100.f;
			DrawParams.DrawLine(OriginBonePos, OriginBonePos + DeltaPos * NormalizeDisplacementLength, Color);
		}
		else
		{
			DrawParams.DrawPoint(BonePos, Color);

			const bool bDrawOrigin = !DeltaPos.IsNearlyZero() && (SchemaOriginBoneIdx != RootSchemaBoneIdx || !FMath::IsNearlyZero(OriginTimeOffset) ||
				SampleRole != OriginRole || PermutationTimeType != EPermutationTimeType::UseSampleTime || bUseBlueprintQueryOverride);
			if (bDrawOrigin)
			{
				DrawParams.DrawPoint(OriginBonePos, Color);
				DrawParams.DrawLine(OriginBonePos, BonePos, Color);
			}
		}
	}
}
#endif // ENABLE_DRAW_DEBUG

#if WITH_EDITOR
void UPoseSearchFeatureChannel_Position::FillWeights(TArrayView<float> Weights) const
{
	for (int32 i = 0; i < ChannelCardinality; ++i)
	{
		Weights[ChannelDataOffset + i] = Weight;
	}
}

bool UPoseSearchFeatureChannel_Position::IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const
{
	using namespace UE::PoseSearch;

	FVector BonePosition;
	for (int32 SampleIdx = Indexer.GetBeginSampleIdx(); SampleIdx != Indexer.GetEndSampleIdx(); ++SampleIdx)
	{
		if (Indexer.GetSamplePosition(BonePosition, SampleTimeOffset, OriginTimeOffset, SampleIdx, SchemaBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole, PermutationTimeType, SamplingAttributeId))
		{
			if (bNormalizeDisplacement)
			{
				BonePosition = BonePosition.GetClampedToMaxSize(1.f);
			}
			FFeatureVectorHelper::EncodeVector(Indexer.GetPoseVector(SampleIdx), ChannelDataOffset, BonePosition, ComponentStripping, false);
		}
		else
		{
			return false;
		}
	}
	return true;
}

UE::PoseSearch::TLabelBuilder& UPoseSearchFeatureChannel_Position::GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat) const
{
	using namespace UE::PoseSearch;

	GetOuterLabel(LabelBuilder, LabelFormat);
	AppendLabelSeparator(LabelBuilder, LabelFormat);

	LabelBuilder.Append(TEXT("Pos"));
	if (bNormalizeDisplacement)
	{
		LabelBuilder.Append(TEXT("_ND"));
	}

	if (ComponentStripping == EComponentStrippingVector::StripXY)
	{
		LabelBuilder.Append(TEXT("_z"));
	}
	else if (ComponentStripping == EComponentStrippingVector::StripZ)
	{
		LabelBuilder.Append(TEXT("_xy"));
	}

	const UPoseSearchSchema* Schema = GetSchema();
	check(Schema);
	if (SchemaBoneIdx > RootSchemaBoneIdx)
	{
		LabelBuilder.Append(TEXT("_"));
		LabelBuilder.Append(Schema->GetBoneReferences(SampleRole)[SchemaBoneIdx].BoneName.ToString());
	}
	else if (SchemaBoneIdx == TrajectorySchemaBoneIdx)
	{
		LabelBuilder.Append(TEXT("_Trj"));
	}
	
	if (SampleRole != DefaultRole)
	{
		LabelBuilder.Append(TEXT("["));
		LabelBuilder.Append(SampleRole.ToString());
		LabelBuilder.Append(TEXT("]"));
	}

	if (SchemaOriginBoneIdx > RootSchemaBoneIdx)
	{
		LabelBuilder.Append(TEXT("_"));
		LabelBuilder.Append(Schema->GetBoneReferences(OriginRole)[SchemaOriginBoneIdx].BoneName.ToString());
	}
	else if (SchemaOriginBoneIdx == TrajectorySchemaBoneIdx)
	{
		LabelBuilder.Append(TEXT("_Trj"));
	}
	
	if (OriginRole != DefaultRole)
	{
		LabelBuilder.Append(TEXT("["));
		LabelBuilder.Append(OriginRole.ToString());
		LabelBuilder.Append(TEXT("]"));
	}

	if (PermutationTimeType == EPermutationTimeType::UsePermutationTime)
	{
		LabelBuilder.Append(TEXT("_PT"));
	}
	else if (PermutationTimeType == EPermutationTimeType::UseSampleToPermutationTime)
	{
		LabelBuilder.Append(TEXT("_SPT"));
	}
	
	AppendLabelSeparator(LabelBuilder, LabelFormat, true);

	LabelBuilder.Appendf(TEXT("%.2f"), SampleTimeOffset);

	if (!FMath::IsNearlyZero(OriginTimeOffset))
	{
		LabelBuilder.Appendf(TEXT("-%.2f"), OriginTimeOffset);
	}

	return LabelBuilder;
}

USkeleton* UPoseSearchFeatureChannel_Position::GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle)
{
	// blueprint generated classes don't have a schema, until they're instanced by the schema
	if (const UPoseSearchSchema* Schema = GetSchema())
	{
		bInvalidSkeletonIsError = false;
		if (PropertyHandle)
		{
			if (PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UPoseSearchFeatureChannel_Position, Bone))
			{
				return Schema->GetSkeleton(SampleRole);
			}
			if (PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UPoseSearchFeatureChannel_Position, OriginBone))
			{
				return Schema->GetSkeleton(OriginRole);
			}
		}
	}

	return Super::GetSkeleton(bInvalidSkeletonIsError, PropertyHandle);
}
#endif