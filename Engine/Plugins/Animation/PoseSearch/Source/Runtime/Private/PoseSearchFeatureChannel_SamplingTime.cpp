// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchFeatureChannel_SamplingTime.h"
#include "PoseSearch/PoseSearchAssetIndexer.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchSchema.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchFeatureChannel_SamplingTime)

bool UPoseSearchFeatureChannel_SamplingTime::Finalize(UPoseSearchSchema* Schema)
{
	ChannelDataOffset = Schema->SchemaCardinality;
	ChannelCardinality = 1;
	Schema->SchemaCardinality += ChannelCardinality;
	return true;
}

void UPoseSearchFeatureChannel_SamplingTime::BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const
{
	using namespace UE::PoseSearch;
	FFeatureVectorHelper::EncodeFloat(SearchContext.EditFeatureVector(), ChannelDataOffset, TimeToMatch);
}

#if WITH_EDITOR
void UPoseSearchFeatureChannel_SamplingTime::FillWeights(TArrayView<float> Weights) const
{
	for (int32 i = 0; i < ChannelCardinality; ++i)
	{
		Weights[ChannelDataOffset + i] = Weight;
	}
}

bool UPoseSearchFeatureChannel_SamplingTime::IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const
{
	using namespace UE::PoseSearch;

	for (int32 SampleIdx = Indexer.GetBeginSampleIdx(); SampleIdx != Indexer.GetEndSampleIdx(); ++SampleIdx)
	{
		FFeatureVectorHelper::EncodeFloat(Indexer.GetPoseVector(SampleIdx), ChannelDataOffset, Indexer.CalculateSampleTime(SampleIdx));
	}
	return true;
}

UE::PoseSearch::TLabelBuilder& UPoseSearchFeatureChannel_SamplingTime::GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat) const
{
	GetOuterLabel(LabelBuilder, LabelFormat);
	AppendLabelSeparator(LabelBuilder, LabelFormat);
	LabelBuilder.Append(TEXT("SamplTime"));
	return LabelBuilder;
}
#endif
