// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchFeatureChannel_TimeToEvent.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "PoseSearch/PoseSearchAnimNotifies.h"
#include "PoseSearch/PoseSearchAssetIndexer.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchSchema.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchFeatureChannel_TimeToEvent)

UPoseSearchFeatureChannel_TimeToEvent::UPoseSearchFeatureChannel_TimeToEvent()
{
	bUseBlueprintQueryOverride = Cast<UBlueprintGeneratedClass>(GetClass()) != nullptr;
}

bool UPoseSearchFeatureChannel_TimeToEvent::Finalize(UPoseSearchSchema* Schema)
{
	ChannelDataOffset = Schema->SchemaCardinality;
	ChannelCardinality = 1;
	Schema->SchemaCardinality += ChannelCardinality;
	return true;
}

float UPoseSearchFeatureChannel_TimeToEvent::BP_GetTimeToEventFromContext_Implementation(FChooserEvaluationContext& Context) const
{
	using namespace UE::PoseSearch;

	if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(GetAnimContext(&Context)))
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return BP_GetTimeToEvent(AnimInstance);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	UE_LOGF(LogPoseSearch, Warning, "UPoseSearchFeatureChannel_TimeToEvent::BP_GetTimeToEventFromContext_Implementation - unsupported null UAnimInstance: WIP support for AnimNext!");
	return 0.f;
}

void UPoseSearchFeatureChannel_TimeToEvent::BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const
{
	using namespace UE::PoseSearch;

	if (!bUseBlueprintQueryOverride)
	{
		UE_LOGF(LogPoseSearch, Warning, "UPoseSearchFeatureChannel_TimeToEvent::BuildQuery - UPoseSearchFeatureChannel_TimeToEvent is designed to work only as BluePrint overridden class. The query TimeToEvent value will be defaulted to 0");
		return;
	}

	if (SearchContext.GetContexts().IsEmpty())
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchFeatureChannel_Position::BuildQuery - no provided anim contexts!");
		return;
	}
	
	FChooserEvaluationContext* ChooserContext = SearchContext.GetContexts()[0];
	const float TimeToEvent = ensure(ChooserContext) ? BP_GetTimeToEventFromContext(*ChooserContext) : 0.f;
	FFeatureVectorHelper::EncodeFloat(SearchContext.EditFeatureVector(), ChannelDataOffset, TimeToEvent);
}

#if WITH_EDITOR
void UPoseSearchFeatureChannel_TimeToEvent::FillWeights(TArrayView<float> Weights) const
{
	for (int32 i = 0; i < ChannelCardinality; ++i)
	{
		Weights[ChannelDataOffset + i] = Weight;
	}
}

bool UPoseSearchFeatureChannel_TimeToEvent::IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const
{
	using namespace UE::PoseSearch;

	const FPoseSearchTimedNotifies<UAnimNotifyState_PoseSearchSamplingEvent> TimedNotifies(SamplingAttributeId, Indexer);
	for (int32 SampleIdx = Indexer.GetBeginSampleIdx(); SampleIdx != Indexer.GetEndSampleIdx(); ++SampleIdx)
	{
		const float SampleTime = Indexer.CalculateSampleTime(SampleIdx);
		const float EventTime = TimedNotifies.GetClosestFutureEvent(SampleTime).Time;
		FFeatureVectorHelper::EncodeFloat(Indexer.GetPoseVector(SampleIdx), ChannelDataOffset, EventTime - SampleTime);
	}
	
	return true;
}

UE::PoseSearch::TLabelBuilder& UPoseSearchFeatureChannel_TimeToEvent::GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat) const
{
	GetOuterLabel(LabelBuilder, LabelFormat);
	AppendLabelSeparator(LabelBuilder, LabelFormat);
	LabelBuilder.Appendf(TEXT("TimeToEvent_%d"), SamplingAttributeId);
	return LabelBuilder;
}
#endif
