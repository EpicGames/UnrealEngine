// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchFeatureChannel_Trajectory.h"
#include "Animation/Skeleton.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchDerivedDataKey.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearch/PoseSearchFeatureChannel_Heading.h"
#include "PoseSearch/PoseSearchFeatureChannel_Phase.h"
#include "PoseSearch/PoseSearchFeatureChannel_Position.h"
#include "PoseSearch/PoseSearchFeatureChannel_Velocity.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchFeatureChannel_Trajectory)

#define LOCTEXT_NAMESPACE "PoseSearchFeatureChannels"

UPoseSearchFeatureChannel_Trajectory::UPoseSearchFeatureChannel_Trajectory()
{
	// defaulting UPoseSearchFeatureChannel_Trajectory for a meaningful locomotion setup
#if WITH_EDITORONLY_DATA
	Weight = 7.f;
#endif // WITH_EDITORONLY_DATA

	Samples.Add(FPoseSearchTrajectorySample({ -0.4f, int32(EPoseSearchTrajectoryFlags::PositionXY)
#if WITH_EDITORONLY_DATA
		, 0.4f, FName(), FLinearColor::Red
#endif // WITH_EDITORONLY_DATA
		}));

	Samples.Add(FPoseSearchTrajectorySample({ 0.f, int32(EPoseSearchTrajectoryFlags::VelocityXY | EPoseSearchTrajectoryFlags::FacingDirectionXY)
#if WITH_EDITORONLY_DATA
		, 2.f, FName(), FLinearColor::Blue
#endif // WITH_EDITORONLY_DATA
		}));

	Samples.Add(FPoseSearchTrajectorySample({ 0.35f, int32(EPoseSearchTrajectoryFlags::PositionXY | EPoseSearchTrajectoryFlags::FacingDirectionXY)
#if WITH_EDITORONLY_DATA
		, 0.7f, FName(), FLinearColor::Blue
#endif // WITH_EDITORONLY_DATA
		}));

	Samples.Add(FPoseSearchTrajectorySample({ 0.7f, int32(EPoseSearchTrajectoryFlags::VelocityXY | EPoseSearchTrajectoryFlags::PositionXY | EPoseSearchTrajectoryFlags::FacingDirectionXY)
#if WITH_EDITORONLY_DATA
		, 0.5f, FName(), FLinearColor::Blue
#endif // WITH_EDITORONLY_DATA
		}));
}

bool UPoseSearchFeatureChannel_Trajectory::Finalize(UPoseSearchSchema* Schema)
{
	SubChannels.Reset();

	for (const FPoseSearchTrajectorySample& Sample : Samples)
	{
		if (EnumHasAnyFlags(Sample.Flags, EPoseSearchTrajectoryFlags::Position | EPoseSearchTrajectoryFlags::PositionXY))
		{
			UPoseSearchFeatureChannel_Position* Position = NewObject<UPoseSearchFeatureChannel_Position>(this, NAME_None, RF_Transient);
			Position->SampleRole = SampleRole;
			Position->OriginRole = SampleRole;
#if WITH_EDITORONLY_DATA
			Position->Weight = Sample.Weight * Weight;
			Position->NormalizationGroup = Sample.NormalizationGroup;
			Position->DebugColor = Sample.DebugColor;
#endif // WITH_EDITORONLY_DATA
			Position->SampleTimeOffset = Sample.Offset;
			Position->InputQueryPose = EInputQueryPose::UseCharacterPose;
	
			if (EnumHasAnyFlags(Sample.Flags, EPoseSearchTrajectoryFlags::PositionXY))
			{
				Position->ComponentStripping = EComponentStrippingVector::StripZ;
			}
			SubChannels.Add(Position);
		}

		if (EnumHasAnyFlags(Sample.Flags, EPoseSearchTrajectoryFlags::Velocity | EPoseSearchTrajectoryFlags::VelocityXY))
		{
			UPoseSearchFeatureChannel_Velocity* Velocity = NewObject<UPoseSearchFeatureChannel_Velocity>(this, NAME_None, RF_Transient);
			Velocity->SampleRole = SampleRole;
			Velocity->OriginRole = SampleRole;
#if WITH_EDITORONLY_DATA
			Velocity->Weight = Sample.Weight * Weight;
			Velocity->NormalizationGroup = Sample.NormalizationGroup;
			Velocity->DebugColor = Sample.DebugColor;
#endif // WITH_EDITORONLY_DATA
			Velocity->SampleTimeOffset = Sample.Offset;
			Velocity->InputQueryPose = EInputQueryPose::UseCharacterPose;
			Velocity->bUseCharacterSpaceVelocities = false;
			if (EnumHasAnyFlags(Sample.Flags, EPoseSearchTrajectoryFlags::VelocityXY))
			{
				Velocity->ComponentStripping = EComponentStrippingVector::StripZ;
			}
			SubChannels.Add(Velocity);
		}

		if (EnumHasAnyFlags(Sample.Flags, EPoseSearchTrajectoryFlags::VelocityDirection | EPoseSearchTrajectoryFlags::VelocityDirectionXY))
		{
			UPoseSearchFeatureChannel_Velocity* Velocity = NewObject<UPoseSearchFeatureChannel_Velocity>(this, NAME_None, RF_Transient);
			Velocity->SampleRole = SampleRole;
			Velocity->OriginRole = SampleRole;
#if WITH_EDITORONLY_DATA
			Velocity->Weight = Sample.Weight * Weight;
			Velocity->NormalizationGroup = Sample.NormalizationGroup;
			Velocity->DebugColor = Sample.DebugColor;
#endif // WITH_EDITORONLY_DATA
			Velocity->SampleTimeOffset = Sample.Offset;
			Velocity->InputQueryPose = EInputQueryPose::UseCharacterPose;
			Velocity->bUseCharacterSpaceVelocities = false;
			Velocity->bNormalize = true;
			if (EnumHasAnyFlags(Sample.Flags, EPoseSearchTrajectoryFlags::VelocityDirectionXY))
			{
				Velocity->ComponentStripping = EComponentStrippingVector::StripZ;
			}
			SubChannels.Add(Velocity);
		}

		if (EnumHasAnyFlags(Sample.Flags, EPoseSearchTrajectoryFlags::FacingDirection | EPoseSearchTrajectoryFlags::FacingDirectionXY))
		{
			UPoseSearchFeatureChannel_Heading* Heading = NewObject<UPoseSearchFeatureChannel_Heading>(this, NAME_None, RF_Transient);
			Heading->SampleRole = SampleRole;
			Heading->OriginRole = SampleRole;
#if WITH_EDITORONLY_DATA
			Heading->Weight = Sample.Weight * Weight;
			Heading->NormalizationGroup = Sample.NormalizationGroup;
			Heading->DebugColor = Sample.DebugColor;
#endif // WITH_EDITORONLY_DATA
			Heading->SampleTimeOffset = Sample.Offset;
			Heading->InputQueryPose = EInputQueryPose::UseCharacterPose;
			if (EnumHasAnyFlags(Sample.Flags, EPoseSearchTrajectoryFlags::FacingDirectionXY))
			{
				Heading->ComponentStripping = EComponentStrippingVector::StripZ;
			}
			SubChannels.Add(Heading);
		}
	}

	return Super::Finalize(Schema);
}

#if ENABLE_DRAW_DEBUG
void UPoseSearchFeatureChannel_Trajectory::DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseVector) const
{
	using namespace UE::PoseSearch;

	if (DrawParams.IsAnyWeightRelevant(this))
	{
		Super::DebugDraw(DrawParams, PoseVector);

		TArray<const UPoseSearchFeatureChannel_Position*, TInlineAllocator<32>> Positions;
		for (const TObjectPtr<UPoseSearchFeatureChannel>& SubChannelPtr : GetSubChannels())
		{
			if (const UPoseSearchFeatureChannel_Position* Position = Cast<UPoseSearchFeatureChannel_Position>(SubChannelPtr.Get()))
			{
				Positions.Add(Position);
			}
		}

		if (!Positions.IsEmpty())
		{
			Positions.Sort([](const UPoseSearchFeatureChannel_Position& a, const UPoseSearchFeatureChannel_Position& b)
				{
					return a.SampleTimeOffset < b.SampleTimeOffset;
				});

			// big enough negative number that prevents PrevTimeOffset * CurrTimeOffset being infinite (there will never be UPoseSearchFeatureChannel_Trajectory trying to match 1000 seconds in the past)
			float PrevTimeOffset = -1000.f;
			TArray<FVector, TInlineAllocator<32>> TrajSplinePos;
			TArray<FColor, TInlineAllocator<32>> TrajSplineColor;
			for (int32 i = 0; i < Positions.Num(); ++i)
			{
				const float CurrTimeOffset = Positions[i]->SampleTimeOffset;
				FColor Color;
#if WITH_EDITORONLY_DATA
				Color = Positions[i]->DebugColor.ToFColor(true);
#else // WITH_EDITORONLY_DATA
				Color = FColor::Blue;
#endif // WITH_EDITORONLY_DATA

				if (Positions.Num() == 1 || PrevTimeOffset * CurrTimeOffset < UE_KINDA_SMALL_NUMBER)
				{
					// we jumped from negative to positive time offset without having a zero time offset, or we only have one position point. so we add the zero
					TrajSplinePos.Add(DrawParams.ExtractPosition(PoseVector, 0.f, RootSchemaBoneIdx, Positions[i]->OriginRole));
					TrajSplineColor.Add(Color);
				}

				TrajSplinePos.Add(DrawParams.ExtractPosition(PoseVector, CurrTimeOffset, RootSchemaBoneIdx, Positions[i]->OriginRole));
				TrajSplineColor.Add(Color);

				PrevTimeOffset = CurrTimeOffset;
			}

			DrawParams.DrawCentripetalCatmullRomSpline(TrajSplinePos, TrajSplineColor, 0.5f, 8);
		}

		Super::DebugDraw(DrawParams, PoseVector);
	}
}
#endif // ENABLE_DRAW_DEBUG

#if WITH_EDITOR
UE::PoseSearch::TLabelBuilder& UPoseSearchFeatureChannel_Trajectory::GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat) const
{
	using namespace UE::PoseSearch;

	GetOuterLabel(LabelBuilder, LabelFormat);
	AppendLabelSeparator(LabelBuilder, UE::PoseSearch::ELabelFormat::Full_Horizontal);
	LabelBuilder.Append(TEXT("Traj"));
	
	if (SampleRole != DefaultRole)
	{
		LabelBuilder.Append(TEXT("["));
		LabelBuilder.Append(SampleRole.ToString());
		LabelBuilder.Append(TEXT("]"));
	}

	if (DebugWeightGroupID != INDEX_NONE)
	{
		LabelBuilder.Appendf(TEXT("_%d"), DebugWeightGroupID);
	}
	
	return LabelBuilder;
}
#endif // WITH_EDITOR

float UPoseSearchFeatureChannel_Trajectory::GetEstimatedSpeedRatio(TConstArrayView<float> QueryVector, TConstArrayView<float> PoseVector) const
{
	using namespace UE::PoseSearch;

	float EstimatedQuerySpeed = 0.f;
	float EstimatedPoseSpeed = 0.f;

	for (const TObjectPtr<UPoseSearchFeatureChannel>& SubChannelPtr : GetSubChannels())
	{
		if (const UPoseSearchFeatureChannel_Velocity* Velocity = Cast<UPoseSearchFeatureChannel_Velocity>(SubChannelPtr.Get()))
		{
			if (!Velocity->bNormalize)
			{
				const FVector QueryVelocity = FFeatureVectorHelper::DecodeVector(QueryVector, Velocity->GetChannelDataOffset(), Velocity->ComponentStripping);
				const FVector PoseVelocity = FFeatureVectorHelper::DecodeVector(PoseVector, Velocity->GetChannelDataOffset(), Velocity->ComponentStripping);
				EstimatedQuerySpeed += QueryVelocity.Length();
				EstimatedPoseSpeed += PoseVelocity.Length();
			}
		}
	}

	if (EstimatedPoseSpeed > UE_KINDA_SMALL_NUMBER)
	{
		return EstimatedQuerySpeed / EstimatedPoseSpeed;
	}

	return 1.f;
}

FVector UPoseSearchFeatureChannel_Trajectory::GetEstimatedFutureRootMotionVelocity(TConstArrayView<float> PoseVector) const
{
	using namespace UE::PoseSearch;

	float LastSampleTimeOffset = 0.0f;
	int32 LastSampleTimeOffsetIndex = INDEX_NONE;
	FVector OutRootMotionTranslation = FVector::ZeroVector;

	const int32 NumChannels = GetSubChannels().Num();
	for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ++ChannelIdx)
	{
		const TObjectPtr<UPoseSearchFeatureChannel>& SubChannelPtr = GetSubChannels()[ChannelIdx];
		if (const UPoseSearchFeatureChannel_Velocity* Velocity = Cast<UPoseSearchFeatureChannel_Velocity>(SubChannelPtr.Get()))
		{
			if (!Velocity->bNormalize && (Velocity->SampleTimeOffset > LastSampleTimeOffset))
			{
				LastSampleTimeOffset = Velocity->SampleTimeOffset;
				LastSampleTimeOffsetIndex = ChannelIdx;
			}
		}
	}

	if (LastSampleTimeOffsetIndex != INDEX_NONE)
	{
		const TObjectPtr<UPoseSearchFeatureChannel>& SubChannelPtr = GetSubChannels()[LastSampleTimeOffsetIndex];
		const UPoseSearchFeatureChannel_Velocity* Velocity = Cast<UPoseSearchFeatureChannel_Velocity>(SubChannelPtr.Get());
		check(Velocity);
		OutRootMotionTranslation = FFeatureVectorHelper::DecodeVector(PoseVector, Velocity->GetChannelDataOffset(), Velocity->ComponentStripping);
	}

	return OutRootMotionTranslation;
}


#undef LOCTEXT_NAMESPACE
