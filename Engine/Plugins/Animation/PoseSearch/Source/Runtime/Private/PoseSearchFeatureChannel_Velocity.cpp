// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchFeatureChannel_Velocity.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "PoseSearch/PoseSearchAssetIndexer.h"
#include "PoseSearch/PoseSearchAssetSampler.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchHistory.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearch/PoseSearchFeatureChannel_Position.h"

#if WITH_EDITOR
#include "PropertyHandle.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchFeatureChannel_Velocity)

UPoseSearchFeatureChannel_Velocity::UPoseSearchFeatureChannel_Velocity()
{
	bUseBlueprintQueryOverride = Cast<UBlueprintGeneratedClass>(GetClass()) != nullptr;
}

bool UPoseSearchFeatureChannel_Velocity::Finalize(UPoseSearchSchema* Schema)
{
	using namespace UE::PoseSearch;

	ChannelDataOffset = Schema->SchemaCardinality;
	ChannelCardinality = UE::PoseSearch::FFeatureVectorHelper::GetVectorCardinality(ComponentStripping);
	Schema->SchemaCardinality += ChannelCardinality;

	SchemaBoneIdx = Schema->AddBoneReference(Bone, SampleRole, bDefaultWithRootBone);
	SchemaOriginBoneIdx = Schema->AddBoneReference(OriginBone, OriginRole, bDefaultWithRootBone);

	return SchemaBoneIdx != InvalidSchemaBoneIdx && SchemaOriginBoneIdx != InvalidSchemaBoneIdx;
}

void UPoseSearchFeatureChannel_Velocity::AddDependentChannels(UPoseSearchSchema* Schema) const
{
	if (Schema->bInjectAdditionalDebugChannels)
	{
		const EPermutationTimeType DependentChannelsPermutationTimeType = PermutationTimeType != EPermutationTimeType::UseSampleTime ? EPermutationTimeType::UseSampleToPermutationTime : EPermutationTimeType::UseSampleTime;
		UPoseSearchFeatureChannel_Position::FindOrAddToSchema(Schema, SampleTimeOffset, Bone.BoneName, SampleRole, DependentChannelsPermutationTimeType, bDefaultWithRootBone);
	}
}

FVector UPoseSearchFeatureChannel_Velocity::BP_GetWorldVelocityFromContext_Implementation(FChooserEvaluationContext& Context) const
{
	using namespace UE::PoseSearch;

	if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(GetAnimContext(&Context)))
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return BP_GetWorldVelocity(AnimInstance);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	UE_LOGF(LogPoseSearch, Warning, "UPoseSearchFeatureChannel_Velocity::BP_GetWorldVelocityFromContext_Implementation - unsupported null UAnimInstance: WIP support for AnimNext!");
	return FVector::ZeroVector;
}

void UPoseSearchFeatureChannel_Velocity::BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const
{
	using namespace UE::PoseSearch;

	if (bUseBlueprintQueryOverride)
	{
		FChooserEvaluationContext* ChooserContext = SearchContext.GetContext(SampleRole);
		const FVector LinearVelocityWorld = ensure(ChooserContext) ? BP_GetWorldVelocityFromContext(*ChooserContext) : FVector::ZeroVector;
	
		FVector LinearVelocity = SearchContext.GetSampleVelocity(SampleTimeOffset, OriginTimeOffset, SchemaBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole, bUseCharacterSpaceVelocities, EPermutationTimeType::UseSampleTime, &LinearVelocityWorld);
		if (bNormalize)
		{
			LinearVelocity = LinearVelocity.GetClampedToMaxSize(1.f);
		}
		FFeatureVectorHelper::EncodeVector(SearchContext.EditFeatureVector(), ChannelDataOffset, LinearVelocity, ComponentStripping, false);
		return;
	}
	
	// trying to get the BuildQuery data from another schema UPoseSearchFeatureChannel_Velocity already cached in the SearchContext
	if (SearchContext.IsUseCachedChannelData())
	{
		// composing a unique identifier to specify this channel with all the required properties to be able to share the query data with other channels of the same type
		uint32 UniqueIdentifier = GetClass()->GetUniqueID();
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SampleRole));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(OriginRole));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SamplingAttributeId));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SampleTimeOffset));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(OriginTimeOffset));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SchemaBoneIdx));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(SchemaOriginBoneIdx));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(InputQueryPose));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(bUseCharacterSpaceVelocities));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(bNormalize));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(ComponentStripping));
		UniqueIdentifier = HashCombineFast(UniqueIdentifier, GetTypeHash(PermutationTimeType));

		TConstArrayView<float> CachedChannelData;
		if (const UPoseSearchFeatureChannel* CachedChannel = SearchContext.GetCachedChannelData(UniqueIdentifier, this, CachedChannelData))
		{
#if DO_CHECK
			const UPoseSearchFeatureChannel_Velocity* CachedVelocityChannel = Cast<UPoseSearchFeatureChannel_Velocity>(CachedChannel);
			check(CachedVelocityChannel);
			check(CachedVelocityChannel->GetChannelCardinality() == ChannelCardinality);
			check(CachedChannelData.Num() == ChannelCardinality);

			// making sure there were no hash collisions
			check(CachedVelocityChannel->SampleRole == SampleRole);
			check(CachedVelocityChannel->OriginRole == OriginRole);
			check(CachedVelocityChannel->SamplingAttributeId == SamplingAttributeId);
			check(CachedVelocityChannel->SampleTimeOffset == SampleTimeOffset);
			check(CachedVelocityChannel->OriginTimeOffset == OriginTimeOffset);
			check(CachedVelocityChannel->SchemaBoneIdx == SchemaBoneIdx);
			check(CachedVelocityChannel->SchemaOriginBoneIdx == SchemaOriginBoneIdx);
			check(CachedVelocityChannel->InputQueryPose == InputQueryPose);
			check(CachedVelocityChannel->bUseCharacterSpaceVelocities == bUseCharacterSpaceVelocities);
			check(CachedVelocityChannel->bNormalize == bNormalize);
			check(CachedVelocityChannel->ComponentStripping == ComponentStripping);
			check(CachedVelocityChannel->PermutationTimeType == PermutationTimeType);
#endif //DO_CHECK

			// copying the CachedChannelData into this channel portion of the FeatureVectorBuilder
			FFeatureVectorHelper::Copy(SearchContext.EditFeatureVector().Slice(ChannelDataOffset, ChannelCardinality), 0, ChannelCardinality, CachedChannelData);
			return;
		}
	}

	const bool bCanUseContinuingPoseValues = SearchContext.CanUseContinuingPoseValues();
	const bool bSkip = InputQueryPose != EInputQueryPose::UseCharacterPose && bCanUseContinuingPoseValues && SampleRole == OriginRole;
	if (bSkip)
	{
		if (bCanUseContinuingPoseValues)
		{
			FFeatureVectorHelper::Copy(SearchContext.EditFeatureVector(), ChannelDataOffset, ChannelCardinality, SearchContext.GetContinuingPoseValues());
		}
		return;
	}
	
	// calculating the LinearVelocity for the bone indexed by SchemaBoneIdx
	FVector LinearVelocity = SearchContext.GetSampleVelocity(SampleTimeOffset, OriginTimeOffset, SchemaBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole, bUseCharacterSpaceVelocities, PermutationTimeType);
	if (bNormalize)
	{
		LinearVelocity = LinearVelocity.GetClampedToMaxSize(1.f);
	}

	FFeatureVectorHelper::EncodeVector(SearchContext.EditFeatureVector(), ChannelDataOffset, LinearVelocity, ComponentStripping, false);
}

#if ENABLE_DRAW_DEBUG
void UPoseSearchFeatureChannel_Velocity::DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseVector) const
{
	using namespace UE::PoseSearch;

	if (DrawParams.IsAnyWeightRelevant(this))
	{
		Super::DebugDraw(DrawParams, PoseVector);

		FColor Color;
#if WITH_EDITORONLY_DATA
		Color = DebugColor.ToFColor(true);
#else // WITH_EDITORONLY_DATA
		Color = FColor::Green;
#endif // WITH_EDITORONLY_DATA

		const float LinearVelocityScale = bNormalize ? 15.f : 0.08f;
		const float ArrowScale = 0.2f * LinearVelocityScale;

		float PermutationSampleTimeOffset = 0.f;
		float PermutationOriginTimeOffset = 0.f;
		UPoseSearchFeatureChannel::GetPermutationTimeOffsets(PermutationTimeType, DrawParams.ExtractPermutationTime(PoseVector), PermutationSampleTimeOffset, PermutationOriginTimeOffset);
		const EPermutationTimeType SamplePermutationTimeType = PermutationTimeType == EPermutationTimeType::UsePermutationTime ? EPermutationTimeType::UseSampleToPermutationTime : EPermutationTimeType::UseSampleTime;
		const EPermutationTimeType OriginPermutationTimeType = SamplePermutationTimeType;

		const FVector FeaturesVector = FFeatureVectorHelper::DecodeVector(PoseVector, ChannelDataOffset, ComponentStripping);
		// velocity is encoded in OriginRole's root-bone space at OriginTime; rotate it back to world using that frame
		const FQuat OriginRotation = DrawParams.ExtractRotation(PoseVector, OriginTimeOffset, RootSchemaBoneIdx, OriginRole, OriginPermutationTimeType, SamplingAttributeId, PermutationOriginTimeOffset);
		const FVector LinearVelocity = OriginRotation.RotateVector(FeaturesVector);
		const FVector BonePos = DrawParams.ExtractPosition(PoseVector, SampleTimeOffset, SchemaBoneIdx, SampleRole, SamplePermutationTimeType, SamplingAttributeId, PermutationSampleTimeOffset);
		const FVector EndPos = BonePos + LinearVelocity * LinearVelocityScale;

		DrawParams.DrawLine(BonePos, EndPos, Color);

		const FVector SideVector = LinearVelocity.Cross(OriginRotation.RotateVector(FVector::UpVector));
		DrawParams.DrawLine(EndPos, EndPos - (LinearVelocity - SideVector) * ArrowScale, Color);
		DrawParams.DrawLine(EndPos, EndPos - (LinearVelocity + SideVector) * ArrowScale, Color);
	}
}
#endif // ENABLE_DRAW_DEBUG

#if WITH_EDITOR
void UPoseSearchFeatureChannel_Velocity::FillWeights(TArrayView<float> Weights) const
{
	for (int32 i = 0; i < ChannelCardinality; ++i)
	{
		Weights[ChannelDataOffset + i] = Weight;
	}
}

bool UPoseSearchFeatureChannel_Velocity::IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const
{
	using namespace UE::PoseSearch;

	FVector LinearVelocity;
	for (int32 SampleIdx = Indexer.GetBeginSampleIdx(); SampleIdx != Indexer.GetEndSampleIdx(); ++SampleIdx)
	{
		if (Indexer.GetSampleVelocity(LinearVelocity, SampleTimeOffset, OriginTimeOffset, SampleIdx, SchemaBoneIdx, SchemaOriginBoneIdx, SampleRole, OriginRole, bUseCharacterSpaceVelocities, PermutationTimeType, SamplingAttributeId))
		{
			if (bNormalize)
			{
				LinearVelocity = LinearVelocity.GetClampedToMaxSize(1.f);
			}
			FFeatureVectorHelper::EncodeVector(Indexer.GetPoseVector(SampleIdx), ChannelDataOffset, LinearVelocity, ComponentStripping, false);
		}
		else
		{
			return false;
		}
	}
	return true;
}

UE::PoseSearch::TLabelBuilder& UPoseSearchFeatureChannel_Velocity::GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat) const
{
	using namespace UE::PoseSearch;

	GetOuterLabel(LabelBuilder, LabelFormat);
	AppendLabelSeparator(LabelBuilder, LabelFormat);

	LabelBuilder.Append(TEXT("Vel"));
	if (bNormalize)
	{
		LabelBuilder.Append(TEXT("Dir"));
	}

	if (ComponentStripping == EComponentStrippingVector::StripXY)
	{
		LabelBuilder.Append(TEXT("_z"));
	}
	else if (ComponentStripping == EComponentStrippingVector::StripZ)
	{
		LabelBuilder.Append(TEXT("_xy"));
	}

	const UPoseSearchSchema* Schema = GetSchema();
	check(Schema);
	if (SchemaBoneIdx > RootSchemaBoneIdx)
	{
		LabelBuilder.Append(TEXT("_"));
		LabelBuilder.Append(Schema->GetBoneReferences(SampleRole)[SchemaBoneIdx].BoneName.ToString());
	}
	else if (SchemaBoneIdx == TrajectorySchemaBoneIdx)
	{
		LabelBuilder.Append(TEXT("_Trj"));
	}
	
	if (SampleRole != DefaultRole)
	{
		LabelBuilder.Append(TEXT("["));
		LabelBuilder.Append(SampleRole.ToString());
		LabelBuilder.Append(TEXT("]"));
	}

	if (SchemaOriginBoneIdx > RootSchemaBoneIdx)
	{
		LabelBuilder.Append(TEXT("_"));
		LabelBuilder.Append(Schema->GetBoneReferences(OriginRole)[SchemaOriginBoneIdx].BoneName.ToString());
	}
	else if (SchemaOriginBoneIdx == TrajectorySchemaBoneIdx)
	{
		LabelBuilder.Append(TEXT("_Trj"));
	}
	
	if (OriginRole != DefaultRole)
	{
		LabelBuilder.Append(TEXT("["));
		LabelBuilder.Append(OriginRole.ToString());
		LabelBuilder.Append(TEXT("]"));
	}

	if (PermutationTimeType == EPermutationTimeType::UsePermutationTime)
	{
		LabelBuilder.Append(TEXT("_PT"));
	}
	else if (PermutationTimeType == EPermutationTimeType::UseSampleToPermutationTime)
	{
		LabelBuilder.Append(TEXT("_SPT"));
	}

	AppendLabelSeparator(LabelBuilder, LabelFormat, true);

	LabelBuilder.Appendf(TEXT("%.2f"), SampleTimeOffset);

	if (!FMath::IsNearlyZero(OriginTimeOffset))
	{
		LabelBuilder.Appendf(TEXT("-%.2f"), OriginTimeOffset);
	}

	return LabelBuilder;
}

USkeleton* UPoseSearchFeatureChannel_Velocity::GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle)
{
	// blueprint generated classes don't have a schema, until they're instanced by the schema
	if (const UPoseSearchSchema* Schema = GetSchema())
	{
		bInvalidSkeletonIsError = false;
		if (PropertyHandle)
		{
			if (PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UPoseSearchFeatureChannel_Velocity, Bone))
			{
				return Schema->GetSkeleton(SampleRole);
			}
			if (PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UPoseSearchFeatureChannel_Velocity, OriginBone))
			{
				return Schema->GetSkeleton(OriginRole);
			}
		}
	}

	return Super::GetSkeleton(bInvalidSkeletonIsError, PropertyHandle);
}
#endif
