// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchFeatureVectorHelper.h"
#include "CoreMinimal.h"

namespace UE::PoseSearch
{

int32 FFeatureVectorHelper::GetVectorCardinality(EComponentStrippingVector ComponentStrippingVector)
{
	switch (ComponentStrippingVector)
	{
	case EComponentStrippingVector::None:
		return 3;
	case EComponentStrippingVector::StripXY:
		return 1;
	case EComponentStrippingVector::StripZ:
		return 2;
	default:
		checkNoEntry();
		return 0;
	}
}

void FFeatureVectorHelper::EncodeVector(TArrayView<float> Values, int32 DataOffset, const FVector& Vector, EComponentStrippingVector ComponentStrippingVector, bool bNormalize)
{
	check(Values.GetData());

	if (bNormalize)
	{
		FVector NormalizedVector = Vector;
		switch (ComponentStrippingVector)
		{
		case EComponentStrippingVector::None:
			check(Values.Num() > DataOffset + 2);
			NormalizedVector.Normalize();
			Values[DataOffset + 0] = NormalizedVector.X;
			Values[DataOffset + 1] = NormalizedVector.Y;
			Values[DataOffset + 2] = NormalizedVector.Z;
			break;
		case EComponentStrippingVector::StripXY:
			check(Values.Num() > DataOffset + 0);			
			NormalizedVector.X = 0.f;
			NormalizedVector.Y = 0.f;
			NormalizedVector.Normalize();
			Values[DataOffset + 0] = NormalizedVector.Z;
			break;
		case EComponentStrippingVector::StripZ:
			check(Values.Num() > DataOffset + 1);
			NormalizedVector.Z = 0.f;
			NormalizedVector.Normalize();
			Values[DataOffset + 0] = NormalizedVector.X;
			Values[DataOffset + 1] = NormalizedVector.Y;
			break;
		default:
			checkNoEntry();
			break;
		}
	}
	else
	{
		switch (ComponentStrippingVector)
		{
		case EComponentStrippingVector::None:
			check(Values.Num() > DataOffset + 2);
			Values[DataOffset + 0] = Vector.X;
			Values[DataOffset + 1] = Vector.Y;
			Values[DataOffset + 2] = Vector.Z;
			break;
		case EComponentStrippingVector::StripXY:
			check(Values.Num() > DataOffset + 0);
			Values[DataOffset + 0] = Vector.Z;
			break;
		case EComponentStrippingVector::StripZ:
			check(Values.Num() > DataOffset + 1);
			Values[DataOffset + 0] = Vector.X;
			Values[DataOffset + 1] = Vector.Y;
			break;
		default:
			checkNoEntry();
			break;
		}
	}
}

FVector FFeatureVectorHelper::DecodeVector(TConstArrayView<float> Values, int32 DataOffset, EComponentStrippingVector ComponentStrippingVector)
{
	switch (ComponentStrippingVector)
	{
	case EComponentStrippingVector::None:
		check(Values.Num() > DataOffset + 2);
		return FVector(Values[DataOffset + 0], Values[DataOffset + 1], Values[DataOffset + 2]);
	case EComponentStrippingVector::StripXY:
		check(Values.Num() > DataOffset + 0);
		return FVector(0, 0, Values[DataOffset + 0]);
	case EComponentStrippingVector::StripZ:
		check(Values.Num() > DataOffset + 1);
		return FVector(Values[DataOffset + 0], Values[DataOffset + 1], 0);
	default:
		checkNoEntry();
		return FVector::Zero();
	}
}

void FFeatureVectorHelper::EncodeVector2D(TArrayView<float> Values, int32 DataOffset, const FVector2D& Vector2D)
{
	check(Values.Num() > DataOffset + 1);
	Values[DataOffset + 0] = Vector2D.X;
	Values[DataOffset + 1] = Vector2D.Y;
}

FVector2D FFeatureVectorHelper::DecodeVector2D(TConstArrayView<float> Values, int32 DataOffset)
{
	check(Values.Num() > DataOffset + 1);
	return FVector2D(Values[DataOffset + 0], Values[DataOffset + 1]);
}

void FFeatureVectorHelper::EncodeFloat(TArrayView<float> Values, int32 DataOffset, const float Value)
{
	check(Values.Num() > DataOffset + 0);
	Values[DataOffset + 0] = Value;
}

float FFeatureVectorHelper::DecodeFloat(TConstArrayView<float> Values, int32 DataOffset)
{
	check(Values.Num() > DataOffset + 0);
	return Values[DataOffset];
}

void FFeatureVectorHelper::Copy(TArrayView<float> Values, int32 DataOffset, int32 DataCardinality, TConstArrayView<float> OriginValues)
{
	check(Values.GetData() && OriginValues.GetData());
	check(Values.Num() == OriginValues.Num() && DataOffset + DataCardinality <= Values.Num());

	float* RESTRICT ValuesData = Values.GetData() + DataOffset;
	const float* RESTRICT OriginValuesData = OriginValues.GetData() + DataOffset;

	check(ValuesData != OriginValuesData);

	for (int32 i = 0; i < DataCardinality; ++i, ++ValuesData, ++OriginValuesData)
	{
		*ValuesData = *OriginValuesData;
	}
}

} // namespace UE::PoseSearch
