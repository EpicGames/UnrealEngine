// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchFilter.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchSchema.h"

// @todo: add support for generic vector (3 float of data) channel
#include "PoseSearch/PoseSearchFeatureChannel_Position.h"

///////////////////////////////////////////////////
// FPoseSearchFilterPadding 
int32 FPoseSearchFilterPadding::GetRequiredDataSize(const UPoseSearchFeatureChannel* Channel) const
{
	return 1;
}

///////////////////////////////////////////////////
// FPoseSearchFilterMaxPosition 
int32 FPoseSearchFilterMaxPosition::GetRequiredDataSize(const UPoseSearchFeatureChannel* Channel) const
{
	// @todo: add support for generic vector (3 float of data) channel
	if (MaxPositionDistanceSquared > UE_KINDA_SMALL_NUMBER &&
		Cast<UPoseSearchFeatureChannel_Position>(Channel))
	{
		return sizeof(FPoseSearchFilterMaxPosition::FData);
	}
	// @todo: log some warnings?
	return InvalidFilterDataSize;
}


bool FPoseSearchFilterMaxPosition::InitializeData(TArrayView<uint8> Data, const UPoseSearchFeatureChannel* Channel, TConstArrayView<float> QueryValues, UE::PoseSearch::FSearchContext& SearchContext) const
{
	using namespace UE::PoseSearch;

	check(IsAligned(Data.GetData(), alignof(FPoseSearchFilterMaxPosition::FData)));
	check(Data.Num() == sizeof(FPoseSearchFilterMaxPosition::FData));

	// @todo: add support for generic vector (3 float of data) channel
	check(nullptr != Cast<UPoseSearchFeatureChannel_Position>(Channel));
	const UPoseSearchFeatureChannel_Position* PositionChannel = static_cast<const UPoseSearchFeatureChannel_Position*>(Channel);

	const int32 ChannelDataOffset = PositionChannel->GetChannelDataOffset();
	const EComponentStrippingVector ComponentStripping = PositionChannel->ComponentStripping;

	FPoseSearchFilterMaxPosition::FData& FilterMaxPositionData = reinterpret_cast<FPoseSearchFilterMaxPosition::FData&>(*Data.GetData());
	FilterMaxPositionData.Query = FFeatureVectorHelper::DecodeVector(QueryValues, ChannelDataOffset, ComponentStripping);

	return true;
}

void FPoseSearchFilterMaxPosition::DeinitializeData(TArrayView<uint8> Data, const UPoseSearchFeatureChannel* Channel, UE::PoseSearch::FSearchContext& SearchContext) const
{
	check(IsAligned(Data.GetData(), alignof(FPoseSearchFilterMaxPosition::FData)));
	check(Data.Num() == sizeof(FPoseSearchFilterMaxPosition::FData));
}
	
bool FPoseSearchFilterMaxPosition::IsFilterValid(TConstArrayView<float> PoseValues, TConstArrayView<float> QueryValues, int32 PoseIdx, const UE::PoseSearch::FPoseMetadata& Metadata, TConstArrayView<uint8> Data, const UPoseSearchFeatureChannel* Channel) const
{
	using namespace UE::PoseSearch;

	check(IsAligned(Data.GetData(), alignof(FPoseSearchFilterMaxPosition::FData)));
	check(Data.Num() == sizeof(FPoseSearchFilterMaxPosition::FData));

	const FPoseSearchFilterMaxPosition::FData& FilterMaxPositionData = reinterpret_cast<const FPoseSearchFilterMaxPosition::FData&>(*Data.GetData());

	// @todo: add support for generic vector (3 float of data) channel
	check(nullptr != Cast<UPoseSearchFeatureChannel_Position>(Channel));

	const UPoseSearchFeatureChannel_Position* PositionChannel = static_cast<const UPoseSearchFeatureChannel_Position*>(Channel);
	const int32 ChannelDataOffset = PositionChannel->GetChannelDataOffset();
	const EComponentStrippingVector ComponentStripping = PositionChannel->ComponentStripping;

	const FVector Pose = FFeatureVectorHelper::DecodeVector(PoseValues, ChannelDataOffset, ComponentStripping);

	const float SquaredLength = (Pose - FilterMaxPositionData.Query).SquaredLength();

	return SquaredLength <= MaxPositionDistanceSquared;
}

#if ENABLE_DRAW_DEBUG
void FPoseSearchFilterMaxPosition::DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseValues, const UPoseSearchFeatureChannel* Channel) const
{
	using namespace UE::PoseSearch;

	if (DrawParams.GetDrawContext() == FDebugDrawParams::EDrawContext::DrawQuery)
	{
		// @todo: add support for generic vector (3 float of data) channel
		if (const UPoseSearchFeatureChannel_Position* PositionChannel = Cast<UPoseSearchFeatureChannel_Position>(Channel))
		{
			const TConstArrayView<float> PoseVector = PoseValues;
			const int32 ChannelDataOffset = PositionChannel->GetChannelDataOffset();
			const EComponentStrippingVector ComponentStripping = PositionChannel->ComponentStripping;
			const float OriginTimeOffset = PositionChannel->OriginTimeOffset;
			const int8 SchemaOriginBoneIdx = PositionChannel->SchemaOriginBoneIdx;
			const FRole OriginRole = PositionChannel->OriginRole;
			const EPermutationTimeType PermutationTimeType = PositionChannel->PermutationTimeType;

			float PermutationSampleTimeOffset = 0.f;
			float PermutationOriginTimeOffset = 0.f;
			UPoseSearchFeatureChannel::GetPermutationTimeOffsets(PermutationTimeType, DrawParams.ExtractPermutationTime(PoseVector), PermutationSampleTimeOffset, PermutationOriginTimeOffset);
			const EPermutationTimeType OriginPermutationTimeType = PermutationTimeType == EPermutationTimeType::UsePermutationTime ? EPermutationTimeType::UseSampleToPermutationTime : EPermutationTimeType::UseSampleTime;

			const FVector FeaturesVector = FFeatureVectorHelper::DecodeVector(PoseVector, ChannelDataOffset, ComponentStripping);
			const FVector OriginBonePos = DrawParams.ExtractPosition(PoseVector, OriginTimeOffset, SchemaOriginBoneIdx, OriginRole, OriginPermutationTimeType, INDEX_NONE, PermutationOriginTimeOffset);
			const FVector DeltaPos = DrawParams.ExtractRotation(PoseVector, OriginTimeOffset, RootSchemaBoneIdx, OriginRole, OriginPermutationTimeType, INDEX_NONE, PermutationOriginTimeOffset).RotateVector(FeaturesVector);
			const FVector BonePos = OriginBonePos + DeltaPos;

			static constexpr int32 Segments = 32;
			const float Radius = FMath::Sqrt(MaxPositionDistanceSquared);

#if WITH_EDITORONLY_DATA
			const FColor Color = PositionChannel->GetDebugColor().ToFColor(true);
#else // WITH_EDITORONLY_DATA
			const FColor Color = FColor::White;
#endif // WITH_EDITORONLY_DATA

			if (PositionChannel->ComponentStripping == EComponentStrippingVector::StripZ)
			{
				DrawParams.DrawCircle(FMatrix(FVector::ZAxisVector, FVector::XAxisVector, FVector::YAxisVector, BonePos), Radius, Segments, Color);
			}
			else
			{
				DrawParams.DrawSphere(BonePos, Radius, Segments, Color);
			}
		}
	}
}

#endif // ENABLE_DRAW_DEBUG

///////////////////////////////////////////////////
// FPoseSearchFilterWedge 
int32 FPoseSearchFilterWedge::GetRequiredDataSize(const UPoseSearchFeatureChannel* Channel) const
{
	// @todo: add support for generic vector (3 float of data) channel
	if (WedgeWidthInDegrees > UE_KINDA_SMALL_NUMBER && 
		!FMath::IsNearlyEqual(RadiusOffsets.Min, RadiusOffsets.Max,UE_KINDA_SMALL_NUMBER) && 
		Cast<UPoseSearchFeatureChannel_Position>(Channel))
	{
		return sizeof(FPoseSearchFilterWedge::FData);
	}
	// @todo: log some warnings?
	return InvalidFilterDataSize;
}

bool FPoseSearchFilterWedge::InitializeData(TArrayView<uint8> Data, const UPoseSearchFeatureChannel* Channel, TConstArrayView<float> QueryValues, UE::PoseSearch::FSearchContext& SearchContext) const
{
	using namespace UE::PoseSearch;

	check(IsAligned(Data.GetData(), alignof(FPoseSearchFilterWedge::FData)));
	check(Data.Num() == sizeof(FPoseSearchFilterWedge::FData));

	// @todo: add support for generic vector (3 float of data) channel
	check(nullptr != Cast<UPoseSearchFeatureChannel_Position>(Channel));
	const UPoseSearchFeatureChannel_Position* PositionChannel = static_cast<const UPoseSearchFeatureChannel_Position*>(Channel);

	const int32 ChannelDataOffset = PositionChannel->GetChannelDataOffset();
	const EComponentStrippingVector ComponentStripping = PositionChannel->ComponentStripping;

	FPoseSearchFilterWedge::FData& FilterWedgeData = reinterpret_cast<FPoseSearchFilterWedge::FData&>(*Data.GetData());
	FilterWedgeData.NormalizedQuery = FFeatureVectorHelper::DecodeVector(QueryValues, ChannelDataOffset, ComponentStripping);
	const float QueryLength = FilterWedgeData.NormalizedQuery.Length();
	
	const bool bShouldFilter = QueryLength > UE_KINDA_SMALL_NUMBER;
	if (bShouldFilter)
	{
		FilterWedgeData.NormalizedQuery /= QueryLength;
		FilterWedgeData.InnerRadius = FMath::Max(0.f, QueryLength + RadiusOffsets.Min);
		FilterWedgeData.OuterRadius = FMath::Max(0.f, QueryLength + RadiusOffsets.Max);
		FilterWedgeData.DotWedgeWidth = FMath::Cos(FMath::DegreesToRadians(WedgeWidthInDegrees * 0.5f));
		return true;
	}

	return false;
}

void FPoseSearchFilterWedge::DeinitializeData(TArrayView<uint8> Data, const UPoseSearchFeatureChannel* Channel, UE::PoseSearch::FSearchContext& SearchContext) const
{
	check(IsAligned(Data.GetData(), alignof(FPoseSearchFilterWedge::FData)));
	check(Data.Num() == sizeof(FPoseSearchFilterWedge::FData));
}

// @todo: fix the math to be on the plane with normal ZAxisVector. or fix the drawing to support a 3d conical wedge
bool FPoseSearchFilterWedge::IsFilterValid(TConstArrayView<float> PoseValues, TConstArrayView<float> QueryValues, int32 PoseIdx, const UE::PoseSearch::FPoseMetadata& Metadata, TConstArrayView<uint8> Data, const UPoseSearchFeatureChannel* Channel) const
{
	using namespace UE::PoseSearch;

	check(IsAligned(Data.GetData(), alignof(FPoseSearchFilterWedge::FData)));
	check(Data.Num() == sizeof(FPoseSearchFilterWedge::FData));

	const FPoseSearchFilterWedge::FData& FilterWedgeData = reinterpret_cast<const FPoseSearchFilterWedge::FData&>(*Data.GetData());

	// @todo: add support for generic vector (3 float of data) channel
	check(nullptr != Cast<UPoseSearchFeatureChannel_Position>(Channel));
	const UPoseSearchFeatureChannel_Position* PositionChannel = static_cast<const UPoseSearchFeatureChannel_Position*>(Channel);
	
	const int32 ChannelDataOffset = PositionChannel->GetChannelDataOffset();
	const EComponentStrippingVector ComponentStripping = PositionChannel->ComponentStripping;

	const FVector Pose = FFeatureVectorHelper::DecodeVector(PoseValues, ChannelDataOffset, ComponentStripping);
	const float PoseLength = Pose.Length();

	if (PoseLength > UE_KINDA_SMALL_NUMBER)
	{
		if (PoseLength >= FilterWedgeData.InnerRadius && PoseLength <= FilterWedgeData.OuterRadius)
		{
			const FVector NormalizedPose = Pose / PoseLength;
			const float Dot = NormalizedPose | FilterWedgeData.NormalizedQuery;

			if (Dot > FilterWedgeData.DotWedgeWidth)
			{
				return true;
			}
		}
	}
	
	return false;
}

#if ENABLE_DRAW_DEBUG
void FPoseSearchFilterWedge::DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseValues, const UPoseSearchFeatureChannel* Channel) const
{
	using namespace UE::PoseSearch;

	if (DrawParams.GetDrawContext() == FDebugDrawParams::EDrawContext::DrawQuery)
	{
		// @todo: add support for generic vector (3 float of data) channel
		if (const UPoseSearchFeatureChannel_Position* PositionChannel = Cast<UPoseSearchFeatureChannel_Position>(Channel))
		{
			// @todo: avoid this duplicate calculation already done in UPoseSearchFeatureChannel_Position::DebugDraw

			const TConstArrayView<float> PoseVector = PoseValues;
			const int32 ChannelDataOffset = PositionChannel->GetChannelDataOffset();
			const EComponentStrippingVector ComponentStripping = PositionChannel->ComponentStripping;
			const float OriginTimeOffset = PositionChannel->OriginTimeOffset;
			const int8 SchemaOriginBoneIdx = PositionChannel->SchemaOriginBoneIdx;
			const FRole OriginRole = PositionChannel->OriginRole;
			const EPermutationTimeType PermutationTimeType = PositionChannel->PermutationTimeType;

			float PermutationSampleTimeOffset = 0.f;
			float PermutationOriginTimeOffset = 0.f;
			UPoseSearchFeatureChannel::GetPermutationTimeOffsets(PermutationTimeType, DrawParams.ExtractPermutationTime(PoseVector), PermutationSampleTimeOffset, PermutationOriginTimeOffset);
			const EPermutationTimeType OriginPermutationTimeType = PermutationTimeType == EPermutationTimeType::UsePermutationTime ? EPermutationTimeType::UseSampleToPermutationTime : EPermutationTimeType::UseSampleTime;

			const FVector FeaturesVector = FFeatureVectorHelper::DecodeVector(PoseVector, ChannelDataOffset, ComponentStripping);
			const FVector OriginBonePos = DrawParams.ExtractPosition(PoseVector, OriginTimeOffset, SchemaOriginBoneIdx, OriginRole, OriginPermutationTimeType, INDEX_NONE, PermutationOriginTimeOffset);
			const FVector DeltaPos = DrawParams.ExtractRotation(PoseVector, OriginTimeOffset, RootSchemaBoneIdx, OriginRole, OriginPermutationTimeType, INDEX_NONE, PermutationOriginTimeOffset).RotateVector(FeaturesVector);
			const FVector BonePos = OriginBonePos + DeltaPos;

			//const FVector RootBonePos = DrawParams.GetRootBoneTransform(OriginRole, SamplingRootTime).GetLocation();
			const FVector WedgeDirection = BonePos - OriginBonePos;

			static constexpr int32 Segments = 32;

#if WITH_EDITORONLY_DATA
			const FColor Color = PositionChannel->GetDebugColor().ToFColor(true);
#else // WITH_EDITORONLY_DATA
			const FColor Color = FColor::White;
#endif // WITH_EDITORONLY_DATA

			const float WedgeLength = WedgeDirection.Length();
			const float InnerRadius = FMath::Max(0.f, WedgeLength + RadiusOffsets.Min);
			const float OuterRadius = FMath::Max(0.f, WedgeLength + RadiusOffsets.Max);
			DrawParams.DrawWedge(OriginBonePos, WedgeDirection, InnerRadius, OuterRadius, WedgeWidthInDegrees, Segments, Color);
		}
	}
}
#endif // ENABLE_DRAW_DEBUG

namespace UE::PoseSearch
{
	
FSearchFilters::FSearchFilters(const UPoseSearchSchema* Schema, TConstArrayView<int32> NonSelectableIdx, TConstArrayView<int32> SelectableAssetIdx,
	bool bAddBlockTransitionFilter, TConstArrayView<float> QueryValues, FSearchContext& InSearchContext)
	: SearchContext(InSearchContext)
{
	struct FFilterPreContext
	{
		FFilterPreContext(const FPoseSearchFilter* InFilter, const UPoseSearchFeatureChannel* InChannel, int32 InDataSize)
			: Filter(InFilter)
			, Channel(InChannel)
			, DataSize(InDataSize)
		{
		}

		const FPoseSearchFilter* Filter;
		const UPoseSearchFeatureChannel* Channel;
		int32 DataSize;
	};
	TArray<FFilterPreContext, TInlineAllocator<64, TMemStackAllocator<>>> FilterPreContexts;

	int32 TotalFiltersDataSize = 0;
	if (bAddBlockTransitionFilter)
	{
		const int32 BlockTransitionFilterDataSize = BlockTransitionFilter.GetRequiredDataSize(nullptr);
		TotalFiltersDataSize += Align(BlockTransitionFilterDataSize, FPoseSearchFilter::DataAlignment);
		FilterPreContexts.Emplace(&BlockTransitionFilter, nullptr, BlockTransitionFilterDataSize);
	}

	const int32 NonSelectableIdxFilterDataSize = NonSelectableIdxFilter.Init(NonSelectableIdx).GetRequiredDataSize(nullptr);
	if (NonSelectableIdxFilterDataSize != FPoseSearchFilter::InvalidFilterDataSize)
	{
		TotalFiltersDataSize += Align(NonSelectableIdxFilterDataSize, FPoseSearchFilter::DataAlignment);
		FilterPreContexts.Emplace(&NonSelectableIdxFilter, nullptr, NonSelectableIdxFilterDataSize);
	}

	const int32 SelectableAssetIdxFilterDataSize = SelectableAssetIdxFilter.Init(SelectableAssetIdx).GetRequiredDataSize(nullptr);
	if (SelectableAssetIdxFilterDataSize != FPoseSearchFilter::InvalidFilterDataSize)
	{
		TotalFiltersDataSize += Align(SelectableAssetIdxFilterDataSize, FPoseSearchFilter::DataAlignment);
		FilterPreContexts.Emplace(&SelectableAssetIdxFilter, nullptr, SelectableAssetIdxFilterDataSize);
	}

	// initialing Filters from all the channels FPoseSearchFilter(s) by exploring the schema recursively
	// to collect FilterPreContexts and sum up the TotalFiltersDataSize
	Schema->IterateChannels([&FilterPreContexts, &TotalFiltersDataSize](const UPoseSearchFeatureChannel* Channel)
		{
			const int32 NumFilters = Channel->GetNumFilters();
			for (int32 FilterIndex = 0; FilterIndex < NumFilters; ++FilterIndex)
			{
				const FPoseSearchFilter* Filter = Channel->GetFilter(FilterIndex);
				check(Filter);
				const int32 ChannelFilterDataSize = Filter->GetRequiredDataSize(Channel);
				if (ChannelFilterDataSize != FPoseSearchFilter::InvalidFilterDataSize)
				{
					TotalFiltersDataSize += Align(ChannelFilterDataSize, FPoseSearchFilter::DataAlignment);
					FilterPreContexts.Emplace(Filter, Channel, ChannelFilterDataSize);
				}
			}
		});

	// initializing the data required by the filters
	FiltersData.SetNumUninitialized(TotalFiltersDataSize);
	check(IsAligned(FiltersData.GetData(), FPoseSearchFilter::DataAlignment));

	// converting FFilterPreContext into FFilterContext
	int32 CurrentFilterDataOffset = 0;
	for (const FFilterPreContext& FilterPreContext : FilterPreContexts)
	{
		TArrayView<uint8> FilterData = MakeArrayView(FiltersData.GetData() + CurrentFilterDataOffset, FilterPreContext.DataSize);
		if (FilterPreContext.Filter->InitializeData(FilterData, FilterPreContext.Channel, QueryValues, SearchContext))
		{
			CurrentFilterDataOffset += Align(FilterPreContext.DataSize, FPoseSearchFilter::DataAlignment);
			FilterContexts.Emplace(FilterPreContext.Filter, FilterPreContext.Channel, FilterData);
		}
		else
		{
			// initialization failed. Deinitializing the filter and continue
			FilterPreContext.Filter->DeinitializeData(FilterData, FilterPreContext.Channel, SearchContext);
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// adding the deprecated filters that don't require any data
	for (const IPoseSearchFilter* Filter : Schema->GetChannels())
	{
		if (Filter->IsFilterActive())
		{
			Filters_DEPRECATED.Add(Filter);
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FSearchFilters::~FSearchFilters()
{
	for (FFilterContext& FilterContext : FilterContexts)
	{
		FilterContext.Filter->DeinitializeData(FilterContext.Data, FilterContext.Channel, SearchContext);
	}
}

bool FSearchFilters::AreFiltersValid(const FSearchIndex& SearchIndex, TConstArrayView<float> PoseValues, TConstArrayView<float> QueryValues, TConstArrayView<float> DynamicWeightsSqrt, int32 PoseIdx
#if UE_POSE_SEARCH_TRACE_ENABLED
	, float ContinuingPoseCostAddend, float ContinuingInteractionCostAddend, float ContinuingContextInteractionCostAddend, const UPoseSearchDatabase* Database
#endif // UE_POSE_SEARCH_TRACE_ENABLED
) const
{
	for (const FFilterContext& FilterContext : FilterContexts)
	{
		if (!FilterContext.Filter->IsFilterValid(PoseValues, QueryValues, PoseIdx, SearchIndex.PoseMetadata[PoseIdx], FilterContext.Data, FilterContext.Channel))
		{
#if UE_POSE_SEARCH_TRACE_ENABLED
			if (FilterContext.Filter == &NonSelectableIdxFilter)
			{
				// candidate already added to SearchContext.BestCandidates by PopulateNonSelectableIdx
			}
			else if (FilterContext.Filter == &SelectableAssetIdxFilter)
			{
				SearchContext.Track(Database, PoseIdx, EPoseCandidateFlags::DiscardedBy_AssetIdxFilter, DynamicWeightsSqrt, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend);
			}
			else if (FilterContext.Filter == &BlockTransitionFilter)
			{
				SearchContext.Track(Database, PoseIdx, EPoseCandidateFlags::DiscardedBy_BlockTransition, DynamicWeightsSqrt, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend);
			}
			else
			{
				SearchContext.Track(Database, PoseIdx, EPoseCandidateFlags::DiscardedBy_PoseFilter, DynamicWeightsSqrt, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend);
			}
#endif // UE_POSE_SEARCH_TRACE_ENABLED
			return false;
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (const IPoseSearchFilter* Filter : Filters_DEPRECATED)
	{
		if (!Filter->IsFilterValid(PoseValues, QueryValues, PoseIdx, SearchIndex.PoseMetadata[PoseIdx]))
		{
#if UE_POSE_SEARCH_TRACE_ENABLED
			SearchContext.Track(Database, PoseIdx, EPoseCandidateFlags::DiscardedBy_PoseFilter, DynamicWeightsSqrt, ContinuingPoseCostAddend, ContinuingInteractionCostAddend, ContinuingContextInteractionCostAddend);
#endif // UE_POSE_SEARCH_TRACE_ENABLED
			return false;
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
	return true;
}

} // namespace UE::PoseSearch