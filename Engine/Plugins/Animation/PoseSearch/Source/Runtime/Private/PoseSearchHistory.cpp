// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchHistory.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Animation/SkeletonRemapping.h"
#include "Animation/SkeletonRemappingRegistry.h"
#include "Animation/TrajectoryTypes.h"
#include "AnimationRuntime.h"
#include "BonePose.h"
#include "Components/SkeletalMeshComponent.h"
#include "DrawDebugHelpers.h"
#include "Engine/SkeletalMesh.h"
#include "PoseSearch/PoseSearchCustomVersion.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "PoseSearch/PoseSearchResult.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchHistory)

IMPLEMENT_ANIMGRAPH_MESSAGE(UE::PoseSearch::FPoseHistoryProvider);

namespace UE::PoseSearch
{

#if ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
static bool GVarAnimPoseHistoryDebugDrawPose = false;
static FAutoConsoleVariableRef CVarAnimPoseHistoryDebugDrawPose(TEXT("a.AnimNode.PoseHistory.DebugDrawPose"), GVarAnimPoseHistoryDebugDrawPose, TEXT("Enable / Disable Pose History Pose DebugDraw"));

static bool GVarAnimPoseHistoryDebugDrawTrajectory = false;
static FAutoConsoleVariableRef CVarAnimPoseHistoryDebugDrawTrajectory(TEXT("a.AnimNode.PoseHistory.DebugDrawTrajectory"), GVarAnimPoseHistoryDebugDrawTrajectory, TEXT("Enable / Disable Pose History Trajectory DebugDraw"));

static float GVarAnimPoseHistoryDebugDrawTrajectoryThickness = 0.f;
static FAutoConsoleVariableRef CVarAnimPoseHistoryDebugDrawTrajectoryThickness(TEXT("a.AnimNode.PoseHistory.DebugDrawTrajectoryThickness"), GVarAnimPoseHistoryDebugDrawTrajectoryThickness, TEXT("Thickness of the trajectory debug draw (Default 0.0f)"));

static int32 GVarAnimPoseHistoryDebugDrawTrajectoryMaxNumOfHistorySamples = -1;
static FAutoConsoleVariableRef CVarAnimPoseHistoryDebugDrawTrajectoryMaxNumOfHistorySamples(TEXT("a.AnimNode.PoseHistory.DebugDrawMaxNumOfHistorySamples"), GVarAnimPoseHistoryDebugDrawTrajectoryMaxNumOfHistorySamples, TEXT("Max number of history samples to debug draw. All history samples will be drawn if value is negative. (Default -1)"));

static int32 GVarAnimPoseHistoryDebugDrawTrajectoryMaxNumOfPredictionSamples = -1;
static FAutoConsoleVariableRef CVarAnimPoseHistoryDebugDrawTrajectoryMaxNumOfPredictionSamples(TEXT("a.AnimNode.PoseHistory.DebugDrawMaxNumOfPredictionSamples"), GVarAnimPoseHistoryDebugDrawTrajectoryMaxNumOfPredictionSamples, TEXT("Max number of prediction samples to debug draw. All prediction samples will be drawn if value is negative. (Default -1)"));
#endif

/**
* Algo::LowerBound adapted to TIndexedContainerIterator for use with indexable but not necessarily contiguous containers. Used here with TRingBuffer.
*
* Performs binary search, resulting in position of the first element >= Value using predicate
*
* @param First TIndexedContainerIterator beginning of range to search through, must be already sorted by SortPredicate
* @param Last TIndexedContainerIterator end of range
* @param Value Value to look for
* @param SortPredicate Predicate for sort comparison, defaults to <
*
* @returns Position of the first element >= Value, may be position after last element in range
*/
template <typename IteratorType, typename ValueType, typename SortPredicateType = TLess<>()>
FORCEINLINE auto LowerBound(IteratorType First, IteratorType Last, const ValueType& Value, SortPredicateType SortPredicate) -> decltype(First.GetIndex())
{
	using SizeType = decltype(First.GetIndex());

	check(First.GetIndex() <= Last.GetIndex());

	// Current start of sequence to check
	SizeType Start = First.GetIndex();

	// Size of sequence to check
	SizeType Size = Last.GetIndex() - Start;

	// With this method, if Size is even it will do one more comparison than necessary, but because Size can be predicted by the CPU it is faster in practice
	while (Size > 0)
	{
		const SizeType LeftoverSize = Size % 2;
		Size = Size / 2;

		const SizeType CheckIndex = Start + Size;
		const SizeType StartIfLess = CheckIndex + LeftoverSize;
		Start = SortPredicate(*(First + CheckIndex), Value) ? StartIfLess : Start;
	}
	return Start;
}

static FBoneIndexType GetRemappedBoneIndexType(FBoneIndexType BoneIndexType, const USkeleton* BoneIndexSkeleton, const USkeleton* LastUpdateSkeleton)
{
	// remapping the skeleton bone index (encoded as BoneIndexType) in case the skeleton we used to store the history (LastUpdateSkeleton) is different from BoneIndexSkeleton
	if (LastUpdateSkeleton != nullptr && LastUpdateSkeleton != BoneIndexSkeleton)
	{
		const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(BoneIndexSkeleton, LastUpdateSkeleton);
		if (SkeletonRemapping.IsValid())
		{
			BoneIndexType = SkeletonRemapping.GetTargetSkeletonBoneIndex(BoneIndexType);
		}
	}

	return BoneIndexType;
}

static FComponentSpaceTransformIndex GetRemappedComponentSpaceTransformIndex(const USkeleton* BoneIndexSkeleton, const USkeleton* LastUpdateSkeleton, const FBoneToTransformMap& BoneToTransformMap, FBoneIndexType BoneIndexType, bool& bSuccess)
{
	check(BoneIndexType != WorldSpaceIndexType);

	FComponentSpaceTransformIndex BoneTransformIndex = FComponentSpaceTransformIndex(BoneIndexType);
	if (BoneTransformIndex != ComponentSpaceIndexType)
	{
		BoneTransformIndex = GetRemappedBoneIndexType(BoneTransformIndex, BoneIndexSkeleton, LastUpdateSkeleton);

		if (!BoneToTransformMap.IsEmpty())
		{
			if (const FComponentSpaceTransformIndex* FoundBoneTransformIndex = BoneToTransformMap.Find(BoneTransformIndex))
			{
				BoneTransformIndex = *FoundBoneTransformIndex;
			}
			else
			{
				BoneTransformIndex = RootBoneIndexType;
				bSuccess = false;
			}
		}
	}
	return BoneTransformIndex;
}

static bool LerpEntries(float Time, bool bExtrapolate, const FPoseHistoryEntry& PrevEntry, const FPoseHistoryEntry& NextEntry, const FName& CurveName, const TConstArrayView<FName>& CollectedCurves, float& OutCurveValue)
{
	bool bSuccess = true;

	const int32 CurveIndex = CollectedCurves.Find(CurveName);
	if (CurveIndex == INDEX_NONE)
	{
		OutCurveValue = 0.0f;
		bSuccess = false;
	}
	else
	{
		const float Denominator = NextEntry.AccumulatedSeconds - PrevEntry.AccumulatedSeconds;
		float LerpValue = 0.f;
		if (!FMath::IsNearlyZero(Denominator))
		{
			const float Numerator = Time - PrevEntry.AccumulatedSeconds;
			LerpValue = bExtrapolate ? Numerator / Denominator : FMath::Clamp(Numerator / Denominator, 0.f, 1.f);
		}

		if (FMath::IsNearlyZero(LerpValue, ZERO_ANIMWEIGHT_THRESH))
		{
			OutCurveValue = PrevEntry.GetCurveValue(CurveIndex);
		}
		else if (FMath::IsNearlyZero(LerpValue - 1.f, ZERO_ANIMWEIGHT_THRESH))
		{
			OutCurveValue = NextEntry.GetCurveValue(CurveIndex);
		}
		else
		{
			OutCurveValue = FMath::Lerp(PrevEntry.GetCurveValue(CurveIndex), NextEntry.GetCurveValue(CurveIndex), LerpValue);
		}
	}

	return bSuccess;
}

static bool LerpEntries(float Time, bool bExtrapolate, const FPoseHistoryEntry& PrevEntry, const FPoseHistoryEntry& NextEntry, const USkeleton* BoneIndexSkeleton, const USkeleton* LastUpdateSkeleton,
	const FBoneToTransformMap& BoneToTransformMap, FBoneIndexType BoneIndexType, FBoneIndexType ReferenceBoneIndexType, FTransform& OutBoneTransform)
{
	check(BoneIndexType != ReferenceBoneIndexType);

	bool bSuccess = true;
	const FComponentSpaceTransformIndex BoneTransformIndex = GetRemappedComponentSpaceTransformIndex(BoneIndexSkeleton, LastUpdateSkeleton, BoneToTransformMap, BoneIndexType, bSuccess);
	if (!ensureMsgf(BoneTransformIndex != ComponentSpaceIndexType, TEXT("BoneIndexSkeleton [%s], LastUpdateSkeleton [%s] mapped to the same index."), *GetNameSafe(BoneIndexSkeleton), *GetNameSafe(LastUpdateSkeleton)))
	{
		OutBoneTransform = FTransform::Identity;
		return false;
	}
	
	const float Denominator = NextEntry.AccumulatedSeconds - PrevEntry.AccumulatedSeconds;
	float LerpValue = 0.f;
	if (!FMath::IsNearlyZero(Denominator))
	{
		const float Numerator = Time - PrevEntry.AccumulatedSeconds;
		LerpValue = bExtrapolate ? Numerator / Denominator : FMath::Clamp(Numerator / Denominator, 0.f, 1.f);
	}

	const FComponentSpaceTransformIndex ReferenceBoneTransformIndex = GetRemappedComponentSpaceTransformIndex(BoneIndexSkeleton, LastUpdateSkeleton, BoneToTransformMap, ReferenceBoneIndexType, bSuccess);
	if (ReferenceBoneTransformIndex == ComponentSpaceIndexType)
	{
		if (FMath::IsNearlyZero(LerpValue, ZERO_ANIMWEIGHT_THRESH))
		{
			OutBoneTransform = PrevEntry.GetComponentSpaceTransform(BoneTransformIndex);
		}
		else if (FMath::IsNearlyZero(LerpValue - 1.f, ZERO_ANIMWEIGHT_THRESH))
		{
			OutBoneTransform = NextEntry.GetComponentSpaceTransform(BoneTransformIndex);
		}
		else
		{
			OutBoneTransform.Blend(
				PrevEntry.GetComponentSpaceTransform(BoneTransformIndex),
				NextEntry.GetComponentSpaceTransform(BoneTransformIndex),
				LerpValue);
		}
	}
	else
	{
		if (FMath::IsNearlyZero(LerpValue, ZERO_ANIMWEIGHT_THRESH))
		{
			OutBoneTransform = PrevEntry.GetComponentSpaceTransform(BoneTransformIndex) * PrevEntry.GetComponentSpaceTransform(ReferenceBoneTransformIndex).Inverse();
		}
		else if (FMath::IsNearlyZero(LerpValue - 1.f, ZERO_ANIMWEIGHT_THRESH))
		{
			OutBoneTransform = NextEntry.GetComponentSpaceTransform(BoneTransformIndex) * NextEntry.GetComponentSpaceTransform(ReferenceBoneTransformIndex).Inverse();
		}
		else
		{
			OutBoneTransform.Blend(
				PrevEntry.GetComponentSpaceTransform(BoneTransformIndex) * PrevEntry.GetComponentSpaceTransform(ReferenceBoneTransformIndex).Inverse(),
				NextEntry.GetComponentSpaceTransform(BoneTransformIndex) * NextEntry.GetComponentSpaceTransform(ReferenceBoneTransformIndex).Inverse(),
				LerpValue);
		}
	}

	return bSuccess;
}

static uint32 GetTypeHash(const FBoneToTransformMap& BoneToTransformMap)
{
	const int32 Num = BoneToTransformMap.Num();

	if (Num == 0)
	{
		return 0;
	}

	TArrayView<FBoneToTransformPair> Pairs((FBoneToTransformPair*)FMemory_Alloca(Num * sizeof(FBoneToTransformPair)), Num);
	
	int32 Index = 0;
	for (const FBoneToTransformPair& BoneToTransformPair : BoneToTransformMap)
	{
		Pairs[Index] = BoneToTransformPair;
		++Index;
	}

	Pairs.StableSort();

	uint32 TypeHash = ::GetTypeHash(Pairs[0]);
	for (Index = 1; Index < Num; ++Index)
	{
		TypeHash = HashCombineFast(TypeHash, ::GetTypeHash(Pairs[Index]));
	}

	return TypeHash;
}

//////////////////////////////////////////////////////////////////////////
// FComponentSpacePoseProvider
FComponentSpacePoseProvider::FComponentSpacePoseProvider(FCSPose<FCompactPose>& InComponentSpacePose)
	: ComponentSpacePose(InComponentSpacePose)
{
	check(GetSkeletonAsset());
}

FTransform FComponentSpacePoseProvider::CalculateComponentSpaceTransform(const FSkeletonPoseBoneIndex SkeletonBoneIdx)
{
	const FBoneContainer& BoneContainer = ComponentSpacePose.GetPose().GetBoneContainer();
	const FCompactPoseBoneIndex CompactBoneIdx = BoneContainer.GetCompactPoseIndexFromSkeletonPoseIndex(SkeletonBoneIdx);
	if (CompactBoneIdx.IsValid())
	{
		return ComponentSpacePose.GetComponentSpaceTransform(CompactBoneIdx);
	}
	
	// NoTe: this chunk of code is very unlikely to be called, but in case:
	// @todo: cache any transform outside the domain of ComponentSpacePose if needed
	// @todo: use the skeletal mesh reference pose instead of the one from the skeleton if needed
	const USkeleton* Skeleton = BoneContainer.GetSkeletonAsset();
	const FReferenceSkeleton& ReferenceSkeleton = Skeleton->GetReferenceSkeleton();
	const int32 ParentIndex = ReferenceSkeleton.GetParentIndex(SkeletonBoneIdx.GetInt());
	check(ParentIndex >= 0);

	const TArray<FTransform>& RefBonePose = Skeleton->GetReferenceSkeleton().GetRefBonePose();
	return RefBonePose[SkeletonBoneIdx.GetInt()] * CalculateComponentSpaceTransform(FSkeletonPoseBoneIndex(ParentIndex));
}

const USkeleton* FComponentSpacePoseProvider::GetSkeletonAsset() const
{
	return ComponentSpacePose.GetPose().GetBoneContainer().GetSkeletonAsset();
}

//////////////////////////////////////////////////////////////////////////
// FAIPComponentSpacePoseProvider
FAIPComponentSpacePoseProvider::FAIPComponentSpacePoseProvider(const FAnimInstanceProxy* InAnimInstanceProxy)
{
	check(InAnimInstanceProxy);
	// initializing PoseHistory with a ref pose at FAnimInstanceProxy location/facing
	const FBoneContainer& BoneContainer = InAnimInstanceProxy->GetRequiredBones();

	// BoneContainer can be invalid when recompiling ABP while PIE is running
	if (BoneContainer.IsValid())
	{
		ComponentSpacePose.InitPose(&BoneContainer);
	}
}

FTransform FAIPComponentSpacePoseProvider::CalculateComponentSpaceTransform(const FSkeletonPoseBoneIndex SkeletonBoneIdx)
{
	// NoTe: calling GetBoneTransform on the mesh returns Identity on the first frame of simulation, so we use a different approach
	// const FMeshPoseBoneIndex MeshPoseBoneIndex = AnimInstanceProxy->GetRequiredBones().GetMeshPoseIndexFromSkeletonPoseIndex(SkeletonBoneIdx);
	// return AnimInstanceProxy->GetSkelMeshComponent()->GetBoneTransform(MeshPoseBoneIndex.GetInt(), FTransform::Identity);

	if (!ComponentSpacePose.GetPose().IsValid())
	{
		return FTransform::Identity;
	}

	const FBoneContainer& BoneContainer = ComponentSpacePose.GetPose().GetBoneContainer();
	const FCompactPoseBoneIndex CompactBoneIdx = BoneContainer.GetCompactPoseIndexFromSkeletonPoseIndex(SkeletonBoneIdx);
	if (CompactBoneIdx.IsValid())
	{
		return ComponentSpacePose.GetComponentSpaceTransform(CompactBoneIdx);
	}
	
	// NoTe: this chunk of code is very unlikely to be called, but in case:
	// @todo: cache any transform outside the domain of ComponentSpacePose if needed
	// @todo: use the skeletal mesh reference pose instead of the one from the skeleton if needed
	const USkeleton* Skeleton = BoneContainer.GetSkeletonAsset();
	const FReferenceSkeleton& ReferenceSkeleton = Skeleton->GetReferenceSkeleton();
	const int32 ParentIndex = ReferenceSkeleton.GetParentIndex(SkeletonBoneIdx.GetInt());

	const FTransform ParentTransform = (ParentIndex >= 0) ? CalculateComponentSpaceTransform(FSkeletonPoseBoneIndex(ParentIndex)) : FTransform::Identity;
	const TArray<FTransform>& RefBonePose = ReferenceSkeleton.GetRefBonePose();
	
	if (RefBonePose.IsValidIndex(SkeletonBoneIdx.GetInt()))
	{
		return RefBonePose[SkeletonBoneIdx.GetInt()] * ParentTransform;
	}

	return FTransform::Identity;
}

const USkeleton* FAIPComponentSpacePoseProvider::GetSkeletonAsset() const
{
	// return AnimInstanceProxy->GetSkelMeshComponent()->GetSkeletalMeshAsset()->GetSkeleton();
	if (ComponentSpacePose.GetPose().IsValid())
	{
		return ComponentSpacePose.GetPose().GetBoneContainer().GetSkeletonAsset();
	}
	return nullptr;
}

//////////////////////////////////////////////////////////////////////////
// FSKMCComponentSpacePoseProvider
FSKMCComponentSpacePoseProvider::FSKMCComponentSpacePoseProvider(const USkeletalMeshComponent* InSkeletalMeshComponent)
	: SkeletalMeshComponent(InSkeletalMeshComponent)
{
	if (ensure(SkeletalMeshComponent))
	{
		USkeletalMesh* SkeletalMeshAsset = SkeletalMeshComponent->GetSkeletalMeshAsset();
		if (ensure(SkeletalMeshAsset))
		{
			Skeleton = SkeletalMeshAsset->GetSkeleton();
			if (ensure(Skeleton))
			{
				const FSkeletonToMeshLinkup& SkeletonToMeshLinkupTable = Skeleton->FindOrAddMeshLinkupData(SkeletalMeshAsset);
				SkeletonToMeshTable = &SkeletonToMeshLinkupTable.SkeletonToMeshTable;
			}
		}
	}
}

FTransform FSKMCComponentSpacePoseProvider::CalculateComponentSpaceTransform(const FSkeletonPoseBoneIndex SkeletonBoneIdx)
{
	if (ensure(SkeletonToMeshTable))
	{
		check(SkeletalMeshComponent);

		if (SkeletonToMeshTable->IsValidIndex(SkeletonBoneIdx.GetInt()))
		{
			const int32 MeshIndex = (*SkeletonToMeshTable)[SkeletonBoneIdx.GetInt()];
			const TArray<FTransform>& ComponentSpaceTMs = SkeletalMeshComponent->GetComponentSpaceTransforms();
			if (ComponentSpaceTMs.IsValidIndex(MeshIndex))
			{
				return ComponentSpaceTMs[MeshIndex];
			}
		}
	}
	return FTransform::Identity;
}

const USkeleton* FSKMCComponentSpacePoseProvider::GetSkeletonAsset() const
{
	return Skeleton;
}

//////////////////////////////////////////////////////////////////////////
// FPoseHistoryEntry
void FPoseHistoryEntry::Update(float Time, IComponentSpacePoseProvider& ComponentSpacePoseProvider, const FBoneToTransformMap& BoneToTransformMap, bool bStoreScales, const FBlendedCurve& Curves, const TConstArrayView<FName>& CollectedCurves
#if WITH_EDITORONLY_DATA
	, const FTransform& InDebugMeshTransform
#endif //WITH_EDITORONLY_DATA
)
{
#if WITH_EDITORONLY_DATA
	DebugMeshTransform = InDebugMeshTransform;
#endif //WITH_EDITORONLY_DATA

	AccumulatedSeconds = Time;

	const USkeleton* Skeleton = ComponentSpacePoseProvider.GetSkeletonAsset();
	check(Skeleton);
	const int32 NumSkeletonBones = Skeleton->GetReferenceSkeleton().GetNum();
	if (BoneToTransformMap.IsEmpty())
	{
		// no mapping: we add all the transforms
		SetNum(NumSkeletonBones, bStoreScales);
		for (FSkeletonPoseBoneIndex SkeletonBoneIdx(0); SkeletonBoneIdx != NumSkeletonBones; ++SkeletonBoneIdx)
		{
			SetComponentSpaceTransform(SkeletonBoneIdx.GetInt(), ComponentSpacePoseProvider.CalculateComponentSpaceTransform(SkeletonBoneIdx));
		}
	}
	else
	{
		SetNum(BoneToTransformMap.Num(), true);
		for (const FBoneToTransformPair& BoneToTransformPair : BoneToTransformMap)
		{
			const FSkeletonPoseBoneIndex SkeletonBoneIdx(BoneToTransformPair.Key);
			SetComponentSpaceTransform(BoneToTransformPair.Value, ComponentSpacePoseProvider.CalculateComponentSpaceTransform(SkeletonBoneIdx));
		}
	}

	const int32 NumCurves = CollectedCurves.Num();
	CurveValues.SetNum(NumCurves);
	for (int i = 0; i < NumCurves; ++i)
	{
		const FName& CurveName = CollectedCurves[i];
		CurveValues[i] = Curves.Get(CurveName);
	}
}

void FPoseHistoryEntry::SetNum(int32 Num, bool bStoreScales)
{
	ComponentSpaceRotations.SetNum(Num);
	ComponentSpacePositions.SetNum(Num);
	ComponentSpaceScales.SetNum(bStoreScales ? Num : 0);
}

int32 FPoseHistoryEntry::Num() const
{
	return ComponentSpaceRotations.Num();
}

void FPoseHistoryEntry::SetComponentSpaceTransform(int32 Index, const FTransform& Transform)
{
	check( Transform.IsRotationNormalized() );
	ComponentSpaceRotations[Index] = FQuat4f(Transform.GetRotation());
	ComponentSpacePositions[Index] = Transform.GetTranslation();
	
	if (!ComponentSpaceScales.IsEmpty())
	{
		ComponentSpaceScales[Index] = FVector3f(Transform.GetScale3D());
	}
}

FTransform FPoseHistoryEntry::GetComponentSpaceTransform(int32 Index) const
{
	if (ComponentSpaceRotations.IsValidIndex(Index))
	{
		check(ComponentSpacePositions.Num() == ComponentSpaceRotations.Num());
		check(ComponentSpaceScales.IsEmpty() || ComponentSpaceRotations.Num() == ComponentSpaceScales.Num());

		const FQuat Quat(ComponentSpaceRotations[Index]);
		const FVector Scale(ComponentSpaceScales.IsEmpty() ? FVector3f::OneVector : ComponentSpaceScales[Index]);
		return FTransform(Quat, ComponentSpacePositions[Index], Scale);
	}

	UE_LOGF(LogPoseSearch, Error, "FPoseHistoryEntry::GetComponentSpaceTransform - Index %d out of bound [0, %d)", Index, ComponentSpaceRotations.Num());
	return FTransform::Identity;
}

float FPoseHistoryEntry::GetCurveValue(int32 Index) const
{
	if (CurveValues.IsValidIndex(Index))
	{
		return CurveValues[Index];
	}
	
	UE_LOGF(LogPoseSearch, Error, "FPoseHistoryEntry::GetCurveValue - Index %d out of bound [0, %d)", Index, CurveValues.Num());
	return 0.0f;
}

FArchive& operator<<(FArchive& Ar, FPoseHistoryEntry& Entry)
{
	Ar << Entry.ComponentSpaceRotations;
	Ar << Entry.ComponentSpacePositions;
	Ar << Entry.ComponentSpaceScales;
	Ar << Entry.CurveValues;
	Ar << Entry.AccumulatedSeconds;

	// this method is used by rewind debugger to archive data for pose search debugger and shared across game and editor.
	// when captures come from the game DebugMeshTransform will be invalid and displaying incorrectly when previewed by the editor
	// @todo: should we make DebugMeshTransform !WITH_EDITORONLY_DATA?
#if WITH_EDITORONLY_DATA
	Ar << Entry.DebugMeshTransform;
#else // WITH_EDITORONLY_DATA
	FTransform DebugMeshTransform = FTransform::Identity;
	Ar << DebugMeshTransform;
#endif // WITH_EDITORONLY_DATA

	return Ar;
}

//////////////////////////////////////////////////////////////////////////
// IPoseHistory
#if ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
void IPoseHistory::DebugDraw(FAnimInstanceProxy& AnimInstanceProxy, FColor Color, float Time, float PointSize, bool bExtrapolate) const
{
	const FBoneContainer& BoneContainer = AnimInstanceProxy.GetRequiredBones();
	if (Color.A > 0 && BoneContainer.IsValid())
	{
		const USkeleton* Skeleton = BoneContainer.GetSkeletonAsset();
		check(Skeleton);

		FTransform OutBoneTransform;

		const FBoneToTransformMap& BoneToTransformMap = GetBoneToTransformMap();
		if (BoneToTransformMap.IsEmpty())
		{
			const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
			const int32 NumSkeletonBones = RefSkeleton.GetNum();
			for (FSkeletonPoseBoneIndex SkeletonBoneIdx(0); SkeletonBoneIdx != NumSkeletonBones; ++SkeletonBoneIdx)
			{
				if (GetTransformAtTime(Time, OutBoneTransform, Skeleton, SkeletonBoneIdx.GetInt(), WorldSpaceIndexType, bExtrapolate))
				{
					AnimInstanceProxy.AnimDrawDebugPoint(OutBoneTransform.GetTranslation(), PointSize, Color, false, 0.f, ESceneDepthPriorityGroup::SDPG_Foreground);
				}
			}
		}
		else
		{
			for (const FBoneToTransformPair& BoneToTransformPair : BoneToTransformMap)
			{
				const FSkeletonPoseBoneIndex SkeletonBoneIdx(BoneToTransformPair.Key);
				if (GetTransformAtTime(Time, OutBoneTransform, Skeleton, SkeletonBoneIdx.GetInt(), WorldSpaceIndexType, bExtrapolate))
				{
					AnimInstanceProxy.AnimDrawDebugPoint(OutBoneTransform.GetTranslation(), PointSize, Color, false, 0.f, ESceneDepthPriorityGroup::SDPG_Foreground);
				}
			}
		}
	}
}
#endif // ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG

//////////////////////////////////////////////////////////////////////////
// FArchivedPoseHistory
void FArchivedPoseHistory::InitFrom(const IPoseHistory* PoseHistory)
{
	check(PoseHistory);

	Trajectory.Samples.Reset();
	BoneToTransformMap.Reset();
	Entries.Reset();
	
	Trajectory = PoseHistory->GetTrajectory();
	BoneToTransformMap = PoseHistory->GetBoneToTransformMap();
	CollectedCurves = PoseHistory->GetCollectedCurves();
	const int32 NumEntries = PoseHistory->GetNumEntries();
	Entries.SetNum(NumEntries);

	for (int32 EntryIndex = 0; EntryIndex < NumEntries; ++EntryIndex)
	{
		Entries[EntryIndex] = PoseHistory->GetEntry(EntryIndex);
		// validating input PoseHistory to have Entries properly sorted by time
		check(EntryIndex == 0 || (Entries[EntryIndex - 1].AccumulatedSeconds <= Entries[EntryIndex].AccumulatedSeconds));
	}
}

// in here FBoneIndexType BoneIndexType is a skeleton bone index, and it's used to dereference a BoneToTransformMap (skeleton bone index -> pose history bone index)
bool FArchivedPoseHistory::GetTransformAtTime(float Time, FTransform& OutBoneTransform, const USkeleton* BoneIndexSkeleton, FBoneIndexType BoneIndexType, FBoneIndexType ReferenceBoneIndexType, bool bExtrapolate) const
{
	static_assert(RootBoneIndexType == 0 && ComponentSpaceIndexType == FBoneIndexType(-1) && WorldSpaceIndexType == FBoneIndexType(-2)); // some assumptions
	
	if (BoneIndexType == ReferenceBoneIndexType)
	{
		OutBoneTransform = FTransform::Identity;
		return true;
	}

	if (ReferenceBoneIndexType == WorldSpaceIndexType)
	{
		if (BoneIndexType == ComponentSpaceIndexType)
		{
			OutBoneTransform = Trajectory.GetSampleAtTime(Time, bExtrapolate).GetTransform();
			return true;
		}

		// getting BoneIndexType in ComponentSpaceIndexType and then multiplying it with the component to world transform from the trajectory
		const bool bSuccess = GetTransformAtTime(Time, OutBoneTransform, BoneIndexSkeleton, BoneIndexType, ComponentSpaceIndexType, bExtrapolate);
		OutBoneTransform *= Trajectory.GetSampleAtTime(Time, bExtrapolate).GetTransform();
		return bSuccess;
	}

	if (BoneIndexType == WorldSpaceIndexType || BoneIndexType == ComponentSpaceIndexType)
	{
		const bool bSuccess = GetTransformAtTime(Time, OutBoneTransform, BoneIndexSkeleton, ReferenceBoneIndexType, BoneIndexType, bExtrapolate);
		OutBoneTransform = OutBoneTransform.Inverse();
		return bSuccess;
	}

	check(BoneIndexType != ComponentSpaceIndexType && BoneIndexType != WorldSpaceIndexType && ReferenceBoneIndexType != WorldSpaceIndexType);

	const int32 NumEntries = Entries.Num();
	if (NumEntries > 0)
	{
		int32 NextIdx = 0;
		int32 PrevIdx = 0;

		if (NumEntries > 1)
		{
			const int32 LowerBoundIdx = Algo::LowerBound(Entries, Time, [](const FPoseHistoryEntry& Entry, float Value) { return Value > Entry.AccumulatedSeconds; });
			NextIdx = FMath::Clamp(LowerBoundIdx, 1, NumEntries - 1);
			PrevIdx = NextIdx - 1;
		}
	
		const FPoseHistoryEntry& PrevEntry = Entries[PrevIdx];
		const FPoseHistoryEntry& NextEntry = Entries[NextIdx];

		return LerpEntries(Time, bExtrapolate, PrevEntry, NextEntry, BoneIndexSkeleton, nullptr, BoneToTransformMap, BoneIndexType, ReferenceBoneIndexType, OutBoneTransform);
	}

	OutBoneTransform = FTransform::Identity;
	return false;
}

bool FArchivedPoseHistory::GetCurveValueAtTime(float Time, const FName& CurveName, float& OutCurveValue, bool bExtrapolate /*= false*/) const
{
	bool bSuccess = false;

	const int32 NumEntries = Entries.Num();
	if (NumEntries > 0)
	{
		int32 NextIdx = 0;
		int32 PrevIdx = 0;

		if (NumEntries > 1)
		{
			const int32 LowerBoundIdx = Algo::LowerBound(Entries, Time, [](const FPoseHistoryEntry& Entry, float Value) { return Value > Entry.AccumulatedSeconds; });
			NextIdx = FMath::Clamp(LowerBoundIdx, 1, NumEntries - 1);
			PrevIdx = NextIdx - 1;
		}

		const FPoseHistoryEntry& PrevEntry = Entries[PrevIdx];
		const FPoseHistoryEntry& NextEntry = Entries[NextIdx];

		bSuccess = LerpEntries(Time, bExtrapolate, PrevEntry, NextEntry, CurveName, GetCollectedCurves(), OutCurveValue);
	}
	else
	{
		OutCurveValue = 0.0f;
	}

	return bSuccess;
}

#if ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
void FArchivedPoseHistory::DebugDraw(const UWorld* World, FColor Color) const
{
	if (Color.A > 0 && !Trajectory.Samples.IsEmpty())
	{
		TArray<FTransform, TInlineAllocator<128>> PrevGlobalTransforms;

		for (int32 EntryIndex = 0; EntryIndex < Entries.Num(); ++EntryIndex)
		{
			const FPoseHistoryEntry& Entry = Entries[EntryIndex];

			const int32 PrevGlobalTransformsNum = PrevGlobalTransforms.Num();
			const int32 Max = FMath::Max(PrevGlobalTransformsNum, Entry.Num());

			PrevGlobalTransforms.SetNum(Max, EAllowShrinking::No);

			const bool bIsCurrentTimeEntry = FMath::IsNearlyZero(Entry.AccumulatedSeconds);

#if WITH_EDITORONLY_DATA
			if (bIsCurrentTimeEntry)
			{
				DrawDebugLine(World, Entry.DebugMeshTransform.GetTranslation(), Entry.DebugMeshTransform.GetTranslation() + Entry.DebugMeshTransform.GetUnitAxis(EAxis::Type::X) * 30.f, FColor::Turquoise, false, -1.f, SDPG_Foreground);
			}
#endif // WITH_EDITORONLY_DATA

			for (int32 i = 0; i < Entry.Num(); ++i)
			{
				const FTransform RootTransform = Trajectory.GetSampleAtTime(Entry.AccumulatedSeconds).GetTransform();
				const FTransform GlobalTransforms = Entry.GetComponentSpaceTransform(i) * RootTransform;

				if (i < PrevGlobalTransformsNum)
				{
					DrawDebugLine(World, PrevGlobalTransforms[i].GetTranslation(), GlobalTransforms.GetTranslation(), Color, false, -1.f, SDPG_Foreground);
				}

				if (bIsCurrentTimeEntry)
				{
					DrawDebugPoint(World, GlobalTransforms.GetTranslation(), 6.f, Color, false, -1.f, SDPG_Foreground);

					if (i == 0)
					{
						DrawDebugLine(World, GlobalTransforms.GetTranslation(), GlobalTransforms.GetTranslation() + RootTransform.GetUnitAxis(EAxis::Type::X) * 25.f, FColor::Black, false, -1.f, SDPG_Foreground);
						DrawDebugLine(World, GlobalTransforms.GetTranslation(), GlobalTransforms.GetTranslation() + GlobalTransforms.GetUnitAxis(EAxis::Type::X) * 20.f, FColor::White, false, -1.f, SDPG_Foreground);
					}
				}

				if (i == 0)
				{
					DrawDebugLine(World, GlobalTransforms.GetTranslation(), RootTransform.GetTranslation(), FColor::Purple, false, -1.f, SDPG_Foreground);
				}

				PrevGlobalTransforms[i] = GlobalTransforms;
			}
		}
	}
}
#endif // ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG

FArchive& operator<<(FArchive& Ar, FArchivedPoseHistory& Entry)
{
	Ar << Entry.BoneToTransformMap;
	Ar << Entry.CollectedCurves;
	Ar << Entry.Entries;
	
	// Convert old FPoseSearchTrajectory to new FTransformTrajectory type at load time.
	if (Ar.CustomVer(FPoseSearchCustomVersion::GUID) < FPoseSearchCustomVersion::DeprecatedTrajectoryTypes)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FPoseSearchQueryTrajectory OldTrajectoryType;
		Ar << OldTrajectoryType;
		Entry.Trajectory = OldTrajectoryType;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}	
	else
	{
		Ar << Entry.Trajectory;
	}
	
	return Ar;
}

//////////////////////////////////////////////////////////////////////////
// FPoseHistory
FPoseHistory::FPoseHistory(const FPoseHistory& Other)
	: FPoseHistory()
{
	*this = Other;
}

FPoseHistory::FPoseHistory(FPoseHistory&& Other)
	: FPoseHistory()
{
	*this = MoveTemp(Other);
}

FPoseHistory& FPoseHistory::operator=(const FPoseHistory& Other)
{
	if (this != &Other)
	{
#if ENABLE_ANIM_DEBUG
		UE_MT_SCOPED_READ_ACCESS(Other.PoseDataThreadSafeCounter);
		UE_MT_SCOPED_WRITE_ACCESS(PoseDataThreadSafeCounter);
#endif // ENABLE_ANIM_DEBUG

		PoseSearchHistory = Other.PoseSearchHistory;
		TrajectoryDataState = Other.TrajectoryDataState;
	}
	return *this;
}

FPoseHistory& FPoseHistory::operator=(FPoseHistory&& Other)
{
	if (this != &Other)
	{
#if ENABLE_ANIM_DEBUG
		UE_MT_SCOPED_WRITE_ACCESS(Other.PoseDataThreadSafeCounter);
		UE_MT_SCOPED_WRITE_ACCESS(PoseDataThreadSafeCounter);
#endif // ENABLE_ANIM_DEBUG

		PoseSearchHistory = MoveTemp(Other.PoseSearchHistory);
		TrajectoryDataState = MoveTemp(Other.TrajectoryDataState);
	}
	return *this;
}

void FPoseHistory::Initialize_AnyThread(int32 InNumPoses, float InSamplingInterval)
{
	UE_MT_SCOPED_WRITE_ACCESS(PoseDataThreadSafeCounter);
	PoseSearchHistory.Initialize(InNumPoses, InSamplingInterval);
	TrajectoryDataState = FPoseSearchTrajectoryData::FState();
}

bool FPoseHistory::GetTransformAtTime(float Time, FTransform& OutBoneTransform, const USkeleton* BoneIndexSkeleton, FBoneIndexType BoneIndexType, FBoneIndexType ReferenceBoneIndexType, bool bExtrapolate) const
{
	UE_MT_SCOPED_READ_ACCESS(PoseDataThreadSafeCounter);
	return PoseSearchHistory.GetTransformAtTime(Time, OutBoneTransform, BoneIndexSkeleton, BoneIndexType, ReferenceBoneIndexType, bExtrapolate);
}

bool FPoseHistory::GetCurveValueAtTime(float Time, const FName& CurveName, float& OutCurveValue, bool bExtrapolate /*= false*/) const
{
	UE_MT_SCOPED_READ_ACCESS(PoseDataThreadSafeCounter);
	return PoseSearchHistory.GetCurveValueAtTime(Time, CurveName, OutCurveValue, bExtrapolate);
}

const FTransformTrajectory& FPoseHistory::GetTrajectory() const
{
	UE_MT_SCOPED_READ_ACCESS(PoseDataThreadSafeCounter);
	return PoseSearchHistory.GetTrajectory();
}

float FPoseHistory::GetTrajectorySpeedMultiplier() const
{
	UE_MT_SCOPED_READ_ACCESS(PoseDataThreadSafeCounter);
	return PoseSearchHistory.GetTrajectorySpeedMultiplier();
}

bool FPoseHistory::IsEmpty() const
{
	UE_MT_SCOPED_READ_ACCESS(PoseDataThreadSafeCounter);
	return PoseSearchHistory.IsEmpty();
}

const FBoneToTransformMap& FPoseHistory::GetBoneToTransformMap() const
{
	UE_MT_SCOPED_READ_ACCESS(PoseDataThreadSafeCounter);
	return PoseSearchHistory.GetBoneToTransformMap();
}

const TConstArrayView<FName> FPoseHistory::GetCollectedCurves() const
{
	UE_MT_SCOPED_READ_ACCESS(PoseDataThreadSafeCounter);
	return PoseSearchHistory.GetCollectedCurves();
}

int32 FPoseHistory::GetNumEntries() const
{
	UE_MT_SCOPED_READ_ACCESS(PoseDataThreadSafeCounter);
	return PoseSearchHistory.GetNumEntries();
}

const FPoseHistoryEntry& FPoseHistory::GetEntry(int32 EntryIndex) const
{
	UE_MT_SCOPED_READ_ACCESS(PoseDataThreadSafeCounter);
	return PoseSearchHistory.GetEntry(EntryIndex);
}

const FPoseIndicesHistory* FPoseHistory::GetPoseIndicesHistory() const
{
	UE_MT_SCOPED_READ_ACCESS(PoseDataThreadSafeCounter);
	return PoseSearchHistory.GetPoseIndicesHistory();
}

int32 FPoseHistory::GetMaxNumPoses() const
{
	UE_MT_SCOPED_READ_ACCESS(PoseDataThreadSafeCounter);
	return PoseSearchHistory.GetMaxNumPoses();
}

float FPoseHistory::GetSamplingInterval() const
{
	UE_MT_SCOPED_READ_ACCESS(PoseDataThreadSafeCounter);
	return PoseSearchHistory.GetSamplingInterval();
}

void FPoseHistory::SetSamplingInterval(float InSamplingInterval)
{
	UE_MT_SCOPED_WRITE_ACCESS(PoseDataThreadSafeCounter);
	return PoseSearchHistory.SetSamplingInterval(InSamplingInterval);
}

const FPoseSearchHistory& FPoseHistory::GetPoseSearchHistory() const
{
	UE_MT_SCOPED_READ_ACCESS(PoseDataThreadSafeCounter);
	return PoseSearchHistory;
}

FPoseSearchHistory& FPoseHistory::EditPoseSearchHistory()
{
	UE_MT_SCOPED_WRITE_ACCESS(PoseDataThreadSafeCounter);
	return PoseSearchHistory;
}

void FPoseHistory::GenerateTrajectory(const UObject* AnimContext, float DeltaTime, const FPoseSearchTrajectoryData& TrajectoryData, const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling)
{
	// @todo: Synchronize the FPoseSearchQueryTrajectorySample::AccumulatedSeconds of the generated trajectory with the FPoseHistoryEntry::AccumulatedSeconds of the captured poses
	FPoseSearchTrajectoryData::FDerived TrajectoryDataDerived;
	if (TrajectoryData.UpdateData(DeltaTime, AnimContext, TrajectoryDataDerived, TrajectoryDataState))
	{
		UPoseSearchTrajectoryLibrary::InitTrajectorySamples(PoseSearchHistory.EditTrajectory(), TrajectoryDataDerived.Position, TrajectoryDataDerived.Facing, TrajectoryDataSampling, DeltaTime);
		UPoseSearchTrajectoryLibrary::UpdateHistory_TransformHistory(PoseSearchHistory.EditTrajectory(), TrajectoryDataDerived.Position, TrajectoryDataDerived.Velocity, TrajectoryDataSampling, DeltaTime);
		UPoseSearchTrajectoryLibrary::UpdatePrediction_SimulateCharacterMovement(PoseSearchHistory.EditTrajectory(), TrajectoryData, TrajectoryDataDerived, TrajectoryDataSampling, DeltaTime);

		// @todo: support TrajectorySpeedMultiplier
		//TrajectorySpeedMultiplier = 1.f;
	}
}

void FPoseHistory::GenerateTrajectory(const UObject* AnimContext, float DeltaTime)
{
	GenerateTrajectory(AnimContext, DeltaTime, FPoseSearchTrajectoryData(), FPoseSearchTrajectoryData::FSampling());
}

void FPoseHistory::SetTrajectory(const FTransformTrajectory& InTrajectory, float InTrajectorySpeedMultiplier)
{
	if (!InTrajectory.Samples.IsEmpty())
	{
		UE_MT_SCOPED_WRITE_ACCESS(PoseDataThreadSafeCounter);
		PoseSearchHistory.SetTrajectory(InTrajectory, InTrajectorySpeedMultiplier);
	}
}

void FPoseHistory::EvaluateComponentSpace_AnyThread(float DeltaTime, IComponentSpacePoseProvider& ComponentSpacePoseProvider, bool bStoreScales,
	float RootBoneRecoveryTime, float RootBoneTranslationRecoveryRatio, float RootBoneRotationRecoveryRatio,
	bool bNeedsReset, bool bCacheBones, const TArray<FBoneIndexType>& RequiredBones,
	const FBlendedCurve& Curves, const TConstArrayView<FName>& InCollectedCurves
#if WITH_EDITORONLY_DATA
	, const FTransform& InDebugMeshTransform
#endif //WITH_EDITORONLY_DATA
	)
{
	if (!ensure(PoseSearchHistory.GetMaxNumPoses() >= 2))
	{
		UE_LOGF(LogPoseSearch, Error, "FPoseHistory::EvaluateComponentSpace_AnyThread called before FPoseHistory::Initialize_AnyThread! Please double check the implementation of any FAnimNode_Base calling this API, correctly propagating Initialize_AnyThread to their FPoseLink(s)! To mitigate issues FPoseHistory will be set to to default values");
		Initialize_AnyThread(2, 0.f);
	}

	UE_MT_SCOPED_WRITE_ACCESS(PoseDataThreadSafeCounter);

	PoseSearchHistory.EvaluateComponentSpace(DeltaTime, ComponentSpacePoseProvider, bStoreScales, RootBoneRecoveryTime
		, RootBoneTranslationRecoveryRatio, RootBoneRotationRecoveryRatio, bNeedsReset, bCacheBones, RequiredBones, Curves, InCollectedCurves
#if WITH_EDITORONLY_DATA
		, InDebugMeshTransform
#endif //WITH_EDITORONLY_DATA
	);
}

#if ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
void FPoseHistory::DebugDraw(FAnimInstanceProxy& AnimInstanceProxy, FColor Color) const
{
	UE_MT_SCOPED_READ_ACCESS(PoseDataThreadSafeCounter);

	if (GVarAnimPoseHistoryDebugDrawTrajectory)
	{
		const float DebugThickness = GVarAnimPoseHistoryDebugDrawTrajectoryThickness;
		const int MaxHistorySamples = GVarAnimPoseHistoryDebugDrawTrajectoryMaxNumOfHistorySamples;
		const int MaxPredictionSamples = GVarAnimPoseHistoryDebugDrawTrajectoryMaxNumOfPredictionSamples;
		UTransformTrajectoryBlueprintLibrary::DebugDrawTrajectory(PoseSearchHistory.GetTrajectory(), AnimInstanceProxy, DebugThickness, 0, MaxHistorySamples, MaxPredictionSamples);
	}

	if (Color.A > 0 && GVarAnimPoseHistoryDebugDrawPose)
	{
		const bool bValidTrajectory = !PoseSearchHistory.GetTrajectory().Samples.IsEmpty();
		TArray<FTransform, TInlineAllocator<128>> PrevGlobalTransforms;

		for (int32 EntryIndex = 0; EntryIndex < PoseSearchHistory.GetNumEntries(); ++EntryIndex)
		{
			const FPoseHistoryEntry& Entry = PoseSearchHistory.GetEntry(EntryIndex);

			const int32 PrevGlobalTransformsNum = PrevGlobalTransforms.Num();
			const int32 Max = FMath::Max(PrevGlobalTransformsNum, Entry.Num());

			PrevGlobalTransforms.SetNum(Max, EAllowShrinking::No);

			for (int32 i = 0; i < Entry.Num(); ++i)
			{
				const FTransform RootTransform = bValidTrajectory ? PoseSearchHistory.GetTrajectory().GetSampleAtTime(Entry.AccumulatedSeconds).GetTransform() : AnimInstanceProxy.GetComponentTransform();
				const FTransform GlobalTransforms = Entry.GetComponentSpaceTransform(i) * RootTransform;

				if (i < PrevGlobalTransformsNum)
				{
					AnimInstanceProxy.AnimDrawDebugLine(PrevGlobalTransforms[i].GetTranslation(), GlobalTransforms.GetTranslation(), Color, false, 0.f, ESceneDepthPriorityGroup::SDPG_Foreground);
				}

				PrevGlobalTransforms[i] = GlobalTransforms;
			}
		}
	}
}
#endif // ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG

//////////////////////////////////////////////////////////////////////////
// FGenerateTrajectoryPoseHistory
void FGenerateTrajectoryPoseHistory::GenerateTrajectory(const UObject* AnimContext, float DeltaTime)
{
	if (bGenerateTrajectory && !bIsTrajectoryGeneratedBeforePreUpdate)
	{
		FPoseHistory::GenerateTrajectory(AnimContext, DeltaTime, TrajectoryData, TrajectoryDataSampling);
		bIsTrajectoryGeneratedBeforePreUpdate = true;
	}
}

//////////////////////////////////////////////////////////////////////////
// FMemStackPoseHistory
void FMemStackPoseHistory::Init(const IPoseHistory* InPoseHistory)
{
	check(InPoseHistory);
	PoseHistory = InPoseHistory;
}

void FMemStackPoseHistory::SetTrajectory(const FTransformTrajectory& InTrajectory, float InTrajectorySpeedMultiplier)
{
	check(PoseHistory);
	// FMemStackPoseHistory should never change the trajectory!
	checkNoEntry();
}

void FMemStackPoseHistory::GenerateTrajectory(const UObject* AnimContext, float DeltaTime)
{
	check(PoseHistory);
	// FMemStackPoseHistory should never change the trajectory!
	checkNoEntry();
}

bool FMemStackPoseHistory::GetTransformAtTime(float Time, FTransform& OutBoneTransform, const USkeleton* BoneIndexSkeleton, FBoneIndexType BoneIndexType, FBoneIndexType ReferenceBoneIndexType, bool bExtrapolate) const
{
	check(PoseHistory);

	const int32 Num = FutureEntries.Num();
	if (Time > 0.f && Num > 0)
	{
		if (BoneIndexType == ReferenceBoneIndexType)
		{
			OutBoneTransform = FTransform::Identity;
			return true;
		}

		if (ReferenceBoneIndexType == WorldSpaceIndexType)
		{
			const FTransformTrajectory& Trajectory = GetTrajectory();

			if (BoneIndexType == ComponentSpaceIndexType)
			{
				OutBoneTransform = Trajectory.GetSampleAtTime(Time, bExtrapolate).GetTransform();
				return true;
			}

			// getting BoneIndexType in ComponentSpaceIndexType and then multiplying it with the component to world transform from the trajectory
			const bool bSuccess = GetTransformAtTime(Time, OutBoneTransform, BoneIndexSkeleton, BoneIndexType, ComponentSpaceIndexType, bExtrapolate);
			OutBoneTransform *= Trajectory.GetSampleAtTime(Time, bExtrapolate).GetTransform();
			return bSuccess;
		}

		if (BoneIndexType == WorldSpaceIndexType || BoneIndexType == ComponentSpaceIndexType)
		{
			const bool bSuccess = GetTransformAtTime(Time, OutBoneTransform, BoneIndexSkeleton, ReferenceBoneIndexType, BoneIndexType, bExtrapolate);
			OutBoneTransform = OutBoneTransform.Inverse();
			return bSuccess;
		}

		check(BoneIndexType != ComponentSpaceIndexType && BoneIndexType != WorldSpaceIndexType && ReferenceBoneIndexType != WorldSpaceIndexType);

		const int32 LowerBoundIdx = Algo::LowerBound(FutureEntries, Time, [](const FPoseHistoryEntry& Entry, float Value) { return Value > Entry.AccumulatedSeconds; });
		const int32 NextIdx = FMath::Min(LowerBoundIdx, Num - 1);
		const FPoseHistoryEntry& NextEntry = FutureEntries[NextIdx];
		const FPoseHistoryEntry& PrevEntry = NextIdx > 0 ? FutureEntries[NextIdx - 1] : PoseHistory->GetNumEntries() > 0 ? PoseHistory->GetEntry(PoseHistory->GetNumEntries() - 1) : NextEntry;
						
		return LerpEntries(Time, bExtrapolate, PrevEntry, NextEntry, BoneIndexSkeleton, nullptr, GetBoneToTransformMap(), BoneIndexType, ReferenceBoneIndexType, OutBoneTransform);
	}
	
	return PoseHistory->GetTransformAtTime(Time, OutBoneTransform, BoneIndexSkeleton, BoneIndexType, ReferenceBoneIndexType, bExtrapolate);
}

bool FMemStackPoseHistory::GetCurveValueAtTime(float Time, const FName& CurveName, float& OutCurveValue, bool bExtrapolate /*= false*/) const
{
	check(PoseHistory);
	
	const int32 Num = FutureEntries.Num();
	if (Time > 0.f && Num > 0)
	{
		const int32 LowerBoundIdx = Algo::LowerBound(FutureEntries, Time, [](const FPoseHistoryEntry& Entry, float Value) { return Value > Entry.AccumulatedSeconds; });
		const int32 NextIdx = FMath::Min(LowerBoundIdx, Num - 1);
		const FPoseHistoryEntry& NextEntry = FutureEntries[NextIdx];
		const FPoseHistoryEntry& PrevEntry = NextIdx > 0 ? FutureEntries[NextIdx - 1] : PoseHistory->GetNumEntries() > 0 ? PoseHistory->GetEntry(PoseHistory->GetNumEntries() - 1) : NextEntry;

		return LerpEntries(Time, bExtrapolate, PrevEntry, NextEntry, CurveName, GetCollectedCurves(), OutCurveValue);
	}

	return PoseHistory->GetCurveValueAtTime(Time, CurveName, OutCurveValue, bExtrapolate);
}

void FMemStackPoseHistory::AddFutureRootBone(float Time, const FTransform& FutureRootBoneTransform, bool bStoreScales)
{
	// we don't allow to add "past" or "present" poses to FutureEntries
	check(Time > 0.f);

	const int32 LowerBoundIdx = Algo::LowerBound(FutureEntries, Time, [](const FPoseHistoryEntry& Entry, float Value) { return Value > Entry.AccumulatedSeconds; });
	FPoseHistoryEntry& FutureEntry = FutureEntries.InsertDefaulted_GetRef(LowerBoundIdx);
	FutureEntry.SetNum(1, bStoreScales);
	FutureEntry.SetComponentSpaceTransform(RootBoneIndexType, FutureRootBoneTransform);
	FutureEntry.AccumulatedSeconds = Time;
}

void FMemStackPoseHistory::AddFuturePose(float Time, IComponentSpacePoseProvider& ComponentSpacePoseProvider, const FBlendedCurve& Curves
#if WITH_EDITORONLY_DATA
	, const FTransform& InDebugMeshTransform
#endif //WITH_EDITORONLY_DATA
	)
{
	// we don't allow to add "past" or "present" poses to FutureEntries
	check(Time > 0.f);
	check(PoseHistory);	
	const int32 LowerBoundIdx = Algo::LowerBound(FutureEntries, Time, [](const FPoseHistoryEntry& Entry, float Value) { return Value > Entry.AccumulatedSeconds; });
	FutureEntries.InsertDefaulted_GetRef(LowerBoundIdx).Update(Time, ComponentSpacePoseProvider, GetBoneToTransformMap(), true, Curves, MakeConstArrayView(GetCollectedCurves())
#if WITH_EDITORONLY_DATA
	, InDebugMeshTransform
#endif //WITH_EDITORONLY_DATA
	);
}

void FMemStackPoseHistory::ExtractAndAddFuturePoses(const UAnimationAsset* AnimationAsset, float AnimationTime, float FiniteDeltaTime, const FVector& BlendParame1ters, float IntervalTime, const USkeleton* OverrideSkeleton, bool bUseRefPoseRootBone
#if WITH_EDITORONLY_DATA
	, const FTransform& InDebugMeshTransform
#endif //WITH_EDITORONLY_DATA
)
{
	if (!ensure(AnimationAsset))
	{
		UE_LOGF(LogPoseSearch, Error, "FMemStackPoseHistory::ExtractAndAddFuturePose - input AnimationAsset is null");
		return;
	}

	FMemMark Mark(FMemStack::Get());

	check(FiniteDeltaTime >= 0.f);
	if (AnimationTime < FiniteDeltaTime)
	{
		UE_LOGF(LogPoseSearch, Error, "FMemStackPoseHistory::ExtractAndAddFuturePose - provided AnimationTime (%f) is too small. Clamping it to minimum value of %f", AnimationTime, FiniteDeltaTime);
		AnimationTime = FiniteDeltaTime;
	}

	if (IntervalTime < FiniteDeltaTime)
	{
		UE_LOGF(LogPoseSearch, Error, "FMemStackPoseHistory::ExtractAndAddFuturePose - provided IntervalTime (%f) is too small. Clamping it to minimum value of %f", IntervalTime, FiniteDeltaTime);
		IntervalTime = FiniteDeltaTime;
	}

	const USkeleton* Skeleton = OverrideSkeleton ? OverrideSkeleton : AnimationAsset->GetSkeleton();
	TArray<uint16> BoneIndices;
	const FBoneToTransformMap& BoneToTransformMap = GetBoneToTransformMap();
	if (BoneToTransformMap.IsEmpty())
	{
		const int32 NumBones = Skeleton->GetReferenceSkeleton().GetNum();
		BoneIndices.SetNum(NumBones);
		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			BoneIndices[BoneIndex] = BoneIndex;
		}
	}
	else
	{
		for (const FBoneToTransformPair& BoneToTransformPair : BoneToTransformMap)
		{
			BoneIndices.Add(BoneToTransformPair.Key);
		}

		BoneIndices.Sort();
		FAnimationRuntime::EnsureParentsPresent(BoneIndices, Skeleton->GetReferenceSkeleton());
	}

	FBoneContainer BoneContainer;
	BoneContainer.InitializeTo(BoneIndices, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *Skeleton);

	// extracting 2 poses to be able to calculate velocities
	FCSPose<FCompactPose> ComponentSpacePose;
	FCompactPose Pose;
	FBlendedCurve Curves;
	Pose.SetBoneContainer(&BoneContainer);

	// extracting 2 poses to be able to calculate velocities
	const FAnimationAssetSampler Sampler(AnimationAsset, FTransform::Identity, FVector::ZeroVector, FAnimationAssetSampler::DefaultRootTransformSamplingRate, false, false);

	const int32 NumOfPoseExtractions = FMath::IsNearlyZero(FiniteDeltaTime) ? 1 : 2;
	for (int32 i = 0; i < NumOfPoseExtractions; ++i)
	{
		const float FuturePoseExtractionTime = AnimationTime + (i - 1) * FiniteDeltaTime;
		const float FuturePoseAnimationTime = IntervalTime + (i - 1) * FiniteDeltaTime;

		Sampler.ExtractPose(FuturePoseExtractionTime, Pose, Curves);

		if (bUseRefPoseRootBone)
		{
			Pose[FCompactPoseBoneIndex(RootBoneIndexType)] = Skeleton->GetReferenceSkeleton().GetRefBonePose()[RootBoneIndexType];
		}

		ComponentSpacePose.InitPose(Pose);
		FComponentSpacePoseProvider ComponentSpacePoseProvider(ComponentSpacePose);
		AddFuturePose(FuturePoseAnimationTime, ComponentSpacePoseProvider
#if WITH_EDITORONLY_DATA
			, FBlendedCurve()
			// @todo: calculate the predicted future InDebugMeshTransform properly, by extracting the delta transform between the 
			//        FuturePoseExtractionTime - IntervalTime and FuturePoseExtractionTime and applying it to InDebugMeshTransform!
			, InDebugMeshTransform
#endif // WITH_EDITORONLY_DATA
		);
	}
}

int32 FMemStackPoseHistory::GetNumEntries() const
{
	check(PoseHistory);
	return PoseHistory->GetNumEntries() + FutureEntries.Num();
}

const FPoseHistoryEntry& FMemStackPoseHistory::GetEntry(int32 EntryIndex) const
{
	check(PoseHistory);

	const int32 PoseHistoryNumEntries = PoseHistory->GetNumEntries();
	if (EntryIndex < PoseHistoryNumEntries)
	{
		return PoseHistory->GetEntry(EntryIndex);
	}
	return FutureEntries[EntryIndex - PoseHistoryNumEntries];
}

const FPoseIndicesHistory* FMemStackPoseHistory::GetPoseIndicesHistory() const
{
	check(PoseHistory);
	return PoseHistory->GetPoseIndicesHistory();
}

#if ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
void FMemStackPoseHistory::DebugDraw(FAnimInstanceProxy& AnimInstanceProxy, FColor Color) const
{
	check(PoseHistory);

	if (Color.A > 0 && !FutureEntries.IsEmpty() && GVarAnimPoseHistoryDebugDrawPose)
	{
		const FTransformTrajectory& Trajectory = GetTrajectory();
		const bool bValidTrajectory = !Trajectory.Samples.IsEmpty();
		TArray<FTransform, TInlineAllocator<128>> PrevGlobalTransforms;

		int32 EntriesNum = FutureEntries.Num();
		if (PoseHistory->GetNumEntries() > 0)
		{
			// connecting the future entries with the past entries
			++EntriesNum;
		}

		for (int32 EntryIndex = 0; EntryIndex < EntriesNum; ++EntryIndex)
		{
			const FPoseHistoryEntry& Entry = (EntryIndex == FutureEntries.Num()) ? PoseHistory->GetEntry(PoseHistory->GetNumEntries() - 1) : FutureEntries[EntryIndex];

			const int32 PrevGlobalTransformsNum = PrevGlobalTransforms.Num();
			const int32 Max = FMath::Max(PrevGlobalTransformsNum, Entry.Num());

			PrevGlobalTransforms.SetNum(Max, EAllowShrinking::No);

			for (int32 i = 0; i < Entry.Num(); ++i)
			{
				const FTransform RootTransform = bValidTrajectory ? Trajectory.GetSampleAtTime(Entry.AccumulatedSeconds).GetTransform() : AnimInstanceProxy.GetComponentTransform();
				const FTransform GlobalTransforms = Entry.GetComponentSpaceTransform(i) * RootTransform;

				if (i < PrevGlobalTransformsNum)
				{
					AnimInstanceProxy.AnimDrawDebugLine(PrevGlobalTransforms[i].GetTranslation(), GlobalTransforms.GetTranslation(), Color, false, 0.f, ESceneDepthPriorityGroup::SDPG_Foreground);
				}

				PrevGlobalTransforms[i] = GlobalTransforms;
			}
		}

		// no need to DebugDraw PoseHistory since it'll be drawn anyways by the history collectors
		//PoseHistory->DebugDraw(AnimInstanceProxy, Color);
	}
}
#endif // ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG

//////////////////////////////////////////////////////////////////////////
// FPoseIndicesHistory
void FPoseIndicesHistory::Update(const FSearchResult& SearchResult, float DeltaTime, float MaxTime)
{
	if (MaxTime > 0.f)
	{
		for (auto It = IndexToTime.CreateIterator(); It; ++It)
		{
			It.Value() += DeltaTime;
			if (It.Value() > MaxTime)
			{
				It.RemoveCurrent();
			}
		}

		if (SearchResult.IsValid())
		{
			FHistoricalPoseIndex HistoricalPoseIndex;
			HistoricalPoseIndex.PoseIndex = SearchResult.PoseIdx;
			HistoricalPoseIndex.DatabaseKey = FObjectKey(SearchResult.Database.Get());
			IndexToTime.Add(HistoricalPoseIndex, 0.f);
		}
	}
	else
	{
		IndexToTime.Reset();
	}
}


bool FPoseIndicesHistory::operator==(const FPoseIndicesHistory& Other) const
{
	return IndexToTime.OrderIndependentCompareEqual(Other.IndexToTime);
}

} // namespace UE::PoseSearch


//////////////////////////////////////////////////////////////////////////
// FPoseSearchHistory
void FPoseSearchHistory::Initialize(int32 InNumPoses, float InSamplingInterval)
{
	check(InNumPoses >= 2);

	MaxNumPoses = InNumPoses;
	SamplingInterval = InSamplingInterval;
	
	Trajectory = FTransformTrajectory();
	TrajectorySpeedMultiplier = 1.f;

	LastUpdateSkeleton = nullptr;
	BoneToTransformMap.Reset();
	CollectedCurves.Reset();
	BoneToTransformMapTypeHash = 0;
	Entries.Reset();
	PoseIndicesHistory.Reset();
}

bool FPoseSearchHistory::GetTransformAtTime(float Time, FTransform& OutBoneTransform, const USkeleton* BoneIndexSkeleton, FBoneIndexType BoneIndexType, FBoneIndexType ReferenceBoneIndexType, bool bExtrapolate) const
{
	using namespace UE::PoseSearch;

	static_assert(RootBoneIndexType == 0 && ComponentSpaceIndexType == FBoneIndexType(-1) && WorldSpaceIndexType == FBoneIndexType(-2)); // some assumptions
	
	bool bSuccess = true;
	if (BoneIndexType == ReferenceBoneIndexType)
	{
		OutBoneTransform = FTransform::Identity;
		bSuccess = true;
	}
	else if (ReferenceBoneIndexType == WorldSpaceIndexType)
	{
		if (BoneIndexType == ComponentSpaceIndexType)
		{
			OutBoneTransform = Trajectory.GetSampleAtTime(Time, bExtrapolate).GetTransform();
			bSuccess = true;
		}
		else
		{
			// getting BoneIndexType in ComponentSpaceIndexType and then multiplying it with the component to world transform from the trajectory
			bSuccess = GetTransformAtTime(Time, OutBoneTransform, BoneIndexSkeleton, BoneIndexType, ComponentSpaceIndexType, bExtrapolate);
			OutBoneTransform *= Trajectory.GetSampleAtTime(Time, bExtrapolate).GetTransform();
		}
	}
	else if (BoneIndexType == WorldSpaceIndexType || BoneIndexType == ComponentSpaceIndexType)
	{
		bSuccess = GetTransformAtTime(Time, OutBoneTransform, BoneIndexSkeleton, ReferenceBoneIndexType, BoneIndexType, bExtrapolate);
		OutBoneTransform = OutBoneTransform.Inverse();
	}
	else
	{
		check(BoneIndexType != ComponentSpaceIndexType && BoneIndexType != WorldSpaceIndexType && ReferenceBoneIndexType != WorldSpaceIndexType);

		const int32 NumEntries = Entries.Num();
		if (NumEntries > 0)
		{
			int32 NextIdx = 0;
			int32 PrevIdx = 0;

			if (NumEntries > 1)
			{
				const int32 LowerBoundIdx = LowerBound(Entries.begin(), Entries.end(), Time, [](const FPoseHistoryEntry& Entry, float Value) { return Value > Entry.AccumulatedSeconds; });
				NextIdx = FMath::Clamp(LowerBoundIdx, 1, NumEntries - 1);
				PrevIdx = NextIdx - 1;
			}

			const FPoseHistoryEntry& PrevEntry = Entries[PrevIdx];
			const FPoseHistoryEntry& NextEntry = Entries[NextIdx];

			bSuccess = LerpEntries(Time, bExtrapolate, PrevEntry, NextEntry, BoneIndexSkeleton, LastUpdateSkeleton.Get(), BoneToTransformMap, BoneIndexType, ReferenceBoneIndexType, OutBoneTransform);
		}
		else
		{
			OutBoneTransform = FTransform::Identity;
			bSuccess = false;
		}
	}

	// @todo: reenable this logging after implementing AnimNext PoseHistory initialization (currently spamming on actor spawning with MM active)
//#if WITH_EDITOR
//	// @todo: logging errors only if pose history was properly initialized for now (since AnimNext version doesn't implement initialization with ref pose yet),
//	// and we ultimately want to catch errors of missing bone transforms only checking valid USkeleton(s) against valid PoseSearchSchema(s)
//	if (!bSuccess && BoneIndexSkeleton)
//	{
//		UE_LOGF(LogPoseSearch, Error, "FPoseHistory::GetTransformAtTime - Couldn't find root bone transform at time %f for Skeleton %ls!", Time, *BoneIndexSkeleton->GetName());
//	}
//#endif // WITH_EDITOR

	return bSuccess;
}

bool FPoseSearchHistory::GetCurveValueAtTime(float Time, const FName& CurveName, float& OutCurveValue, bool bExtrapolate /*= false*/) const
{
	using namespace UE::PoseSearch;

	const int32 NumEntries = Entries.Num();
	if (NumEntries > 0)
	{
		int32 NextIdx = 0;
		int32 PrevIdx = 0;

		if (NumEntries > 1)
		{
			const int32 LowerBoundIdx = LowerBound(Entries.begin(), Entries.end(), Time, [](const FPoseHistoryEntry& Entry, float Value) { return Value > Entry.AccumulatedSeconds; });
			NextIdx = FMath::Clamp(LowerBoundIdx, 1, NumEntries - 1);
			PrevIdx = NextIdx - 1;
		}

		const FPoseHistoryEntry& PrevEntry = Entries[PrevIdx];
		const FPoseHistoryEntry& NextEntry = Entries[NextIdx];

		return LerpEntries(Time, bExtrapolate, PrevEntry, NextEntry, CurveName, CollectedCurves, OutCurveValue);
	}
	
	OutCurveValue = 0.0f;
	return false;
}

void FPoseSearchHistory::SetTrajectory(const FTransformTrajectory& InTrajectory, float InTrajectorySpeedMultiplier)
{
	Trajectory = InTrajectory;

	TrajectorySpeedMultiplier = InTrajectorySpeedMultiplier;

	if (!FMath::IsNearlyEqual(TrajectorySpeedMultiplier, 1.f))
	{
		const float TrajectorySpeedMultiplierInv = FMath::IsNearlyZero(TrajectorySpeedMultiplier) ? 1.f : 1.f / TrajectorySpeedMultiplier;
		for (FTransformTrajectorySample& Sample : Trajectory.Samples)
		{
			Sample.TimeInSeconds *= TrajectorySpeedMultiplierInv;
		}
	}
}

void FPoseSearchHistory::EvaluateComponentSpace(float DeltaTime, UE::PoseSearch::IComponentSpacePoseProvider& ComponentSpacePoseProvider, bool bStoreScales,
	float RootBoneRecoveryTime, float RootBoneTranslationRecoveryRatio, float RootBoneRotationRecoveryRatio,
	bool bNeedsReset, bool bCacheBones, const TArray<FBoneIndexType>& RequiredBones,
	const FBlendedCurve& Curves, const TConstArrayView<FName>& InCollectedCurves
#if WITH_EDITORONLY_DATA
	, const FTransform& InDebugMeshTransform
#endif //WITH_EDITORONLY_DATA
	)
{
	using namespace UE::PoseSearch;

	if (!ensure(MaxNumPoses >= 2))
	{
		UE_LOGF(LogPoseSearch, Error, "FPoseSearchHistory::EvaluateComponentSpace called before FPoseSearchHistory::Initialize! Please double check the implementation of any FAnimNode_Base calling this API, correctly propagating Initialize_AnyThread to their FPoseLink(s)! To mitigate issues FPoseHistory will be set to to default values");
		Initialize(2, 0.f);
	}

	const USkeleton* Skeleton = ComponentSpacePoseProvider.GetSkeletonAsset();

	if (bCacheBones)
	{
		const uint32 OldBoneToTransformMapTypeHash = BoneToTransformMapTypeHash;

		BoneToTransformMap.Reset();
		CollectedCurves = InCollectedCurves;
		if (!RequiredBones.IsEmpty())
		{
			// making sure we always collect the root bone transform (by construction BoneToTransformMap[0] = 0)
			const FComponentSpaceTransformIndex ComponentSpaceTransformRootBoneIndex = 0;
			BoneToTransformMap.Add(RootBoneIndexType) = ComponentSpaceTransformRootBoneIndex;

			for (int32 i = 0; i < RequiredBones.Num(); ++i)
			{
				// adding only unique RequiredBones to avoid oversizing Entries::ComponentSpaceTransforms
				if (!BoneToTransformMap.Find(RequiredBones[i]))
				{
					const FComponentSpaceTransformIndex ComponentSpaceTransformIndex = BoneToTransformMap.Num();
					BoneToTransformMap.Add(RequiredBones[i]) = ComponentSpaceTransformIndex;
				}
			}
		}

		BoneToTransformMapTypeHash = GetTypeHash(BoneToTransformMap);
		bNeedsReset |= (OldBoneToTransformMapTypeHash != BoneToTransformMapTypeHash);
	}

	if (LastUpdateSkeleton != Skeleton)
	{
		bNeedsReset = true;
		LastUpdateSkeleton = Skeleton;
	}

	if (bNeedsReset)
	{
		Entries.Reset();
		Entries.Reserve(MaxNumPoses);
	}

	FPoseHistoryEntry FutureEntryTemp;
	if (!Entries.IsEmpty() && Entries.Last().AccumulatedSeconds > 0.f)
	{
		// removing the "future" root bone Entry
		FutureEntryTemp = MoveTemp(Entries.Last());
		Entries.Pop();
	}

	// Age our elapsed times
	for (FPoseHistoryEntry& Entry : Entries)
	{
		Entry.AccumulatedSeconds -= DeltaTime;
	}

	if (Entries.Num() != MaxNumPoses)
	{
		// Consume every pose until the queue is full
		Entries.Emplace();
	}
	else if (SamplingInterval <= 0.f || Entries[Entries.Num() - 2].AccumulatedSeconds <= -SamplingInterval)
	{
		FPoseHistoryEntry EntryTemp = MoveTemp(Entries.First());
		Entries.PopFront();
		Entries.Emplace(MoveTemp(EntryTemp));
	}

	// Regardless of the retention policy, we always update the most recent Entry
	FPoseHistoryEntry& MostRecentEntry = Entries.Last();
	MostRecentEntry.Update(0.f, ComponentSpacePoseProvider, BoneToTransformMap, bStoreScales, Curves, CollectedCurves
#if WITH_EDITORONLY_DATA
		, InDebugMeshTransform
#endif //WITH_EDITORONLY_DATA
	);

	if (RootBoneRecoveryTime > 0.f && !Trajectory.Samples.IsEmpty())
	{
		// adding the updated "future" root bone Entry
		const FTransform& RefRootBone = Skeleton->GetReferenceSkeleton().GetRefBonePose()[RootBoneIndexType];
		const FQuat RootBoneRotationAtRecoveryTime = FMath::Lerp(FQuat(MostRecentEntry.ComponentSpaceRotations[RootBoneIndexType]), RefRootBone.GetRotation(), RootBoneRotationRecoveryRatio);

		FVector RootBoneDeltaTranslationAtRecoveryTime = FVector::ZeroVector;
		if (RootBoneTranslationRecoveryRatio > 0.f)
		{
			const FTransform WorldRootAtCurrentTime = Trajectory.GetSampleAtTime(0.f).GetTransform();
			const FTransform WorldRootBoneAtCurrentTime = MostRecentEntry.GetComponentSpaceTransform(RootBoneIndexType) * WorldRootAtCurrentTime;
			const FVector WorldRootBoneDeltaTranslationAtCurrentTime = (WorldRootBoneAtCurrentTime.GetTranslation() - WorldRootAtCurrentTime.GetTranslation()) * RootBoneTranslationRecoveryRatio;
			const FTransform WorldRootAtRecoveryTime = Trajectory.GetSampleAtTime(RootBoneRecoveryTime).GetTransform();
			RootBoneDeltaTranslationAtRecoveryTime = WorldRootAtRecoveryTime.InverseTransformVector(WorldRootBoneDeltaTranslationAtCurrentTime);
		}

		const FTransform RootBoneTransformAtRecoveryTime(RootBoneRotationAtRecoveryTime, RootBoneDeltaTranslationAtRecoveryTime, RefRootBone.GetScale3D());
		FutureEntryTemp.SetNum(1, bStoreScales);
		FutureEntryTemp.SetComponentSpaceTransform(RootBoneIndexType, RootBoneTransformAtRecoveryTime);
		FutureEntryTemp.AccumulatedSeconds = RootBoneRecoveryTime;
		Entries.Emplace(MoveTemp(FutureEntryTemp));
	}
}
