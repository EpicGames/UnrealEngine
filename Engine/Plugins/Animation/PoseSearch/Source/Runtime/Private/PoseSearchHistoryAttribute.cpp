// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchHistoryAttribute.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchHistoryAttribute)

namespace UE::PoseSearch
{
	const UE::Anim::FAttributeId PoseHistoryAttributeId(TEXT("PoseHistoryAttribute"), FCompactPoseBoneIndex(0));
}
