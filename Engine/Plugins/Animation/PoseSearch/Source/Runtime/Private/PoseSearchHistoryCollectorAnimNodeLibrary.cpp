// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchHistoryCollectorAnimNodeLibrary.h"
#include "PoseSearch/AnimNode_PoseSearchHistoryCollector.h"

#include "Animation/Skeleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchHistoryCollectorAnimNodeLibrary)

FPoseSearchHistoryCollectorAnimNodeReference UPoseSearchHistoryCollectorAnimNodeLibrary::ConvertToPoseHistoryNode(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result)
{
	return FAnimNodeReference::ConvertToType<FPoseSearchHistoryCollectorAnimNodeReference>(Node, Result);
}

void UPoseSearchHistoryCollectorAnimNodeLibrary::ConvertToPoseHistoryNodePure(const FAnimNodeReference& Node, FPoseSearchHistoryCollectorAnimNodeReference& PoseSearchHistoryCollectorNode, bool& Result)
{
	EAnimNodeReferenceConversionResult ConversionResult;
	PoseSearchHistoryCollectorNode = ConvertToPoseHistoryNode(Node, ConversionResult);
	Result = (ConversionResult == EAnimNodeReferenceConversionResult::Succeeded);
}


FPoseHistoryReference UPoseSearchHistoryCollectorAnimNodeLibrary::GetPoseHistoryReference(const FPoseSearchHistoryCollectorAnimNodeReference& Node)
{
	if (const FAnimNode_PoseSearchHistoryCollector* PoseSearchHistoryCollectorNodePtr = Node.GetAnimNodePtr<FAnimNode_PoseSearchHistoryCollector>())
	{
		return PoseSearchHistoryCollectorNodePtr->GetPoseHistoryReference();
	}
	return FPoseHistoryReference();
}


void UPoseSearchHistoryCollectorAnimNodeLibrary::GetPoseHistoryNodeTransformTrajectory(const FPoseSearchHistoryCollectorAnimNodeReference& PoseSearchHistoryCollectorNode, FTransformTrajectory& Trajectory)
{
	if (FAnimNode_PoseSearchHistoryCollector* PoseSearchHistoryCollectorNodePtr = PoseSearchHistoryCollectorNode.GetAnimNodePtr<FAnimNode_PoseSearchHistoryCollector>())
	{
		if (const UE::PoseSearch::FPoseHistory* PoseHistory = PoseSearchHistoryCollectorNodePtr->GetPoseHistoryPtr())
		{
			Trajectory = PoseHistory->GetTrajectory();
		}
	}
}

void UPoseSearchHistoryCollectorAnimNodeLibrary::SetPoseHistoryNodeTransformTrajectory(const FPoseSearchHistoryCollectorAnimNodeReference& PoseSearchHistoryCollectorNode, const FTransformTrajectory& Trajectory)
{
	if (FAnimNode_PoseSearchHistoryCollector* PoseSearchHistoryCollectorNodePtr = PoseSearchHistoryCollectorNode.GetAnimNodePtr<FAnimNode_PoseSearchHistoryCollector>())
	{
		if (UE::PoseSearch::FPoseHistory* PoseHistory = PoseSearchHistoryCollectorNodePtr->GetPoseHistoryPtr())
		{
			PoseHistory->SetTrajectory(Trajectory);
		}
	}
}

bool UPoseSearchHistoryCollectorAnimNodeLibrary::GetPoseHistoryBoneWorldTransform(FTransform& OutTransform, const FPoseSearchHistoryCollectorAnimNodeReference& Node, const USkeleton* Skeleton, const FName BoneName, const float Time)
{
	if (!Skeleton)
	{
		OutTransform = FTransform::Identity;
		return false;
	}

	if (FAnimNode_PoseSearchHistoryCollector* PoseSearchHistoryCollectorNodePtr = Node.GetAnimNodePtr<FAnimNode_PoseSearchHistoryCollector>())
	{
		if (UE::PoseSearch::FPoseHistory* PoseHistory = PoseSearchHistoryCollectorNodePtr->GetPoseHistoryPtr())
		{
			const int32 BoneIndex = Skeleton->GetReferenceSkeleton().FindBoneIndex(BoneName);
			if (BoneIndex != INDEX_NONE && PoseHistory->GetTransformAtTime(Time, OutTransform, Skeleton, BoneIndex, UE::PoseSearch::WorldSpaceIndexType))
			{
				return true;
			}
		}
	}

	OutTransform = FTransform::Identity;
	return false;
}

bool UPoseSearchHistoryCollectorAnimNodeLibrary::GetPoseHistoryBoneWorldLinearVelocity(FVector& OutVelocity, const FPoseSearchHistoryCollectorAnimNodeReference& Node, const USkeleton* Skeleton, const FName BoneName, const float Time, const float DeltaTime)
{
	if (!Skeleton)
	{
		OutVelocity = FVector::ZeroVector;
		return false;
	}

	if (FAnimNode_PoseSearchHistoryCollector* PoseSearchHistoryCollectorNodePtr = Node.GetAnimNodePtr<FAnimNode_PoseSearchHistoryCollector>())
	{
		if (UE::PoseSearch::FPoseHistory* PoseHistory = PoseSearchHistoryCollectorNodePtr->GetPoseHistoryPtr())
		{
			FTransform PrevTransform, CurrTransform;
			const int32 BoneIndex = Skeleton->GetReferenceSkeleton().FindBoneIndex(BoneName);
			if (DeltaTime > UE_SMALL_NUMBER && 
				BoneIndex != INDEX_NONE && 
				PoseHistory->GetTransformAtTime(Time - DeltaTime, PrevTransform, Skeleton, BoneIndex, UE::PoseSearch::WorldSpaceIndexType) &&
				PoseHistory->GetTransformAtTime(Time, CurrTransform, Skeleton, BoneIndex, UE::PoseSearch::WorldSpaceIndexType))
			{
				OutVelocity = (CurrTransform.GetLocation() - PrevTransform.GetLocation()) / DeltaTime;
				return true;
			}
		}
	}

	OutVelocity = FVector::ZeroVector;
	return false;
}
