// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchIndex.inl"

namespace UE::PoseSearch
{

void CompareFeatureVectors(TConstArrayView<float> A, TConstArrayView<float> B, TConstArrayView<float> WeightsSqrt, TArrayView<float> Result)
{
	check(A.Num() == B.Num() && A.Num() == WeightsSqrt.Num() && A.Num() == Result.Num());

	Eigen::Map<const Eigen::ArrayXf> VA(A.GetData(), A.Num());
	Eigen::Map<const Eigen::ArrayXf> VB(B.GetData(), B.Num());
	Eigen::Map<const Eigen::ArrayXf> VW(WeightsSqrt.GetData(), WeightsSqrt.Num());
	Eigen::Map<Eigen::ArrayXf> VR(Result.GetData(), Result.Num());

	VR = ((VA - VB) * VW).square();
}

float CompareFeatureVectors(TConstArrayView<float> A, TConstArrayView<float> B)
{
	check(A.Num() == B.Num() && A.Num());

	Eigen::Map<const Eigen::ArrayXf> VA(A.GetData(), A.Num());
	Eigen::Map<const Eigen::ArrayXf> VB(B.GetData(), B.Num());

	return (VA - VB).square().sum();
}

// pruning utils
struct FPosePair
{
	int32 PoseIdxA = 0;
	int32 PoseIdxB = 0;
};
struct FPosePairSimilarity : public FPosePair
{
	float Similarity = 0.f;
};

static bool CalculateSimilarities(TArray<FPosePairSimilarity>& PosePairSimilarities, float SimilarityThreshold, 
	int32 DataCardinality, int32 NumPoses, const TAlignedArray<float>& Values,
	TFunctionRef<TConstArrayView<float>(int32, int32)> GetValuesVector)
{
	PosePairSimilarities.Reserve(1024 * 64);

	check(Values.Num() == NumPoses * DataCardinality);
	FKDTree KDTree(NumPoses, DataCardinality, Values.GetData());

	TArray<FKDTree::FKNNMaxHeapResultSet::FResult, TInlineAllocator<256>> Results;
	Results.SetNumUninitialized(NumPoses);

	for (int32 PoseIdx = 0; PoseIdx < NumPoses; ++PoseIdx)
	{
		TConstArrayView<float> ValuesA = GetValuesVector(PoseIdx, DataCardinality);

		// searching for duplicates within a radius of SimilarityThreshold
		FKDTree::FRadiusMaxHeapResultSet ResultSet(Results, SimilarityThreshold);
		const int32 NumResults = KDTree.FindNeighbors(ResultSet, ValuesA);

		for (int32 ResultIndex = 0; ResultIndex < NumResults; ++ResultIndex)
		{
			if (PoseIdx != Results[ResultIndex].Index)
			{
				FPosePairSimilarity PosePair;
				PosePair.PoseIdxA = PoseIdx;
				PosePair.PoseIdxB = Results[ResultIndex].Index;
				PosePair.Similarity = Results[ResultIndex].Distance;
				PosePairSimilarities.Emplace(PosePair);
			}
		}
	}

	if (!PosePairSimilarities.IsEmpty())
	{
		PosePairSimilarities.Sort([](const FPosePairSimilarity& A, const FPosePairSimilarity& B)
		{
			return A.Similarity < B.Similarity;
		});
		return true;
	}
	return false;
}

static bool PruneValues(int32 DataCardinality, int32 NumPoses, const TArray<FPosePairSimilarity>& PosePairSimilarities, TAlignedArray<float>& Values,
	TFunctionRef<uint32(int32)> GetValueOffset, TFunctionRef<void(int32, uint32)> SetValueOffset)
{
	// mapping between the one value offset and all the poses sharing it
	TMap<uint32, TArray<int32>> ValueOffsetToPoses;
	for (int32 PoseIdx = 0; PoseIdx < NumPoses; ++PoseIdx)
	{
		const uint32 ValueOffset = GetValueOffset(PoseIdx);
		// FindOrAdd to support the eventuality of having multiple poses already sharing the same value offset
		ValueOffsetToPoses.FindOrAdd(ValueOffset).Add(PoseIdx);
	}

	// at this point ValueOffsetToPoses is fully populated with all the possible value offset, and since we're not adding, but eventually removing entries we can just use the [] operator
	uint32 ValueOffsetLast = Values.Num() - DataCardinality;
	for (int32 PosePairSimilarityIdx = 0; PosePairSimilarityIdx < PosePairSimilarities.Num(); ++PosePairSimilarityIdx)
	{
		const FPosePairSimilarity& PosePairSimilarity = PosePairSimilarities[PosePairSimilarityIdx];
		const uint32 ValueOffsetA = GetValueOffset(PosePairSimilarity.PoseIdxA);
		const uint32 ValueOffsetB = GetValueOffset(PosePairSimilarity.PoseIdxB);
				
		// if the two poses don't point already to the same value offset, we can remove one of them
		if (ValueOffsetA != ValueOffsetB)
		{
			// transferring all the poses associated to ValueOffsetB to ValueOffsetA
			TArray<int32>& PosesAtValueOffsetA = ValueOffsetToPoses[ValueOffsetA];
			TArray<int32>& PosesAtValueOffsetB = ValueOffsetToPoses[ValueOffsetB];

			for (int32 PoseAtValueOffsetB : PosesAtValueOffsetB)
			{
				SetValueOffset(PoseAtValueOffsetB, ValueOffsetA);
				PosesAtValueOffsetA.Add(PoseAtValueOffsetB);
			}

			// moving the ValueOffsetLast values into the location ValueOffsetB, that we just free up
			if (ValueOffsetB != ValueOffsetLast)
			{
				FMemory::Memcpy(&Values[ValueOffsetB], &Values[ValueOffsetLast], DataCardinality * sizeof(float));
				TArray<int32>& PosesAtValueOffsetLast = ValueOffsetToPoses[ValueOffsetLast];
						
				for (int32 PoseAtValueOffsetLast : PosesAtValueOffsetLast)
				{
					SetValueOffset(PoseAtValueOffsetLast, ValueOffsetB);
				}

				PosesAtValueOffsetB = PosesAtValueOffsetLast;
				PosesAtValueOffsetLast.Reset();
			}
			else
			{
				PosesAtValueOffsetB.Reset();
			}

			ValueOffsetLast -= DataCardinality;
		}
	}

	if (ValueOffsetLast + DataCardinality != Values.Num())
	{
		// resizing the Values array  
		Values.SetNum(ValueOffsetLast + DataCardinality);
		return true;
	}

	return false;
}

//////////////////////////////////////////////////////////////////////////
// FPoseMetadata
FArchive& operator<<(FArchive& Ar, FPoseMetadata& Metadata)
{
	// storing more data than necessary for now to avoid to deal with endiannes
	uint32 ValueOffset = Metadata.GetValueOffset();
	uint32 AssetIndex = Metadata.GetAssetIndex();
	bool bInBlockTransition = Metadata.IsBlockTransition();
	FFloat16 CostAddend = Metadata.CostAddend;
	
	// @todo: optimize the archived size of FPoseMetadata, since most members are bitfields
	Ar << ValueOffset;
	Ar << AssetIndex;
	Ar << bInBlockTransition;
	Ar << CostAddend;

	Metadata = FPoseMetadata(ValueOffset, AssetIndex, bInBlockTransition, CostAddend);
	return Ar;
}

//////////////////////////////////////////////////////////////////////////
// FSearchIndexAsset
FFloatInterval FSearchIndexAsset::GetExtrapolationTimeInterval(int32 SchemaSampleRate, const FFloatInterval& AdditionalExtrapolationTime) const
{
	return FFloatInterval(FirstSampleIdx / float(SchemaSampleRate) + AdditionalExtrapolationTime.Min,
		LastSampleIdx / float(SchemaSampleRate) + AdditionalExtrapolationTime.Max);
}

bool FSearchIndexAsset::operator==(const FSearchIndexAsset& Other) const
{
	return
		SourceAssetIdx == Other.SourceAssetIdx &&
		bMirrored == Other.bMirrored &&
		bLooping == Other.bLooping &&
		bDisableReselection == Other.bDisableReselection &&
		PermutationIdx == Other.PermutationIdx &&
		BlendParameterX == Other.BlendParameterX &&
		BlendParameterY == Other.BlendParameterY &&
		FirstPoseIdx == Other.FirstPoseIdx &&
		FirstSampleIdx == Other.FirstSampleIdx &&
		LastSampleIdx == Other.LastSampleIdx &&
		ToRealTimeFactor == Other.ToRealTimeFactor;
}

FArchive& operator<<(FArchive& Ar, FSearchIndexAsset& IndexAsset)
{
	int32 SourceAssetIdx = IndexAsset.SourceAssetIdx;
	bool bMirrored = IndexAsset.bMirrored; 
	bool bLooping = IndexAsset.bLooping;
	bool bDisableReselection = IndexAsset.bDisableReselection;
	int32 PermutationIdx = IndexAsset.PermutationIdx;
	float BlendParameterX = IndexAsset.BlendParameterX;
	float BlendParameterY = IndexAsset.BlendParameterY;
	int32 FirstPoseIdx = IndexAsset.FirstPoseIdx;
	int32 FirstSampleIdx = IndexAsset.FirstSampleIdx;
	int32 LastSampleIdx = IndexAsset.LastSampleIdx;
	float ToRealTimeFactor = IndexAsset.ToRealTimeFactor;

	// @todo: optimize the archived size of FSearchIndexAsset, since most members are bitfields
	Ar << SourceAssetIdx;
	Ar << bMirrored;
	Ar << bLooping;
	Ar << bDisableReselection;
	Ar << PermutationIdx;
	Ar << BlendParameterX;
	Ar << BlendParameterY;
	Ar << FirstPoseIdx;
	Ar << FirstSampleIdx;
	Ar << LastSampleIdx;
	Ar << ToRealTimeFactor;

	new(&IndexAsset) FSearchIndexAsset(SourceAssetIdx, bMirrored, bLooping, bDisableReselection,
		PermutationIdx, FVector(BlendParameterX, BlendParameterY, 0.f), FirstPoseIdx, FirstSampleIdx, LastSampleIdx, ToRealTimeFactor);

	return Ar;
}

#if WITH_EDITORONLY_DATA
//////////////////////////////////////////////////////////////////////////
// FSearchStats
void FSearchStats::Reset()
{
	AverageSpeed = 0.f;
	MaxSpeed = 0.f;
	AverageAcceleration = 0.f;
	MaxAcceleration = 0.f;
}

bool FSearchStats::operator==(const FSearchStats& Other) const
{
	return
		AverageSpeed == Other.AverageSpeed &&
		MaxSpeed == Other.MaxSpeed &&
		AverageAcceleration == Other.AverageAcceleration &&
		MaxAcceleration == Other.MaxAcceleration;
}

FArchive& operator<<(FArchive& Ar, FSearchStats& Stats)
{
	Ar << Stats.AverageSpeed;
	Ar << Stats.MaxSpeed;
	Ar << Stats.AverageAcceleration;
	Ar << Stats.MaxAcceleration;
	return Ar;
}
#endif // WITH_EDITORONLY_DATA

//////////////////////////////////////////////////////////////////////////
// FEventDataCollector
#if WITH_EDITOR
void FEventDataCollector::Emplace(const FGameplayTag& EventTag, int32 PoseIdx)
{
	check(PoseIdx >= 0);
	Data.FindOrAdd(EventTag).Add(PoseIdx);
}

void FEventDataCollector::MergeWith(const FEventDataCollector& Other)
{
	for (const FTagToPoseIndexes& OtherTagToPoseIndexes : Other.Data)
	{
		FPoseIndexes& PoseIndexes = Data.FindOrAdd(OtherTagToPoseIndexes.Key);

		// adding all the missing poses of OtherTagToPoseIndexes to PoseIndexes
		for (int32 OtherPoseIdx : OtherTagToPoseIndexes.Value)
		{
			check(OtherPoseIdx >= 0);
			PoseIndexes.Add(OtherPoseIdx);
		}
	}
}
#endif // WITH_EDITOR

//////////////////////////////////////////////////////////////////////////
// FEventData
FArchive& operator<<(FArchive& Ar, FEventData& EventData)
{
#if WITH_EDITOR
	if (Ar.IsSaving())
	{
		EventData.ValidateEventData();
	}
#endif // WITH_EDITOR

	Ar << EventData.Data;

#if WITH_EDITOR
	if (Ar.IsLoading())
	{
		EventData.ValidateEventData();
	}
#endif // WITH_EDITOR

	return Ar;
}

bool FEventData::operator==(const FEventData& Other) const
{
	return Data == Other.Data;
}

const TConstArrayView<int32> FEventData::GetPosesWithEvent(const FGameplayTag& GameplayTag) const
{
	// @todo: optmize me: Data is sorted! (or can be sorted differently to accomodate fast searches - maybe a serialized perfect hash map)
	for (const FEventData::FTagToPoseIndexes& TagToPoseIndexes : Data)
	{
		if (TagToPoseIndexes.Key == GameplayTag)
		{
			return TagToPoseIndexes.Value;
		}
	}

	return TConstArrayView<int32>();
}

bool FEventData::IsPoseFromEventTag(int32 EventPoseIdx, const FGameplayTag& GameplayTag) const
{
	const TConstArrayView<int32> PoseIndexes = GetPosesWithEvent(GameplayTag);
	// since PoseIndexes is sorted we can perform a binary search instead of PoseIndexes.Contains(EventPoseIdx);
	return Algo::BinarySearch(PoseIndexes, EventPoseIdx) != INDEX_NONE;
}

void FEventData::Reset()
{
	Data.Reset();
}

#if WITH_EDITOR
void FEventData::Initialize(const FEventDataCollector& EventDataCollector)
{
	Data.Reset();
	for (const FEventDataCollector::FTagToPoseIndexes& OtherTagToPoseIndexes : EventDataCollector.GetData())
	{
		FTagToPoseIndexes& TagToPoseIndexes = Data.AddDefaulted_GetRef();
		TagToPoseIndexes.Key = OtherTagToPoseIndexes.Key;

		TagToPoseIndexes.Value.Reserve(OtherTagToPoseIndexes.Value.Num());
		for (int32 PoseIdx : OtherTagToPoseIndexes.Value)
		{
			TagToPoseIndexes.Value.Add(PoseIdx);
		}
		TagToPoseIndexes.Value.Sort();
	}

	Data.Sort([](const FTagToPoseIndexes& A, const FTagToPoseIndexes& B)
		{
			// converting FGameplayTag::TagName to string to be deterministic across multiple editor restarts
			return A.Key.ToString() < B.Key.ToString();
		});
}

SIZE_T FEventData::GetAllocatedSize(void) const
{
	return Data.GetAllocatedSize();
}

void FEventData::ValidateEventData() const
{
	if (!Algo::IsSorted(Data, [](const FTagToPoseIndexes& A, const FTagToPoseIndexes& B)
		{
			// converting FGameplayTag::TagName to string to be deterministic across multiple editor restarts
			return A.Key.ToString() < B.Key.ToString();
		}))
	{
		UE_LOGF(LogPoseSearch, Error, "FEventData::ValidateEventData FGameplayTag are not properly sorted!");
	}

	for (const FEventData::FTagToPoseIndexes& TagToPoseIndexes : Data)
	{
		if (!Algo::IsSorted(TagToPoseIndexes.Value))
		{
			UE_LOGF(LogPoseSearch, Error, "FEventData::ValidateEventData FPoseIndexes are not properly sorted!");
			break;
		}
	}
}
#endif // WITH_EDITOR

//////////////////////////////////////////////////////////////////////////
// FPoseSearchBaseIndex
FSearchIndexBase::FSearchIndexBase()
	: Values()
	, ValuesVectorToPoseIndexes()
	, PoseMetadata()
	, bAnyBlockTransition(false)
	, Assets()
	, EventData()
	, MinCostAddend(-MAX_FLT)
#if WITH_EDITORONLY_DATA
	, Stats()
#endif // WITH_EDITORONLY_DATA
{
}

FSearchIndexBase::~FSearchIndexBase()
{
}

FSearchIndexBase::FSearchIndexBase(const FSearchIndexBase& Other)
	: Values(Other.Values)
	, ValuesVectorToPoseIndexes(Other.ValuesVectorToPoseIndexes)
	, PoseMetadata(Other.PoseMetadata)
	, bAnyBlockTransition(Other.bAnyBlockTransition)
	, Assets(Other.Assets)
	, EventData(Other.EventData)
	, MinCostAddend(Other.MinCostAddend)
#if WITH_EDITORONLY_DATA
	, Stats(Other.Stats)
#endif // WITH_EDITORONLY_DATA
{
}

FSearchIndexBase::FSearchIndexBase(FSearchIndexBase&& Other)
	: Values(MoveTemp(Other.Values))
	, ValuesVectorToPoseIndexes(MoveTemp(Other.ValuesVectorToPoseIndexes))
	, PoseMetadata(MoveTemp(Other.PoseMetadata))
	, bAnyBlockTransition(MoveTemp(Other.bAnyBlockTransition))
	, Assets(MoveTemp(Other.Assets))
	, EventData(MoveTemp(Other.EventData))
	, MinCostAddend(MoveTemp(Other.MinCostAddend))
#if WITH_EDITORONLY_DATA
	, Stats(MoveTemp(Other.Stats))
#endif // WITH_EDITORONLY_DATA
{
}

FSearchIndexBase& FSearchIndexBase::operator=(const FSearchIndexBase& Other)
{
	if (this != &Other)
	{
		this->~FSearchIndexBase();
		new(this) FSearchIndexBase(Other);
	}
	return *this;
}

FSearchIndexBase& FSearchIndexBase::operator=(FSearchIndexBase&& Other)
{
	if (this != &Other)
	{
		this->~FSearchIndexBase();
		new(this) FSearchIndexBase(MoveTemp(Other));
	}
	return *this;
}

const FSearchIndexAsset& FSearchIndexBase::GetAssetForPose(int32 PoseIdx) const
{
	const uint32 AssetIndex = PoseMetadata[PoseIdx].GetAssetIndex();
	check(Assets[AssetIndex].IsPoseInRange(PoseIdx));
	return Assets[AssetIndex];
}

const FSearchIndexAsset* FSearchIndexBase::GetAssetForPoseSafe(int32 PoseIdx) const
{
	if (PoseMetadata.IsValidIndex(PoseIdx))
	{
		const uint32 AssetIndex = PoseMetadata[PoseIdx].GetAssetIndex();
		if (Assets.IsValidIndex(AssetIndex))
		{
			return &Assets[AssetIndex];
		}
	}
	return nullptr;
}

bool FSearchIndexBase::IsEmpty() const
{
	return Assets.IsEmpty() || PoseMetadata.IsEmpty();
}

void FSearchIndexBase::Reset()
{
	Values.Reset();
	ValuesVectorToPoseIndexes = FSparsePoseMultiMap<int32>();
	PoseMetadata.Reset();
	bAnyBlockTransition = false;
	Assets.Reset();
	EventData.Reset();
	MinCostAddend = -MAX_FLT;
#if WITH_EDITORONLY_DATA
	Stats.Reset();
#endif // WITH_EDITORONLY_DATA
}

void FSearchIndexBase::PruneDuplicateValues(float SimilarityThreshold, int32 DataCardinality, bool bDoNotGenerateValuesVectorToPoseIndexes)
{
	ValuesVectorToPoseIndexes.Reset();

	const int32 NumPoses = GetNumPoses();
	if (SimilarityThreshold > 0.f && NumPoses >= 2)
	{
		TArray<FPosePairSimilarity> PosePairSimilarities;
		if (CalculateSimilarities(PosePairSimilarities, SimilarityThreshold, DataCardinality, NumPoses, Values,
			[this](int32 PoseIdx, int32 DataCardinality) { return GetPoseValuesBase(PoseIdx, DataCardinality); }))
		{
			PruneValues(DataCardinality, NumPoses, PosePairSimilarities, Values,
			[this](int32 PoseIdx) {	return PoseMetadata[PoseIdx].GetValueOffset(); },
			[this](int32 PoseIdx, uint32 ValueOffset) {	PoseMetadata[PoseIdx].SetValueOffset(ValueOffset); });
		}

		if (!bDoNotGenerateValuesVectorToPoseIndexes)
		{
			TMap<int32, TArray<int32>> ValuesVectorToPoseIndexesMap;
			ValuesVectorToPoseIndexesMap.Reserve(NumPoses);
			for (int32 PoseIdx = 0; PoseIdx < NumPoses; ++PoseIdx)
			{
				const FPoseMetadata& Metadata = PoseMetadata[PoseIdx];
				check(Metadata.GetValueOffset() % DataCardinality == 0);
				const int32 ValuesVectorIdx = Metadata.GetValueOffset() / DataCardinality;

				TArray<int32>& PoseIndexes = ValuesVectorToPoseIndexesMap.FindOrAdd(ValuesVectorIdx);
				PoseIndexes.Add(PoseIdx);
			}

			// sorting ValuesVectorToPoseIndexesMap keys to create a deterministic FSparsePoseMultiMap later on
			// we're not using TSortedMap for performance reasons, because ValuesVectorToPoseIndexesMap can be quite big
			TArray<int32> SortedKeys;
			SortedKeys.Reserve(ValuesVectorToPoseIndexesMap.Num());
			for (const TPair<int32, TArray<int32>>& Pair : ValuesVectorToPoseIndexesMap)
			{
				SortedKeys.Add(Pair.Key);
			}
			SortedKeys.Sort();

			FSparsePoseMultiMap<int32> SparsePoseMultiMap(ValuesVectorToPoseIndexesMap.Num(), NumPoses - 1);
			for (const int32& Key : SortedKeys)
			{
				const int32 PCAValuesVectorIdx = Key;
				const TArray<int32>& PoseIndexes = ValuesVectorToPoseIndexesMap[Key];
				SparsePoseMultiMap.Insert(PCAValuesVectorIdx, PoseIndexes);
			}

			for (int32 ValuesVectorIdx = 0; ValuesVectorIdx < SparsePoseMultiMap.Num(); ++ValuesVectorIdx)
			{
				const TConstArrayView<int32> PoseIndexes = SparsePoseMultiMap[ValuesVectorIdx];
				const TArray<int32>& TestPoseIndexes = ValuesVectorToPoseIndexesMap[ValuesVectorIdx];
				check(PoseIndexes == TestPoseIndexes);
			}

			ValuesVectorToPoseIndexes = SparsePoseMultiMap;
		}
	}
}

void FSearchIndexBase::AllocateData(int32 DataCardinality, int32 NumPoses)
{
	Values.Reset();
	PoseMetadata.Reset();

	Values.SetNumZeroed(DataCardinality * NumPoses);
	PoseMetadata.SetNumZeroed(NumPoses);

	EventData.Reset();
}

bool FSearchIndexBase::operator==(const FSearchIndexBase& Other) const
{
	return Values == Other.Values
		&& ValuesVectorToPoseIndexes == Other.ValuesVectorToPoseIndexes
		&& PoseMetadata == Other.PoseMetadata
		&& bAnyBlockTransition == Other.bAnyBlockTransition
		&& Assets == Other.Assets
		&& EventData == Other.EventData
		&& MinCostAddend == Other.MinCostAddend
#if WITH_EDITORONLY_DATA
		&& Stats == Other.Stats
#endif // WITH_EDITORONLY_DATA
		;
}

FArchive& operator<<(FArchive& Ar, FSearchIndexBase& Index)
{
	Ar << Index.Values;
	Ar << Index.ValuesVectorToPoseIndexes;
	Ar << Index.PoseMetadata;
	Ar << Index.bAnyBlockTransition;
	Ar << Index.Assets;
	Ar << Index.EventData;
	Ar << Index.MinCostAddend;
	
#if WITH_EDITORONLY_DATA
	if (!Ar.IsFilterEditorOnly())
	{
		Ar << Index.Stats;
	}
#endif // WITH_EDITORONLY_DATA

	return Ar;
}

//////////////////////////////////////////////////////////////////////////
// FSearchIndex
FSearchIndex::FSearchIndex(const FSearchIndex& Other)
	: FSearchIndexBase(Other)
	, WeightsSqrt(Other.WeightsSqrt)
	, PCAValues(Other.PCAValues)
	, PCAValuesVectorToPoseIndexes(Other.PCAValuesVectorToPoseIndexes)
	, PCAProjectionMatrix(Other.PCAProjectionMatrix)
	, Mean(Other.Mean)
	, KDTree(Other.KDTree)
	, VPTree(Other.VPTree)
	#if WITH_EDITORONLY_DATA
	, DeviationEditorOnly(Other.DeviationEditorOnly)
	, PCAExplainedVarianceEditorOnly(Other.PCAExplainedVarianceEditorOnly)
	#endif
{
	check(!PCAValues.IsEmpty() || KDTree.DataSource.PointCount == 0);
	KDTree.DataSource.Data = PCAValues.IsEmpty() ? nullptr : PCAValues.GetData();
}

FSearchIndex::FSearchIndex(FSearchIndex&& Other)
	: FSearchIndexBase(MoveTemp(Other))
	, WeightsSqrt(MoveTemp(Other.WeightsSqrt))
	, PCAValues(MoveTemp(Other.PCAValues))
	, PCAValuesVectorToPoseIndexes(MoveTemp(Other.PCAValuesVectorToPoseIndexes))
	, PCAProjectionMatrix(MoveTemp(Other.PCAProjectionMatrix))
	, Mean(MoveTemp(Other.Mean))
	, KDTree(MoveTemp(Other.KDTree))
	, VPTree(MoveTemp(Other.VPTree))
	#if WITH_EDITORONLY_DATA
	, DeviationEditorOnly(MoveTemp(Other.DeviationEditorOnly))
	, PCAExplainedVarianceEditorOnly(MoveTemp(Other.PCAExplainedVarianceEditorOnly))
	#endif
{
	check(!PCAValues.IsEmpty() || KDTree.DataSource.PointCount == 0);
	KDTree.DataSource.Data = PCAValues.IsEmpty() ? nullptr : PCAValues.GetData();
}

FSearchIndex& FSearchIndex::operator=(const FSearchIndex& Other)
{
	if (this != &Other)
	{
		this->~FSearchIndex();
		new(this) FSearchIndex(Other);
	}
	return *this;
}

FSearchIndex& FSearchIndex::operator=(FSearchIndex&& Other)
{
	if (this != &Other)
	{
		this->~FSearchIndex();
		new(this) FSearchIndex(MoveTemp(Other));
	}
	return *this;
}

void FSearchIndex::Reset()
{
	FSearchIndex Default;
	*this = Default;
}

TConstArrayView<float> FSearchIndex::GetPoseValues(int32 PoseIdx) const
{
	return GetPoseValuesBase(PoseIdx, GetNumDimensions());
}

TConstArrayView<float> FSearchIndex::GetReconstructedPoseValues(int32 PoseIdx, TArrayView<float> BufferUsedForReconstruction) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PoseSearch_PCAReconstruct);

	const bool bArePCAValuesPruned = PCAValuesVectorToPoseIndexes.Num() > 0;

	const int32 NumDimensions = GetNumDimensions();
	const int32 NumPoses = GetNumPoses();
	const int32 NumberOfPrincipalComponents = GetNumberOfPrincipalComponents();

	check(PoseIdx >= 0 && PoseIdx < NumPoses && NumDimensions > 0);
	check(BufferUsedForReconstruction.Num() == NumDimensions);

	const ColMajorMatrixMapConst MapPCAProjectionMatrix(PCAProjectionMatrix.GetData(), NumDimensions, NumberOfPrincipalComponents);
	const RowMajorVectorMapConst MapMean(Mean.GetData(), 1, NumDimensions);

	check(PCAValues.Num() % NumberOfPrincipalComponents == 0);
	const int32 NumPCAValuesVectors = PCAValues.Num() / NumberOfPrincipalComponents;
	const RowMajorMatrixMapConst MapPCAValues(PCAValues.GetData(), NumPCAValuesVectors, NumberOfPrincipalComponents);

	const int32 PCAValueIdx = GetPCAValueIndex(PoseIdx);
	if (PCAValueIdx != INDEX_NONE)
	{
		check(PCAValueIdx >= 0 && PCAValueIdx < NumPCAValuesVectors);
		const RowMajorVector WeightedReconstructedValues = MapPCAValues.row(PCAValueIdx) * MapPCAProjectionMatrix.transpose() + MapMean;

		const RowMajorVectorMapConst MapWeightsSqrt(WeightsSqrt.GetData(), 1, NumDimensions);
		const RowMajorVector ReciprocalWeightsSqrt = MapWeightsSqrt.cwiseInverse();

		RowMajorVectorMap ReconstructedPoseValues(BufferUsedForReconstruction.GetData(), 1, NumDimensions);
		ReconstructedPoseValues = WeightedReconstructedValues.array() * ReciprocalWeightsSqrt.array();

		return BufferUsedForReconstruction;
	}

	return TConstArrayView<float>();
}

int32 FSearchIndex::GetNumDimensions() const
{
	return WeightsSqrt.Num();
}

int32 FSearchIndex::GetNumberOfPrincipalComponents() const
{
	const int32 NumDimensions = GetNumDimensions();
	check(NumDimensions > 0 && PCAProjectionMatrix.Num() > 0 && PCAProjectionMatrix.Num() % NumDimensions == 0);

	const int32 NumberOfPrincipalComponents = PCAProjectionMatrix.Num() / NumDimensions;
	return NumberOfPrincipalComponents;
}

TConstArrayView<float> FSearchIndex::PCAProject(TConstArrayView<float> PoseValues, TArrayView<float> BufferUsedForProjection) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PoseSearch_PCAProject);

	const int32 NumDimensions = GetNumDimensions();
	const int32 NumberOfPrincipalComponents = GetNumberOfPrincipalComponents();
	check(PoseValues.Num() == NumDimensions);
	check(BufferUsedForProjection.Num() == NumberOfPrincipalComponents);

	const RowMajorVectorMapConst WeightsSqrtMap(WeightsSqrt.GetData(), 1, NumDimensions);
	const RowMajorVectorMapConst MeanMap(Mean.GetData(), 1, NumDimensions);
	const ColMajorMatrixMapConst PCAProjectionMatrixMap(PCAProjectionMatrix.GetData(), NumDimensions, NumberOfPrincipalComponents);
	const RowMajorVectorMapConst PoseValuesMap(PoseValues.GetData(), 1, NumDimensions);

	RowMajorVectorMap WeightedPoseValuesMap((float*)FMemory_Alloca(NumDimensions * sizeof(float)), 1, NumDimensions);
	WeightedPoseValuesMap = PoseValuesMap.array() * WeightsSqrtMap.array();

	RowMajorVectorMap CenteredPoseValuesMap((float*)FMemory_Alloca(NumDimensions * sizeof(float)), 1, NumDimensions);
	CenteredPoseValuesMap.noalias() = WeightedPoseValuesMap - MeanMap;

	RowMajorVectorMap ProjectedPoseValuesMap(BufferUsedForProjection.GetData(), 1, NumberOfPrincipalComponents);
	ProjectedPoseValuesMap.noalias() = CenteredPoseValuesMap * PCAProjectionMatrixMap;

	return BufferUsedForProjection;
}

void FSearchIndex::PrunePCAValuesFromBlockTransitionPoses(int32 NumberOfPrincipalComponents)
{
	if (!bAnyBlockTransition)
	{
		return;
	}

	check(PCAValues.Num() % NumberOfPrincipalComponents == 0);
	const int32 NumPCAValuesVectors = PCAValues.Num() / NumberOfPrincipalComponents;

	TArray<TPair<int32, TArray<int32>>> PrunedPCAValuesVectorToPoseIndexes;
	TAlignedArray<float> PrunedPCAValues;
	PrunedPCAValues.Reserve(PCAValues.Num());

	if (PCAValuesVectorToPoseIndexes.Num() > 0)
	{
		TArray<int32> PCAValuesVectorIdxPoseIndexes;
		for (int32 PCAValuesVectorIdx = 0; PCAValuesVectorIdx < NumPCAValuesVectors; ++PCAValuesVectorIdx)
		{
			PCAValuesVectorIdxPoseIndexes.Reset();
			for (int32 PoseIdx : PCAValuesVectorToPoseIndexes[PCAValuesVectorIdx])
			{
				if (!PoseMetadata[PoseIdx].IsBlockTransition())
				{
					PCAValuesVectorIdxPoseIndexes.Add(PoseIdx);
				}
			}

			if (!PCAValuesVectorIdxPoseIndexes.IsEmpty())
			{
				PrunedPCAValuesVectorToPoseIndexes.Emplace(PrunedPCAValuesVectorToPoseIndexes.Num(), PCAValuesVectorIdxPoseIndexes);
				PrunedPCAValues.Append(GetPCAPoseValues(PCAValuesVectorIdx));
			}
		}
	}
	else
	{
		for (int32 PCAValuesVectorIdx = 0; PCAValuesVectorIdx < NumPCAValuesVectors; ++PCAValuesVectorIdx)
		{
			// here there's a 1:1 mapping between PCAValuesVectorIdx and PoseIdx
			
			const int32 PoseIdx = PCAValuesVectorIdx;
			if (!PoseMetadata[PoseIdx].IsBlockTransition())
			{
				TConstArrayView<int32> PCAValuesVectorIdxPoseIndexes = MakeArrayView(&PoseIdx, 1);
				PrunedPCAValuesVectorToPoseIndexes.Emplace(PrunedPCAValuesVectorToPoseIndexes.Num(), PCAValuesVectorIdxPoseIndexes);
				PrunedPCAValues.Append(GetPCAPoseValues(PCAValuesVectorIdx));
			}
		}
	}

	PCAValues = PrunedPCAValues;
	PCAValuesVectorToPoseIndexes.Init(PrunedPCAValuesVectorToPoseIndexes, GetNumPoses() - 1);
}

void FSearchIndex::PruneDuplicatePCAValues(float SimilarityThreshold, int32 NumberOfPrincipalComponents)
{
	PCAValuesVectorToPoseIndexes.Reset();

	const uint32 NumPoses = GetNumPoses();
	if (SimilarityThreshold > 0.f && NumPoses >= 2 && NumberOfPrincipalComponents > 0 && !PCAValues.IsEmpty())
	{
		check(PCAValues.Num() % NumberOfPrincipalComponents == 0);
		const int32 NumPCAValuesVectors = PCAValues.Num() / NumberOfPrincipalComponents;
		// so far we support only pruning an original PCAValues set, where there's a 1:1 mapping between PCAValuesVectors and Poses
		check(NumPCAValuesVectors == NumPoses);

		TArray<int32> PoseToPCAValueOffset;
		PoseToPCAValueOffset.AddUninitialized(NumPoses);
		for (uint32 PoseIdx = 0; PoseIdx < NumPoses; ++PoseIdx)
		{
			PoseToPCAValueOffset[PoseIdx] = PoseIdx * NumberOfPrincipalComponents;
		}

		TArray<FPosePairSimilarity> PosePairSimilarities;
		if (CalculateSimilarities(PosePairSimilarities, SimilarityThreshold, NumberOfPrincipalComponents, NumPoses, PCAValues,
			[this, &PoseToPCAValueOffset](int32 PoseIdx, int32 NumberOfPrincipalComponents) { return MakeArrayView(&PCAValues[PoseToPCAValueOffset[PoseIdx]], NumberOfPrincipalComponents); }))
		{
			if (PruneValues(NumberOfPrincipalComponents, NumPoses, PosePairSimilarities, PCAValues,
				[&PoseToPCAValueOffset](int32 PoseIdx) { return PoseToPCAValueOffset[PoseIdx]; },
				[&PoseToPCAValueOffset](int32 PoseIdx, int32 ValueOffset) { PoseToPCAValueOffset[PoseIdx] = ValueOffset; }))
			{
				// we pruned some PCAValues: we need to construct a mapping between PCAValuesVectorIdx to PoseIdx(s)
				TMap<int32, TArray<int32>> PCAValuesVectorToPoseIndexesMap;
				PCAValuesVectorToPoseIndexesMap.Reserve(NumPoses);
				for (uint32 PoseIdx = 0; PoseIdx < NumPoses; ++PoseIdx)
				{
					check(PoseToPCAValueOffset[PoseIdx] % NumberOfPrincipalComponents == 0);
					const int32 PCAValuesVectorIdx = PoseToPCAValueOffset[PoseIdx] / NumberOfPrincipalComponents;
					TArray<int32>& PoseIndexes = PCAValuesVectorToPoseIndexesMap.FindOrAdd(PCAValuesVectorIdx);
					check(!PoseIndexes.Contains(PoseIdx));
					PoseIndexes.Add(PoseIdx);
				}

				// sorting PCAValuesVectorToPoseIndexesMap keys to create a deterministic FSparsePoseMultiMap later on
				// we're not using TSortedMap for performance reasons, because PCAValuesVectorToPoseIndexesMap can be quite big
				TArray<int32> SortedKeys;
				SortedKeys.Reserve(PCAValuesVectorToPoseIndexesMap.Num());
				for (const TPair<int32, TArray<int32>>& Pair : PCAValuesVectorToPoseIndexesMap)
				{
					SortedKeys.Add(Pair.Key);
				}
				SortedKeys.Sort();

				FSparsePoseMultiMap<int32> SparsePoseMultiMap(PCAValuesVectorToPoseIndexesMap.Num(), NumPoses - 1);
				for (const int32& Key : SortedKeys)
				{
					const int32 PCAValuesVectorIdx = Key;
					const TArray<int32>& PoseIndexes = PCAValuesVectorToPoseIndexesMap[Key];
					SparsePoseMultiMap.Insert(PCAValuesVectorIdx, PoseIndexes);
				}

				for (int32 PCAValuesVectorIdx = 0; PCAValuesVectorIdx < SparsePoseMultiMap.Num(); ++PCAValuesVectorIdx)
				{
					const TConstArrayView<int32> PoseIndexes = SparsePoseMultiMap[PCAValuesVectorIdx];
					const TArray<int32>& TestPoseIndexes = PCAValuesVectorToPoseIndexesMap[PCAValuesVectorIdx];
					check(PoseIndexes == TestPoseIndexes);
				}

				PCAValuesVectorToPoseIndexes = SparsePoseMultiMap;
			}
		}
	}
}

TConstArrayView<float> FSearchIndex::GetPoseValuesSafe(int32 PoseIdx, TArray<float>& BufferUsedForReconstruction) const
{
	if (PoseIdx < 0 || PoseIdx >= GetNumPoses())
	{
		return TConstArrayView<float>();
	}

	if (IsValuesEmpty())
	{
		const int32 NumDimensions = GetNumDimensions();
		BufferUsedForReconstruction.SetNum(NumDimensions, EAllowShrinking::No);
		return GetReconstructedPoseValues(PoseIdx, BufferUsedForReconstruction);
	}
		
	return GetPoseValues(PoseIdx);
}

TConstArrayView<float> FSearchIndex::GetPCAPoseValues(int32 PCAValuesVectorIdx) const
{
	if (PCAValues.IsEmpty())
	{
		return TConstArrayView<float>();
	}

	const int32 NumDimensions = GetNumDimensions();
	const int32 NumberOfPrincipalComponents = GetNumberOfPrincipalComponents();

#if DO_CHECK
	check(PCAValues.Num() % NumberOfPrincipalComponents == 0);
	const int32 NumPCAValuesVectors = PCAValues.Num() / NumberOfPrincipalComponents;
	check(PCAValuesVectorIdx >= 0 && PCAValuesVectorIdx < NumPCAValuesVectors );
#endif // DO_CHECK

	const int32 ValueOffset = PCAValuesVectorIdx * NumberOfPrincipalComponents;
	return MakeArrayView(&PCAValues[ValueOffset], NumberOfPrincipalComponents);
}

void FSearchIndex::GetPoseToPCAValuesVectorIndexes(TArray<uint32>& PoseToPCAValuesVectorIndexes) const
{
	if (PCAValuesVectorToPoseIndexes.Num() > 0)
	{
		PoseToPCAValuesVectorIndexes.Init(INDEX_NONE, PCAValuesVectorToPoseIndexes.MaxValue + 1);
		for (int32 PCAValuesVectorIdx = 0; PCAValuesVectorIdx < PCAValuesVectorToPoseIndexes.Num(); ++PCAValuesVectorIdx)
		{
			for (int32 PoseIdx : PCAValuesVectorToPoseIndexes[PCAValuesVectorIdx])
			{
				PoseToPCAValuesVectorIndexes[PoseIdx] = PCAValuesVectorIdx;
			}
		}
	}
	else
	{
		PoseToPCAValuesVectorIndexes.Reset();
	}
}

int32 FSearchIndex::GetPCAValueIndex(int32 PoseIdx) const
{
	if (PCAValuesVectorToPoseIndexes.Num() > 0)
	{
		for (int32 PCAValuesVectorIdx = 0; PCAValuesVectorIdx < PCAValuesVectorToPoseIndexes.Num(); ++PCAValuesVectorIdx)
		{
			for (int32 PoseIdxForPCAValuesVectorIdx : PCAValuesVectorToPoseIndexes[PCAValuesVectorIdx])
			{
				if (PoseIdxForPCAValuesVectorIdx == PoseIdx)
				{
					return PCAValuesVectorIdx;
				}
			}
		}

		return INDEX_NONE;
	}
	
	return PoseIdx;
}

bool FSearchIndex::operator==(const FSearchIndex& Other) const
{
	return FSearchIndexBase::operator==(Other)
		&& WeightsSqrt == Other.WeightsSqrt
		&& PCAValues == Other.PCAValues
		&& PCAValuesVectorToPoseIndexes == Other.PCAValuesVectorToPoseIndexes
		&& PCAProjectionMatrix == Other.PCAProjectionMatrix
		&& Mean == Other.Mean
		&& KDTree == Other.KDTree
		&& VPTree == Other.VPTree
#if WITH_EDITORONLY_DATA
		&& DeviationEditorOnly == Other.DeviationEditorOnly
		&& PCAExplainedVarianceEditorOnly == Other.PCAExplainedVarianceEditorOnly
#endif
		;
}

FArchive& operator<<(FArchive& Ar, FSearchIndex& Index)
{
	Ar << static_cast<FSearchIndexBase&>(Index);

	Ar << Index.WeightsSqrt;
	Ar << Index.PCAValues;
	Ar << Index.PCAValuesVectorToPoseIndexes;
	Ar << Index.PCAProjectionMatrix;
	Ar << Index.Mean;
	Ar << Index.VPTree;

	Serialize(Ar, Index.KDTree, Index.PCAValues.GetData());

#if WITH_EDITORONLY_DATA
	if (!Ar.IsFilterEditorOnly())
	{
		Ar << Index.DeviationEditorOnly;
		Ar << Index.PCAExplainedVarianceEditorOnly;
	}
#endif

	return Ar;
}

    
float FVPTreeDataSource::GetDistance(const TConstArrayView<float> A, const TConstArrayView<float> B)
{
	// Estracting the Sqrt to satisfy the triangle inequality metric space requirements, since a <= b+c doesn't imply a^2 <= b^2 + c^2
	return FMath::Sqrt(CompareFeatureVectors(A, B));
}

} // namespace UE::PoseSearch
