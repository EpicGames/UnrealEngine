// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchIndex.h"
#include "PoseSearchEigenHelper.h"

namespace UE::PoseSearch
{

template <bool bAlignedAndPadded>
static FORCEINLINE float CompareFeatureVectors(TConstArrayView<float> A, TConstArrayView<float> B, TConstArrayView<float> WeightsSqrt)
{
	if (bAlignedAndPadded)
	{
		check(A.Num() == B.Num() && A.Num() == WeightsSqrt.Num());
		check(A.Num() % 4 == 0);
		check(IsAligned(A.GetData(), alignof(VectorRegister4Float)));
		check(IsAligned(B.GetData(), alignof(VectorRegister4Float)));
		check(IsAligned(WeightsSqrt.GetData(), alignof(VectorRegister4Float)));
		// sufficient condition to check for pointer overlapping
		check(A.GetData() != B.GetData() && A.GetData() != WeightsSqrt.GetData());

		const int32 NumVectors = A.Num() / 4;

		const VectorRegister4Float* RESTRICT VA = reinterpret_cast<const VectorRegister4Float*>(A.GetData());
		const VectorRegister4Float* RESTRICT VB = reinterpret_cast<const VectorRegister4Float*>(B.GetData());
		const VectorRegister4Float* RESTRICT VW = reinterpret_cast<const VectorRegister4Float*>(WeightsSqrt.GetData());

		VectorRegister4Float PartialCost = VectorZero();
		for (int32 VectorIdx = 0; VectorIdx < NumVectors; ++VectorIdx, ++VA, ++VB, ++VW)
		{
			const VectorRegister4Float Diff = VectorSubtract(*VA, *VB);
			const VectorRegister4Float WeightedDiff = VectorMultiply(Diff, *VW);
			PartialCost = VectorMultiplyAdd(WeightedDiff, WeightedDiff, PartialCost);
		}

		// calculating PartialCost.X + PartialCost.Y + PartialCost.Z + PartialCost.W
		VectorRegister4Float Swizzle = VectorSwizzle(PartialCost, 1, 0, 3, 2);	// (Y, X, W, Z) of PartialCost
		PartialCost = VectorAdd(PartialCost, Swizzle);							// (X + Y, Y + X, Z + W, W + Z)
		Swizzle = VectorSwizzle(PartialCost, 2, 3, 0, 1);						// (Z + W, W + Z, X + Y, Y + X)
		PartialCost = VectorAdd(PartialCost, Swizzle);							// (X + Y + Z + W, Y + X + W + Z, Z + W + X + Y, W + Z + Y + X)
		float Cost;
		VectorStoreFloat1(PartialCost, &Cost);

		// keeping this debug code to validate CompareAlignedFeatureVectors against CompareFeatureVectors
		//#if DO_CHECK
		//	const float EigenCost = CompareFeatureVectors(A, B, WeightsSqrt);
		//
		//	if (!FMath::IsNearlyEqual(Cost, EigenCost))
		//	{
		//		const float RelativeDifference = Cost > EigenCost ? (Cost - EigenCost) / Cost : (EigenCost - Cost) / EigenCost;
		//		check(FMath::IsNearlyZero(RelativeDifference, UE_KINDA_SMALL_NUMBER));
		//	}
		//#endif //DO_CHECK

		return Cost;
	}
	else
	{
		check(A.Num() == B.Num() && A.Num() == WeightsSqrt.Num());

		Eigen::Map<const Eigen::ArrayXf> VA(A.GetData(), A.Num());
		Eigen::Map<const Eigen::ArrayXf> VB(B.GetData(), B.Num());
		Eigen::Map<const Eigen::ArrayXf> VW(WeightsSqrt.GetData(), WeightsSqrt.Num());

		return ((VA - VB) * VW).square().sum();
	}
}

} // namespace UE::PoseSearch
