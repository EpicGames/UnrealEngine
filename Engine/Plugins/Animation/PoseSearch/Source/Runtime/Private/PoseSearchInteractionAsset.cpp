// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchInteractionAsset.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "PoseSearch/PoseSearchAssetSampler.h"
#include "PoseSearch/PoseSearchMirrorDataCache.h"

#if ENABLE_ANIM_DEBUG && WITH_EDITOR
#include "DrawDebugHelpers.h"
#endif // ENABLE_ANIM_DEBUG && WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchInteractionAsset)

namespace UE::PoseSearch
{

static FVector FindReferencePosition(const TArrayView<const FTransform> Transforms, const TArrayView<float> NormalizedWarpingWeightTranslation)
{
	const int32 Num = Transforms.Num();
	check(Num == NormalizedWarpingWeightTranslation.Num());

	FVector PositionsSum = FVector::ZeroVector;
	for (int32 ItemIndex = 0; ItemIndex < Num; ++ItemIndex)
	{
		PositionsSum += Transforms[ItemIndex].GetTranslation() * NormalizedWarpingWeightTranslation[ItemIndex];
	}

	return PositionsSum;
}

// creates a reference frame orientation given an array of Transforms and an array of those transforms sorted by weight.
static FQuat FindReferenceOrientation(const TArrayView<const FTransform> Transforms, const TArrayView<int32> SortedByWarpingWeightRotationItemIndex, const TArrayView<float> NormalizedWarpingWeightRotation)
{
	const int32 Num = Transforms.Num();

	check(Num > 0 && Num == SortedByWarpingWeightRotationItemIndex.Num() && Num == NormalizedWarpingWeightRotation.Num());

	// as convention we perform a weighted sum of all the most weighted transforms translations excluding the last one.
	// the idea is to create an "heavy" point we can aim at to create the reference system orientation, supposing that this 
	// would be a "more stable" one, since in the scene you'd align your character towards something that is allow to move the least. 
	FVector WeightedItemPositionSum = FVector::ZeroVector;
	float CurrentWeightSum = 0.f;

	for (int32 ItemIndex = 0; ItemIndex < Num-1; ++ItemIndex)
	{
		const float CurrentWeight = NormalizedWarpingWeightRotation[SortedByWarpingWeightRotationItemIndex[ItemIndex]];
		const FVector& ItemPosition = Transforms[SortedByWarpingWeightRotationItemIndex[ItemIndex]].GetTranslation();

		WeightedItemPositionSum += ItemPosition * CurrentWeight;
		CurrentWeightSum += CurrentWeight;
	}

	if (ensure(CurrentWeightSum > UE_SMALL_NUMBER))
	{
		const FVector HeavyItemsPositionAverage = WeightedItemPositionSum / CurrentWeightSum;
		const FVector& LightItemPosition = Transforms[SortedByWarpingWeightRotationItemIndex[Num - 1]].GetTranslation();
		FVector DeltaPosition = HeavyItemsPositionAverage - LightItemPosition;

		if (!DeltaPosition.IsNearlyZero())
		{
			return DeltaPosition.ToOrientationQuat();
		}
	}

	return FQuat::Identity;
}

void Normalize(TArrayView<float> Array, int32 IndexToExclude = INDEX_NONE)
{
	const int32 Num = Array.Num();
	check(Num >= 2);

	float Sum = 0.f;
	for (int32 Index = 0; Index < Num; ++Index)
	{
		if (Index != IndexToExclude)
		{
			Sum += Array[Index];
		}
	}

	if (Sum > UE_KINDA_SMALL_NUMBER)
	{
		for (int32 Index = 0; Index < Num; ++Index)
		{
			if (Index != IndexToExclude)
			{
				Array[Index] /= Sum;
			}
		}
	}
	else
	{
		const float HomogeneousWeight = (IndexToExclude == INDEX_NONE) ? 1.f / Num : 1.f / (Num - 1);
		for (int32 Index = 0; Index < Num; ++Index)
		{
			if (Index != IndexToExclude)
			{
				Array[Index] = HomogeneousWeight;
			}
		}
	}
}

bool CalculateFullAlignedActorRootBoneTransform(const FTransform& ActorRootBoneTransform, const FTransform& AssetRootBoneTransform, 
	const FVector& ActorsReferencePosition, const FVector& AssetReferencePosition,
	FTransform& OutFullAlignedActorRootBoneTransform, const UWorld* DebugWorld = nullptr)
{
	const FVector ActorRootBonePosition = ActorRootBoneTransform.GetLocation();
				
	// where the asset "reference system position" AssetReferencePosition is in local space to the asset actor
	const FVector AssetReferencePositionLocal = AssetRootBoneTransform.InverseTransformPosition(AssetReferencePosition);
	// where the asset "reference system position" AssetReferencePosition is in world space
	const FVector ActorsReferencePositionFromAsset = ActorRootBoneTransform.TransformPosition(AssetReferencePositionLocal);

	const FVector ActorsReferenceDisplacementFromAsset = ActorsReferencePositionFromAsset - ActorRootBonePosition;
	const FVector ActorsReferenceDisplacement = ActorsReferencePosition - ActorRootBonePosition;

	const float ActorsReferenceDisplacementFromAssetLength = ActorsReferenceDisplacementFromAsset.Length();
	const float ActorsReferenceDisplacementLength = ActorsReferenceDisplacement.Length();

	if (ActorsReferenceDisplacementFromAssetLength > UE_SMALL_NUMBER && ActorsReferenceDisplacementLength > UE_SMALL_NUMBER)
	{
		const FVector ActorsReferenceDisplacementFromAssetYaw(ActorsReferenceDisplacementFromAsset.X, ActorsReferenceDisplacementFromAsset.Y, 0.f);
		const FVector ActorsReferenceDisplacementYaw(ActorsReferenceDisplacement.X, ActorsReferenceDisplacement.Y, 0.f);
		const FVector ActorsReferenceDisplacementFromAssetPitch(0.f, ActorsReferenceDisplacementFromAssetYaw.Length(), ActorsReferenceDisplacementFromAsset.Z);
		const FVector ActorsReferenceDisplacementPitch(0.f, ActorsReferenceDisplacementYaw.Length(), ActorsReferenceDisplacement.Z);

		const FQuat DeltaRotationYaw = FQuat::FindBetweenVectors(ActorsReferenceDisplacementFromAssetYaw, ActorsReferenceDisplacementYaw);
		const FQuat DeltaRotationPitch = FQuat::FindBetweenVectors(ActorsReferenceDisplacementFromAssetPitch, ActorsReferenceDisplacementPitch);

		OutFullAlignedActorRootBoneTransform.SetLocation(ActorsReferencePosition - ActorsReferenceDisplacement * (ActorsReferenceDisplacementFromAssetLength / ActorsReferenceDisplacementLength));
		OutFullAlignedActorRootBoneTransform.SetRotation(ActorRootBoneTransform.GetRotation() * DeltaRotationYaw * DeltaRotationPitch);
		OutFullAlignedActorRootBoneTransform.SetScale3D(ActorRootBoneTransform.GetScale3D());
		
#if ENABLE_ANIM_DEBUG && WITH_EDITOR
		const FVector AssetRootBonePosition = AssetRootBoneTransform.GetLocation();

		DrawDebugLine(DebugWorld, ActorsReferencePosition, ActorRootBonePosition, FColorList::Blue, false, 0.f, SDPG_Foreground, 0.f);
		// line connecting the actor world position and where the reference system world position should be according to the asset
		DrawDebugLine(DebugWorld, ActorsReferencePositionFromAsset, ActorRootBonePosition, FColorList::Red, false, 0.f, SDPG_Foreground, 0.f);
		const FVector ReconstructedForDebugYaw = (DeltaRotationYaw * DeltaRotationPitch).RotateVector(ActorsReferenceDisplacementFromAsset) + ActorRootBonePosition;
		DrawDebugCircle(DebugWorld, FMatrix(FVector::ZAxisVector, FVector::XAxisVector, FVector::YAxisVector, ReconstructedForDebugYaw), 5.f, 16, FColorList::Red, false, 0.f, SDPG_Foreground, 0.f, false);

		// asset actor drawing 
		DrawDebugCircle(DebugWorld, FMatrix(FVector::ZAxisVector, FVector::XAxisVector, FVector::YAxisVector, AssetRootBonePosition), 5.f, 16, FColorList::Green, false, 0.f, SDPG_Foreground, 0.f, false);
		DrawDebugLine(DebugWorld, AssetRootBonePosition, AssetRootBonePosition + AssetRootBoneTransform.GetRotation().RotateVector(FVector::XAxisVector) * 10, FColorList::Green, false, 0.f, SDPG_Foreground, 0.f);
		
		DrawDebugCircle(DebugWorld, FMatrix(FVector::ZAxisVector, FVector::XAxisVector, FVector::YAxisVector, ActorRootBonePosition), 5.f, 16, FColorList::Green, false, 0.f, SDPG_Foreground, 0.f, false);
		DrawDebugLine(DebugWorld, ActorRootBonePosition, ActorRootBonePosition + ActorRootBoneTransform.GetRotation().RotateVector(FVector::XAxisVector) * 10, FColorList::Green, false, 0.f, SDPG_Foreground, 0.f);
#endif // ENABLE_ANIM_DEBUG && WITH_EDITOR

		return true;
	}

	OutFullAlignedActorRootBoneTransform = ActorRootBoneTransform;
	return false;
}

} // namespace UE::PoseSearch

bool UPoseSearchInteractionAsset::IsLooping() const
{
	float CommonPlayLength = -1.f;
	for (const FPoseSearchInteractionAssetItem& Item : Items)
	{
		if (const UAnimationAsset* AnimationAsset = Item.Animation.Get())
		{
			if (const UAnimSequenceBase* SequenceBase = Cast<UAnimSequenceBase>(AnimationAsset))
			{
				if (!SequenceBase->bLoop)
				{
					return false;
				}
			}
			else if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(AnimationAsset))
			{
				if (!BlendSpace->bLoop)
				{
					return false;
				}
			}
			else
			{
				UE_LOGF(LogPoseSearch, Warning, "UPoseSearchInteractionAsset::IsLooping non fully supported UAnimationAsset derived type '%ls' used as item in '%ls'", *AnimationAsset->GetClass()->GetName(), *GetName());
				return false;
			}

			if (CommonPlayLength < 0.f)
			{
				CommonPlayLength = AnimationAsset->GetPlayLength();
			}
			else if (!FMath::IsNearlyEqual(CommonPlayLength, AnimationAsset->GetPlayLength()))
			{
				return false;
			}
		}
	}
	return true;
}

bool UPoseSearchInteractionAsset::HasRootMotion() const
{
	bool bHasAtLeastOneValidItem = false;
	bool bHasRootMotion = true;

	for (const FPoseSearchInteractionAssetItem& Item : Items)
	{
		if (const UAnimationAsset* AnimationAsset = Item.Animation.Get())
		{
			if (const UAnimSequenceBase* SequenceBase = Cast<UAnimSequenceBase>(AnimationAsset))
			{
				bHasRootMotion &= SequenceBase->HasRootMotion();
			}
			else if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(AnimationAsset))
			{
				BlendSpace->ForEachImmutableSample([&bHasRootMotion](const FBlendSample& Sample)
				{
					if (const UAnimSequence* Sequence = Sample.Animation.Get())
					{
						bHasRootMotion &= Sequence->HasRootMotion();
					}
				});
			}
			else
			{
				UE_LOGF(LogPoseSearch, Warning, "UPoseSearchInteractionAsset::HasRootMotion non fully supported UAnimationAsset derived type '%ls' used as item in '%ls'", *AnimationAsset->GetClass()->GetName(), *GetName());
			}
			bHasAtLeastOneValidItem = true;
		}
	}

	return bHasAtLeastOneValidItem && bHasRootMotion;
}

float UPoseSearchInteractionAsset::GetPlayLength(const FVector& BlendParameters) const
{
	float MaxPlayLength = 0.f;
	for (const FPoseSearchInteractionAssetItem& Item : Items)
	{
		if (const UAnimationAsset* AnimationAsset = Item.Animation.Get())
		{
			if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(AnimationAsset))
			{
				int32 TriangulationIndex = 0;
				TArray<FBlendSampleData> BlendSamples;
				BlendSpace->GetSamplesFromBlendInput(BlendParameters, BlendSamples, TriangulationIndex, true);
				const float PlayLength = BlendSpace->GetAnimationLengthFromSampleData(BlendSamples);
				MaxPlayLength = FMath::Max(MaxPlayLength, PlayLength);
			}
			else
			{
				MaxPlayLength = FMath::Max(MaxPlayLength, AnimationAsset->GetPlayLength());
			}
		}
	}
	return MaxPlayLength;
}

UAnimationAsset* UPoseSearchInteractionAsset::GetAnimationAsset(const UE::PoseSearch::FRole& Role) const
{
	for (const FPoseSearchInteractionAssetItem& Item : Items)
	{
		if (Item.Role == Role)
		{
			return Item.Animation;
		}
	}
	return nullptr;
}

FTransform UPoseSearchInteractionAsset::GetOrigin(const UE::PoseSearch::FRole& Role) const
{
	for (const FPoseSearchInteractionAssetItem& Item : Items)
	{
		if (Item.Role == Role)
		{
			return Item.Origin;
		}
	}
	return FTransform::Identity;
}

#if WITH_EDITOR
FTransform UPoseSearchInteractionAsset::GetDebugWarpOrigin(const UE::PoseSearch::FRole& Role, bool bComposeWithDebugWarpOffset) const
{
	for (int32 ItemIndex = 0; ItemIndex < Items.Num(); ++ItemIndex)
	{
		const FPoseSearchInteractionAssetItem& Item = Items[ItemIndex];
		if (Item.Role == Role)
		{
#if WITH_EDITORONLY_DATA
			if (bComposeWithDebugWarpOffset && bEnableDebugWarp && DebugWarpOffsets.IsValidIndex(ItemIndex))
			{
				return DebugWarpOffsets[ItemIndex] * Item.Origin;
			}
#endif // WITH_EDITORONLY_DATA

			return Item.Origin;
		}
	}
	return FTransform::Identity;
}

USkeletalMesh* UPoseSearchInteractionAsset::GetPreviewMesh(const UE::PoseSearch::FRole& Role) const
{
	for (int32 ItemIndex = 0; ItemIndex < Items.Num(); ++ItemIndex)
	{
		const FPoseSearchInteractionAssetItem& Item = Items[ItemIndex];
		if (Item.Role == Role)
		{
			return Item.PreviewMesh.Get();
		}
	}
	return nullptr;
}

#endif // WITH_EDITOR

void UPoseSearchInteractionAsset::CalculateWarpTransforms(float Time, float TimeOffset, const TConstArrayView<const FTransform> ActorRootBoneTransforms, TArrayView<FTransform> FullAlignedActorRootBoneTransforms,
	const TConstArrayView<const UMirrorDataTable*> MirrorDataTables, const TConstArrayView<TObjectPtr<const UObject>> DebugAnimContexts) const
{
	using namespace UE::PoseSearch;

	check(ActorRootBoneTransforms.Num() == GetNumRoles());
	check(FullAlignedActorRootBoneTransforms.Num() == GetNumRoles());
	
	const int32 ItemsNum = Items.Num();
	switch (ItemsNum)
	{
	case 0:
	{
		break;
	}
	case 1:
	{
		FullAlignedActorRootBoneTransforms[0] = ActorRootBoneTransforms[0];
		break;
	}
	default:
	{
		check(ItemsNum > 1);

		TArray<FTransform, TInlineAllocator<PreallocatedRolesNum>> AssetRootBoneTransforms;
		AssetRootBoneTransforms.SetNum(ItemsNum);

		TArray<FTransform, TInlineAllocator<PreallocatedRolesNum>> OffsettedAssetRootBoneTransforms;
		OffsettedAssetRootBoneTransforms.SetNum(ItemsNum);

		// ItemIndex is the RoleIndex and Role = Item.Role
		for (int32 ItemIndex = 0; ItemIndex < ItemsNum; ++ItemIndex)
		{
			const FPoseSearchInteractionAssetItem& Item = Items[ItemIndex];

			// sampling the AnimationAsset to extract the current time transform and the initial (time of 0) transform
			const FAnimationAssetSampler Sampler(Item.Animation, Item.Origin);
			AssetRootBoneTransforms[ItemIndex] = Sampler.ExtractRootTransform(Time);

			if (MirrorDataTables.IsValidIndex(ItemIndex) && MirrorDataTables[ItemIndex])
			{
				const FMirrorDataCache MirrorDataCache(MirrorDataTables[ItemIndex]);
				AssetRootBoneTransforms[ItemIndex] = MirrorDataCache.MirrorTransform(AssetRootBoneTransforms[ItemIndex]);

				if (TimeOffset > UE_KINDA_SMALL_NUMBER)
				{
					OffsettedAssetRootBoneTransforms[ItemIndex] = MirrorDataCache.MirrorTransform(Sampler.ExtractRootTransform(Time + TimeOffset));
				}
			}
			else if (TimeOffset > UE_KINDA_SMALL_NUMBER)
			{
				OffsettedAssetRootBoneTransforms[ItemIndex] = Sampler.ExtractRootTransform(Time + TimeOffset);
			}

#if ENABLE_ANIM_DEBUG && WITH_EDITOR
			if (Items[ItemIndex].Animation)
			{
				// array containing the bone index of the root bone (0)
				TArray<uint16, TInlineAllocator<1>> BoneIndices;
				BoneIndices.SetNumZeroed(1);

				// extracting the pose, containing only the root bone from the Sampler 
				FMemMark Mark(FMemStack::Get());
				FCompactPose Pose;
				FBoneContainer BoneContainer;
				BoneContainer.InitializeTo(BoneIndices, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *Items[ItemIndex].Animation->GetSkeleton());
				Pose.SetBoneContainer(&BoneContainer);
				Sampler.ExtractPose(Time, Pose);

				// making sure the animation root bone transform is Identity, so we can confuse the root with the root BONE transform and preserve performances!
				const FTransform& RootBoneTransform = Pose.GetBones()[0];
				if (!RootBoneTransform.Equals(FTransform::Identity))
				{
					const FVector Pos = RootBoneTransform.GetLocation();
					const FRotator Rot(RootBoneTransform.GetRotation());
					UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionAsset::CalculateWarpTransforms unsupported non identity root bone in %ls at time %f Pos(%f, %f, %f), Rot(%f, %f, %f)", *Items[ItemIndex].Animation->GetName(), Time, Pos.X, Pos.Y, Pos.Z, Rot.Pitch, Rot.Yaw, Rot.Roll);
				}
			}
#endif // ENABLE_ANIM_DEBUG && WITH_EDITOR
		}

		TArray<float, TInlineAllocator<PreallocatedRolesNum>> NormalizedWarpingWeightTranslation;
		NormalizedWarpingWeightTranslation.SetNumUninitialized(ItemsNum);
		for (int32 ItemIndex = 0; ItemIndex < ItemsNum; ++ItemIndex)
		{
			NormalizedWarpingWeightTranslation[ItemIndex] = Items[ItemIndex].WarpingWeightTranslation;
		}
		Normalize(NormalizedWarpingWeightTranslation);

		const FVector AssetReferencePosition = FindReferencePosition(AssetRootBoneTransforms, NormalizedWarpingWeightTranslation);

		TArray<FTransform, TInlineAllocator<PreallocatedRolesNum>> AdjustedActorRootBoneTransforms(ActorRootBoneTransforms);
		FVector ActorsReferencePosition = FindReferencePosition(AdjustedActorRootBoneTransforms, NormalizedWarpingWeightTranslation);
		if (WarpingBankingWeight < 1.f)
		{
			// adjusting AdjustedActorRootBoneTransforms to account for WarpingBankingWeight iteratively
			// we should iterate until converging (bContinueIterating = false at the end of the loop), but couple of iterations should be enough
			constexpr int32 MaxNumberOfIterations = 3;

			bool bContinueIterating = false;
			for (int32 IterationIndex = 0; IterationIndex < MaxNumberOfIterations; ++IterationIndex)
			{
				for (int32 ItemIndex = 0; ItemIndex < ItemsNum; ++ItemIndex)
				{
					// @todo: remove the assumption that z axis is up by passing an up vector as input
					// @todo: this value could be cached between iterations
					const float AssetDelta = AssetRootBoneTransforms[ItemIndex].GetLocation().Z - AssetReferencePosition.Z;
					const float ActorDelta = AdjustedActorRootBoneTransforms[ItemIndex].GetLocation().Z - ActorsReferencePosition.Z;

					const float WantedDelta = FMath::Lerp(AssetDelta, ActorDelta, WarpingBankingWeight);
					if (!FMath::IsNearlyEqual(ActorDelta, WantedDelta, UE_KINDA_SMALL_NUMBER))
					{
						FVector NewLocation = AdjustedActorRootBoneTransforms[ItemIndex].GetLocation();
						NewLocation.Z = WantedDelta + ActorsReferencePosition.Z;
						AdjustedActorRootBoneTransforms[ItemIndex].SetLocation(NewLocation);
						bContinueIterating = true;
					}
				}

				if (!bContinueIterating)
				{
					break;
				}

				ActorsReferencePosition = FindReferencePosition(AdjustedActorRootBoneTransforms, NormalizedWarpingWeightTranslation);
			}
		}

		const UWorld* DebugWorld = nullptr;
#if ENABLE_ANIM_DEBUG && WITH_EDITOR
		for (const TObjectPtr<const UObject>& DebugAnimContextPtr : DebugAnimContexts)
		{
			if (const UObject* Outer = DebugAnimContextPtr->GetOuter())
			{
				DebugWorld = Outer->GetWorld();
				if (DebugWorld)
				{
					break;
				}
			}
		}
#endif // ENABLE_ANIM_DEBUG && WITH_EDITOR

		if (MinimalTranslationWeight > 0.f)
		{
			// finding FullAlignedActorRootBoneTransforms with the minimal possible translations
			for (int32 ItemIndex = 0; ItemIndex < ItemsNum; ++ItemIndex)
			{
				if (!CalculateFullAlignedActorRootBoneTransform(AdjustedActorRootBoneTransforms[ItemIndex], AssetRootBoneTransforms[ItemIndex],
					ActorsReferencePosition, AssetReferencePosition, FullAlignedActorRootBoneTransforms[ItemIndex], DebugWorld))
				{
					// finding the references without the ItemIndex, since the current actors and asset references are too close to AdjustedActorRootBoneTransforms[ItemIndex] and AssetRootBoneTransforms[ItemIndex]
					TArray<float, TInlineAllocator<PreallocatedRolesNum>> NormalizedWarpingWeightTranslationWithoutItemIndex;
					NormalizedWarpingWeightTranslationWithoutItemIndex = NormalizedWarpingWeightTranslation;
					NormalizedWarpingWeightTranslationWithoutItemIndex[ItemIndex] = 0.f;

					Normalize(NormalizedWarpingWeightTranslationWithoutItemIndex, ItemIndex);

					const FVector AssetReferencePositionWithoutItemIndex = FindReferencePosition(AssetRootBoneTransforms, NormalizedWarpingWeightTranslationWithoutItemIndex);
					const FVector ActorsReferencePositionWithoutItemIndex = FindReferencePosition(AdjustedActorRootBoneTransforms, NormalizedWarpingWeightTranslationWithoutItemIndex);

#if ENABLE_ANIM_DEBUG && WITH_EDITOR
					DrawDebugCircle(DebugWorld, FMatrix(FVector::ZAxisVector, FVector::XAxisVector, FVector::YAxisVector, AssetReferencePositionWithoutItemIndex), 11.f, 16, FColorList::Black, false, 0.f, SDPG_Foreground, 0.f, false);
					DrawDebugCircle(DebugWorld, FMatrix(FVector::ZAxisVector, FVector::XAxisVector, FVector::YAxisVector, ActorsReferencePositionWithoutItemIndex), 13.f, 16, FColorList::Grey, false, 0.f, SDPG_Foreground, 0.f, false);
#endif // ENABLE_ANIM_DEBUG && WITH_EDITOR

					if (CalculateFullAlignedActorRootBoneTransform(AdjustedActorRootBoneTransforms[ItemIndex], AssetRootBoneTransforms[ItemIndex],
						ActorsReferencePositionWithoutItemIndex, AssetReferencePositionWithoutItemIndex, FullAlignedActorRootBoneTransforms[ItemIndex], DebugWorld))
					{
						FullAlignedActorRootBoneTransforms[ItemIndex].SetLocation(AdjustedActorRootBoneTransforms[ItemIndex].GetLocation());
					}
					else
					{
						UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionAsset::CalculateWarpTransforms actors are too close to each other to be able to calculate warp transforms ['%ls' at time %f]", *GetName(), Time);
					}
				}
			}

#if ENABLE_ANIM_DEBUG && WITH_EDITOR
			DrawDebugCircle(DebugWorld, FMatrix(FVector::ZAxisVector, FVector::XAxisVector, FVector::YAxisVector, AssetReferencePosition), 7.f, 16, FColorList::Blue, false, 0.f, SDPG_Foreground, 0.f, false);
			DrawDebugCircle(DebugWorld, FMatrix(FVector::ZAxisVector, FVector::XAxisVector, FVector::YAxisVector, ActorsReferencePosition), 9.f, 16, FColorList::Green, false, 0.f, SDPG_Foreground, 0.f, false);
#endif // ENABLE_ANIM_DEBUG && WITH_EDITOR
		}

		if (MinimalTranslationWeight < 1.f)
		{
			TArray<int32, TInlineAllocator<PreallocatedRolesNum>> SortedByWarpingWeightRotationItemIndex;
			TArray<float, TInlineAllocator<PreallocatedRolesNum>> NormalizedWarpingWeightRotation;
			SortedByWarpingWeightRotationItemIndex.SetNum(ItemsNum);
			NormalizedWarpingWeightRotation.SetNum(ItemsNum);

			float WarpingWeightRotationSum = 0.f;
			for (int32 ItemIndex = 0; ItemIndex < ItemsNum; ++ItemIndex)
			{
				SortedByWarpingWeightRotationItemIndex[ItemIndex] = ItemIndex;
				WarpingWeightRotationSum += Items[ItemIndex].WarpingWeightRotation;
			}

			const float NormalizedHomogeneousWeight = 1.f / ItemsNum;
			if (WarpingWeightRotationSum > UE_KINDA_SMALL_NUMBER)
			{
				SortedByWarpingWeightRotationItemIndex.Sort([this](const int32 A, const int32 B)
					{
						return Items[A].WarpingWeightRotation < Items[B].WarpingWeightRotation;
					});

				for (int32 ItemIndex = 0; ItemIndex < ItemsNum; ++ItemIndex)
				{
					NormalizedWarpingWeightRotation[ItemIndex] = Items[ItemIndex].WarpingWeightRotation / WarpingWeightRotationSum;
				}
			}
			else
			{
				for (int32 ItemIndex = 0; ItemIndex < ItemsNum; ++ItemIndex)
				{
					NormalizedWarpingWeightRotation[ItemIndex] = NormalizedHomogeneousWeight;
				}
			}

			const FQuat AssetReferenceOrientation = FindReferenceOrientation(AssetRootBoneTransforms, SortedByWarpingWeightRotationItemIndex, NormalizedWarpingWeightRotation);
			const FQuat ActorsReferenceOrientation = FindReferenceOrientation(AdjustedActorRootBoneTransforms, SortedByWarpingWeightRotationItemIndex, NormalizedWarpingWeightRotation);

			FQuat WeightedActorsReferenceOrientation = ActorsReferenceOrientation;
			if (WarpingWeightRotationSum > UE_KINDA_SMALL_NUMBER)
			{
				// ItemIndex are in order of WarpingWeightRotation. the last one is the one with the highest most WarpingWeightRotation, the most "important"
				for (int32 ItemIndex : SortedByWarpingWeightRotationItemIndex)
				{
					const FPoseSearchInteractionAssetItem& Item = Items[ItemIndex];
					if (NormalizedWarpingWeightRotation[ItemIndex] > NormalizedHomogeneousWeight)
					{
						// NormalizedHomogeneousWeight is one only if ItemsNum is one, 
						// BUT NormalizedWarpingWeightRotation[ItemIndex] > NormalizedHomogeneousWeight should always be false
						check(!FMath::IsNearlyEqual(NormalizedHomogeneousWeight, 1.f));

						// how much this item wants to reorient the ReferenceOrientation from the homogeneous "fair" value
						const float SlerpParam = (NormalizedWarpingWeightRotation[ItemIndex] - NormalizedHomogeneousWeight) / (1.f - NormalizedHomogeneousWeight);

						// calculating the reference orientation relative to the character
						// AssetReferenceOrientation in actor world orientation
						const FQuat ActorAssetReferenceOrientation = AdjustedActorRootBoneTransforms[ItemIndex].GetRotation() * (AssetRootBoneTransforms[ItemIndex].GetRotation().Inverse() * AssetReferenceOrientation);

						WeightedActorsReferenceOrientation = FQuat::Slerp(WeightedActorsReferenceOrientation, ActorAssetReferenceOrientation, SlerpParam);
					}
				}
			}

			// aligning all the actors to ActorsReferencePosition, WeightedActorsReferenceOrientation
			const FTransform AssetReferenceTransform(AssetReferenceOrientation, AssetReferencePosition);
			const FTransform ActorsReferenceTransform(WeightedActorsReferenceOrientation, ActorsReferencePosition);
			const FTransform AssetReferenceInverseTransform = AssetReferenceTransform.Inverse();

			// blend FullAlignedActorRootBoneTransforms with what has been calcualted by finding FullAlignedActorRootBoneTransforms with the minimal possible translations
			for (int32 ItemIndex = 0; ItemIndex < ItemsNum; ++ItemIndex)
			{
				const FTransform WantedFullAlignedActorRootBoneTransforms = (AssetRootBoneTransforms[ItemIndex] * AssetReferenceInverseTransform) * ActorsReferenceTransform;
				if (MinimalTranslationWeight > 0.f)
				{
					FullAlignedActorRootBoneTransforms[ItemIndex].SetLocation(FMath::Lerp<FVector>(WantedFullAlignedActorRootBoneTransforms.GetLocation(), FullAlignedActorRootBoneTransforms[ItemIndex].GetLocation(), MinimalTranslationWeight));
					FullAlignedActorRootBoneTransforms[ItemIndex].SetRotation(FQuat::Slerp(WantedFullAlignedActorRootBoneTransforms.GetRotation(), FullAlignedActorRootBoneTransforms[ItemIndex].GetRotation(), MinimalTranslationWeight));
					FullAlignedActorRootBoneTransforms[ItemIndex].SetScale3D(FMath::Lerp<FVector>(WantedFullAlignedActorRootBoneTransforms.GetScale3D(), FullAlignedActorRootBoneTransforms[ItemIndex].GetScale3D(), MinimalTranslationWeight));
				}
				else
				{
					FullAlignedActorRootBoneTransforms[ItemIndex] = WantedFullAlignedActorRootBoneTransforms;
				}
			}

#if ENABLE_ANIM_DEBUG && WITH_EDITOR
			DrawDebugCircle(DebugWorld, FMatrix(FVector::ZAxisVector, FVector::XAxisVector, FVector::YAxisVector, AssetReferencePosition), 7.f, 16, FColorList::Blue, false, 0.f, SDPG_Foreground, 0.f, false);
			DrawDebugCircle(DebugWorld, FMatrix(FVector::ZAxisVector, FVector::XAxisVector, FVector::YAxisVector, ActorsReferencePosition), 9.f, 16, FColorList::Green, false, 0.f, SDPG_Foreground, 0.f, false);

			DrawDebugLine(DebugWorld, AssetReferencePosition, AssetReferencePosition + AssetReferenceOrientation.RotateVector(FVector::XAxisVector) * 10.f, FColorList::Blue, false, 0.f, SDPG_Foreground, 0.f);
			DrawDebugLine(DebugWorld, ActorsReferencePosition, ActorsReferencePosition + ActorsReferenceOrientation.RotateVector(FVector::XAxisVector) * 10.f, FColorList::Green, false, 0.f, SDPG_Foreground, 0.f);
#endif // ENABLE_ANIM_DEBUG && WITH_EDITOR
		}

		if (TimeOffset > UE_KINDA_SMALL_NUMBER)
		{
			for (int32 ItemIndex = 0; ItemIndex < ItemsNum; ++ItemIndex)
			{
				// calculating the delta transform that moves the root bone from Time to Time + TimeOffset
				const FTransform DeltaAssetRootBoneTransform = OffsettedAssetRootBoneTransforms[ItemIndex].GetRelativeTransform(AssetRootBoneTransforms[ItemIndex]);
				FullAlignedActorRootBoneTransforms[ItemIndex] = DeltaAssetRootBoneTransform * FullAlignedActorRootBoneTransforms[ItemIndex];
			}
		}

		break;
	}
	}
}
