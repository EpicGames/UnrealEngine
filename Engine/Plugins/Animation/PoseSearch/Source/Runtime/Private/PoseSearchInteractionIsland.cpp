// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchInteractionIsland.h"
#include "Animation/AnimClassInterface.h"
#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "DrawDebugHelpers.h"
#include "Engine/SkeletalMesh.h"
#include "Features/IModularFeatures.h"
#include "PoseSearch/AnimNode_PoseSearchHistoryCollector.h"
#include "PoseSearch/MultiAnimAsset.h"
#include "PoseSearch/PoseSearchAnimNotifies.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchInteractionLibrary.h"
#include "PoseSearch/PoseSearchInteractionSubsystem.h"
#include "PoseSearch/PoseSearchInteractionValidator.h"
#include "PoseSearch/PoseSearchLibrary.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "VisualLogger/VisualLogger.h"

namespace UE::PoseSearch
{

#if ENABLE_ANIM_DEBUG
static bool GVarPoseSearchInteractionDiagnoseTickDependencies = false;
static FAutoConsoleVariableRef CVarPoseSearchInteractionDiagnoseTickDependencies(TEXT("a.PoseSearchInteraction.DiagnoseTickDependencies"), GVarPoseSearchInteractionDiagnoseTickDependencies, TEXT("Enable Pose Search Interaction Tick Dependencies Diagnostic (SLOW!)"));

// recursion safe FTickFunction logging functions
static void ShowPrerequistes(FTickFunction& NestedTick, int32 Indent, TSet<FTickFunction*>& VisitedTickFunctions)
{
	bool bAlreadyInSet = false;
	VisitedTickFunctions.Add(&NestedTick, &bAlreadyInSet);

	if (bAlreadyInSet)
	{
		if (ensure(Indent > 0))
		{
			UE_LOGF(LogPoseSearch, Log, "%ls ERROR! Cycle detected! Use 'a.PoseSearchInteraction.DiagnoseTickDependencies' CVar to gather more infos to diagnose the problem", FCString::Spc((Indent - 1) * 2));
		}
	}
	else
	{
		for (const FTickPrerequisite& Prereq : NestedTick.GetPrerequisites())
		{
			if (Prereq.PrerequisiteTickFunction)
			{
				UE_LOGF(LogPoseSearch, Log, "%ls prereq %ls", FCString::Spc(Indent * 2), *Prereq.PrerequisiteTickFunction->DiagnosticMessage());
				ShowPrerequistes(*Prereq.PrerequisiteTickFunction, Indent + 1, VisitedTickFunctions);
			}
		}
		VisitedTickFunctions.Remove(&NestedTick);
	}
}

static void LogTickFunction(FTickFunction& Tick, ENamedThreads::Type CurrentThread, bool bLogPrerequisites, int32 Indent)
{
	// scoping brackets to save some heap for the recursion
	{
		UE_LOGF(LogPoseSearch, Log, "%lstick %ls [%1d, %1d] %6llu %2d %ls", FCString::Spc(Indent * 2), Tick.bHighPriority ? TEXT("*") : TEXT(" "), (int32)Tick.GetActualTickGroup(), (int32)Tick.GetActualEndTickGroup(), (uint64)GFrameCounter, (int32)CurrentThread, *Tick.DiagnosticMessage());
		if (bLogPrerequisites)
		{
			TSet<FTickFunction*> VisitedTickFunctions;
			ShowPrerequistes(Tick, Indent, VisitedTickFunctions);
			ensure(VisitedTickFunctions.IsEmpty());
		}
	}

	// Handle nested ticks
	Tick.ForEachNestedTick([CurrentThread, bLogPrerequisites, Indent](FTickFunction& NestedTick)
		{
			LogTickFunction(NestedTick, CurrentThread, bLogPrerequisites, Indent + 1);
		});
}

// check if there's any cycle within the prerequisites of Tick
static bool ValidateTickDependenciesCycles(FTickFunction& Tick, TSet<FTickFunction*>& VisitedTickFunctions)
{
	bool bValidatedCorrectly = true;

	bool bAlreadyInSet = false;
	VisitedTickFunctions.Add(&Tick, &bAlreadyInSet);

	if (bAlreadyInSet)
	{
		bValidatedCorrectly = false;
	}
	else
	{
		for (const FTickPrerequisite& Prereq : Tick.GetPrerequisites())
		{
			if (Prereq.PrerequisiteTickFunction)
			{
				if (!ValidateTickDependenciesCycles(*Prereq.PrerequisiteTickFunction, VisitedTickFunctions))
				{
					bValidatedCorrectly = false;
					break;
				}
			}
		}

		if (bValidatedCorrectly)
		{
			Tick.ForEachNestedTick([&VisitedTickFunctions, &bValidatedCorrectly](FTickFunction& NestedTick)
				{
					if (bValidatedCorrectly)
					{
						if (!ValidateTickDependenciesCycles(NestedTick, VisitedTickFunctions))
						{
							bValidatedCorrectly = false;
						}
					}
				});
		}
		VisitedTickFunctions.Remove(&Tick);
	}
	
	return bValidatedCorrectly;
}
#endif // ENABLE_ANIM_DEBUG

static FInteractionSearchResult InitSearchResult(const FSearchResult& SearchResult, const FInteractionSearchContext& SearchContext, int32 SearchIndex)
{
	FInteractionSearchResult InteractionSearchResult;
	static_cast<FSearchResult&>(InteractionSearchResult) = SearchResult;
	InteractionSearchResult.SearchIndex = SearchIndex;

	const int32 AnimContextsNum = SearchContext.Num();
	InteractionSearchResult.ActorRootTransforms.SetNum(AnimContextsNum);
	InteractionSearchResult.ActorRootBoneTransforms.SetNum(AnimContextsNum);

	for (int32 AnimContextIndex = 0; AnimContextIndex < AnimContextsNum; ++AnimContextIndex)
	{
		if (const IPoseHistory* PoseHistory = SearchContext.GetPoseHistory(AnimContextIndex))
		{
			const UObject* AnimContext = SearchContext.GetAnimContext(AnimContextIndex);
			if (ensure(AnimContext))
			{
				const USkeleton* Skeleton = GetContextSkeleton(AnimContext, false);
				if (ensure(Skeleton))
				{
					PoseHistory->GetTransformAtTime(0.f, InteractionSearchResult.ActorRootTransforms[AnimContextIndex], Skeleton, ComponentSpaceIndexType, WorldSpaceIndexType);
					PoseHistory->GetTransformAtTime(0.f, InteractionSearchResult.ActorRootBoneTransforms[AnimContextIndex], Skeleton, RootBoneIndexType, ComponentSpaceIndexType);
				}
				else
				{
					InteractionSearchResult = FInteractionSearchResult();
					break;
				}
			}
			else
			{
				InteractionSearchResult = FInteractionSearchResult();
				break;
			}
		}
		else
		{
			// a pose history got disposed, this result is invalid
			InteractionSearchResult = FInteractionSearchResult();
			break;
		}
	}

	return InteractionSearchResult;
}

static bool IsPoseSearchResultUsable(int32 SearchIndex, TConstArrayView<FSearchResult> PoseSearchResults, TConstArrayView<FInteractionSearchContext> SearchContexts, const FStackAssetSet& VisitedAnimContexts)
{
	if (!PoseSearchResults[SearchIndex].IsValid())
	{
		return false;
	}

	for (int32 AnimContextIndex = 0; AnimContextIndex < SearchContexts[SearchIndex].Num(); ++AnimContextIndex)
	{
		if (const UObject* ValidSearchAnimContext = SearchContexts[SearchIndex].GetAnimContext(AnimContextIndex))
		{
			if (VisitedAnimContexts.Contains(ValidSearchAnimContext))
			{
				return false;
			}
		}
		else
		{
			return false;
		}
	}

	return true;
}

static void InitSearchResults(TArray<FInteractionSearchResult>& SearchResults, TConstArrayView<FSearchResult> PoseSearchResults, TConstArrayView<FInteractionSearchContext> SearchContexts)
{
	SearchResults.Reset();

	if (!PoseSearchResults.IsEmpty())
	{
		TArray<int32, TMemStackAllocator<>> SortedPoseSearchResults;
		SortedPoseSearchResults.SetNum(PoseSearchResults.Num());
		for (int32 Index = 0; Index < PoseSearchResults.Num(); ++Index)
		{
			SortedPoseSearchResults[Index] = Index;
		}

		SortedPoseSearchResults.StableSort([&SearchContexts, &PoseSearchResults](int32 IndexA, int32 IndexB)
			{
				const bool IsValidA = PoseSearchResults[IndexA].IsValid();
				const bool IsValidB = PoseSearchResults[IndexB].IsValid();

				if (IsValidA && !IsValidB)
				{
					return true;
				}

				if (!IsValidA && IsValidB)
				{
					return false;
				}

				if (!IsValidA && !IsValidB)
				{
					return true;
				}

				const int32 NumRolesA = SearchContexts[IndexA].Num();
				const int32 NumRolesB = SearchContexts[IndexB].Num();

				if (NumRolesA > NumRolesB)
				{
					return true;
				}

				if (NumRolesA < NumRolesB)
				{
					return false;
				}

				return PoseSearchResults[IndexA].PoseCost < PoseSearchResults[IndexB].PoseCost;
			});

		// assign from best to worst result
		FStackAssetSet VisitedAnimContexts;
		for (int32 SearchIndex : SortedPoseSearchResults)
		{
			if (IsPoseSearchResultUsable(SearchIndex, PoseSearchResults, SearchContexts, VisitedAnimContexts))
			{
				SearchResults.Add(InitSearchResult(PoseSearchResults[SearchIndex], SearchContexts[SearchIndex], SearchIndex));

				for (int32 AnimContextIndex = 0; AnimContextIndex < SearchContexts[SearchIndex].Num(); ++AnimContextIndex)
				{
					const UObject* SearchAnimContext = SearchContexts[SearchIndex].GetAnimContext(AnimContextIndex);
					if (ensure(SearchAnimContext))
					{
						VisitedAnimContexts.Add(SearchAnimContext);
					}
				}
			}
		}
	}
}

static UActorComponent* FindComponentForTickDependencies(const UObject* AnimContext)
{
	ensure(AnimContext);
	
	if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(AnimContext))
	{
		return AnimInstance->GetSkelMeshComponent();
	}
	
	// this is the AnimNext case
	return const_cast<UActorComponent*>(Cast<UActorComponent>(AnimContext));
}

static void AddPrerequisite(FTickFunction& TickFunction, UObject* TargetObject, FTickFunction& TargetTickFunction)
{
	if (TargetObject)
	{
#if ENABLE_ANIM_DEBUG
		if (GVarPoseSearchInteractionDiagnoseTickDependencies)
		{
			const ENamedThreads::Type Type = IsInGameThread() ? ENamedThreads::GameThread : ENamedThreads::AnyThread;
			TSet<FTickFunction*> VisitedTickFunctions;
			if (!ValidateTickDependenciesCycles(TickFunction, VisitedTickFunctions))
			{
				LogTickFunction(TickFunction, Type, true, 1);
			}
			ensure(VisitedTickFunctions.IsEmpty());
			if (!ValidateTickDependenciesCycles(TargetTickFunction, VisitedTickFunctions))
			{
				LogTickFunction(TargetTickFunction, Type, true, 1);
			}
			ensure(VisitedTickFunctions.IsEmpty());
		}
#endif // ENABLE_ANIM_DEBUG

		TickFunction.AddPrerequisite(TargetObject, TargetTickFunction);

#if ENABLE_ANIM_DEBUG
		if (TickFunction.bCanEverTick && !TickFunction.GetPrerequisites().Contains(FTickPrerequisite(TargetObject, TargetTickFunction)))
		{
			UE_LOGF(LogPoseSearch, Error, "UE::PoseSearch::AddPrerequisite, Failed to add prerequisite from [%ls] to [%ls, %ls]!", *TickFunction.DiagnosticMessage(), *TargetObject->GetName(), *TargetTickFunction.DiagnosticMessage());
		}

		if (GVarPoseSearchInteractionDiagnoseTickDependencies)
		{
			const ENamedThreads::Type Type = IsInGameThread() ? ENamedThreads::GameThread : ENamedThreads::AnyThread;
			TSet<FTickFunction*> VisitedTickFunctions;
			if (!ValidateTickDependenciesCycles(TickFunction, VisitedTickFunctions))
			{
				LogTickFunction(TickFunction, Type, true, 1);
			}
			ensure(VisitedTickFunctions.IsEmpty());
		}
#endif // ENABLE_ANIM_DEBUG
	}
}

// FInteractionSearchContextBase
///////////////////////////////////////////////////////////
const UE::PoseSearch::IPoseHistory* FInteractionSearchContextBase::GetPoseHistory(int32 Index) const
{
	if (ensure(PoseHistories.IsValidIndex(Index)))
	{
		if (TSharedPtr<const IPoseHistory, ESPMode::ThreadSafe> Pin = PoseHistories[Index].Pin())
		{
			return Pin.Get();
		}
	}
	return nullptr;
}

void FInteractionSearchContextBase::CacheHashesForEquivalence()
{
	// trying to inefficiently call CalculateHashesForEquivalence multiple times?
	ensure(CachedHashForEquivalence == InvalidCachedHashForEquivalence && CachedContextHashForEquivalence == InvalidCachedHashForEquivalence);
	CalculateHashesForEquivalence(CachedHashForEquivalence, CachedContextHashForEquivalence);
}

void FInteractionSearchContextBase::CalculateHashesForEquivalence(uint32& OutCachedHashForEquivalence, uint32& OutCachedContextHashForEquivalence) const
{
	if (ensure(!AnimContexts.IsEmpty()))
	{
		OutCachedContextHashForEquivalence = GetTypeHash(AnimContexts[0]);

		for (int32 Index = 1; Index < AnimContexts.Num(); ++Index)
		{
			// AnimContexts must be sorted to have deterministic searches across multiple frames
			ensure(AnimContexts[Index - 1] < AnimContexts[Index]);
			OutCachedContextHashForEquivalence = HashCombineFast(OutCachedContextHashForEquivalence, GetTypeHash(AnimContexts[Index]));
		}

		ensure(OutCachedContextHashForEquivalence != InvalidCachedHashForEquivalence);

		// skipping bDisableCollisions and PoseHistories for hashing
		OutCachedHashForEquivalence = HashCombineFast(OutCachedContextHashForEquivalence, GetTypeHash(Database));

		for (int32 Index = 0; Index < Roles.Num(); ++Index)
		{
			OutCachedHashForEquivalence = HashCombineFast(OutCachedHashForEquivalence, GetTypeHash(Roles[Index]));
		}

		// only an invalid hash is zero!
		ensure(OutCachedHashForEquivalence != InvalidCachedHashForEquivalence);
	}
	else
	{
		OutCachedContextHashForEquivalence = InvalidCachedHashForEquivalence;
		OutCachedContextHashForEquivalence = InvalidCachedHashForEquivalence;
	}
}

uint32 FInteractionSearchContextBase::GetHashForEquivalence() const
{
#if DO_CHECK
	ensure(CachedHashForEquivalence != InvalidCachedHashForEquivalence && CachedContextHashForEquivalence != InvalidCachedHashForEquivalence);
	// making sure properties didn't change without updating CachedHashForEquivalence
	uint32 TestCachedHashForEquivalence, TestCachedContextHashForEquivalence;
	CalculateHashesForEquivalence(TestCachedHashForEquivalence, TestCachedContextHashForEquivalence);
	ensure(CachedHashForEquivalence == TestCachedHashForEquivalence && CachedContextHashForEquivalence == TestCachedContextHashForEquivalence);
#endif //DO_CHECK
	return CachedHashForEquivalence;
}

uint32 FInteractionSearchContextBase::GetContextHashForEquivalence() const
{
#if DO_CHECK
	ensure(CachedHashForEquivalence != InvalidCachedHashForEquivalence && CachedContextHashForEquivalence != InvalidCachedHashForEquivalence);
	// making sure properties didn't change without updating CachedHashForEquivalence
	uint32 TestCachedHashForEquivalence, TestCachedContextHashForEquivalence;
	CalculateHashesForEquivalence(TestCachedHashForEquivalence, TestCachedContextHashForEquivalence);
	ensure(CachedHashForEquivalence == TestCachedHashForEquivalence && CachedContextHashForEquivalence == TestCachedContextHashForEquivalence);
#endif // DO_CHECK
	return CachedContextHashForEquivalence;
}

void FInteractionSearchContextBase::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Database);

	for (TObjectPtr<const UObject>& AnimContext : AnimContexts)
	{
		Collector.AddReferencedObject(AnimContext);
	}
}

void FInteractionSearchContextBase::Add(const UObject* AnimContext, const IPoseHistory* PoseHistory, const FRole Role, bool bWantsDisableCollisions)
{
	// trying to inefficiently call Add again after having calculated the hash via CalculateHashForEquivalence?
	ensure(CachedHashForEquivalence == InvalidCachedHashForEquivalence && CachedContextHashForEquivalence == InvalidCachedHashForEquivalence);

	ensure(AnimContext);
	// AnimContexts must be sorted to have deterministic searches across multiple frames
	check(AnimContexts.IsEmpty() || AnimContexts.Last() < AnimContext);

	AnimContexts.Add(AnimContext);
	
	if (ensure(PoseHistory))
	{
		PoseHistories.Add(PoseHistory->AsWeak());
	}
	else
	{
		PoseHistories.Add(nullptr);
	}

	Roles.Add(Role);

	bDisableCollisions |= bWantsDisableCollisions;
}

#if DO_CHECK
bool FInteractionSearchContextBase::CheckForConsistency() const
{
	if (Database == nullptr)
	{
		return false;
	}

	const int32 Num = AnimContexts.Num();
	if (Num < 1)
	{
		return false;
	}

	if (Num != PoseHistories.Num())
	{
		return false;
	}

	if (Num != Roles.Num())
	{
		return false;
	}

	for (int32 IndexA = 0; IndexA < Num; ++IndexA)
	{
		if (AnimContexts[IndexA] == nullptr)
		{
			return false;
		}

		for (int32 IndexB = IndexA + 1; IndexB < Num; ++IndexB)
		{
			if (AnimContexts[IndexA] == AnimContexts[IndexB])
			{
				return false;
			}
		}
	}

	for (int32 IndexA = 1; IndexA < Num; ++IndexA)
	{
		// AnimContexts must be sorted to have deterministic searches across multiple frames
		if (AnimContexts[IndexA - 1] >= AnimContexts[IndexA])
		{
			return false;
		}
	}

	for (int32 IndexA = 0; IndexA < Num; ++IndexA)
	{
		for (int32 IndexB = IndexA + 1; IndexB < Num; ++IndexB)
		{
			if (Roles[IndexA] == Roles[IndexB])
			{
				return false;
			}
		}
	}

	for (int32 IndexA = 0; IndexA < Num; ++IndexA)
	{
		if (!PoseHistories[IndexA].IsValid())
		{
			return false;
		}

		for (int32 IndexB = IndexA + 1; IndexB < Num; ++IndexB)
		{
			if (PoseHistories[IndexA] == PoseHistories[IndexB])
			{
				return false;
			}
		}
	}

	return true;
}

#endif // DO_CHECK

// FValidInteractionSearch
///////////////////////////////////////////////////////////
FValidInteractionSearch& FValidInteractionSearch::Init(const FInteractionSearchContextBase& SearchContextBase)
{
#if ENABLE_ANIM_DEBUG
	UE_MT_SCOPED_WRITE_ACCESS(ThreadSafeCounter);
#endif // ENABLE_ANIM_DEBUG

	static_cast<FInteractionSearchContextBase&>(*this) = SearchContextBase;
	return *this;
}

void FValidInteractionSearch::Update(const FInteractionSearchResult& InSearchResult)
{
#if ENABLE_ANIM_DEBUG
	UE_MT_SCOPED_WRITE_ACCESS(ThreadSafeCounter);
#endif // ENABLE_ANIM_DEBUG

	SearchResult = InSearchResult;
}

void FValidInteractionSearch::DoDisableCollisions()
{
#if ENABLE_ANIM_DEBUG
	UE_MT_SCOPED_WRITE_ACCESS(ThreadSafeCounter);
#endif // ENABLE_ANIM_DEBUG

	ensure(DisabledCollisions.IsEmpty());
	if (IsDisableCollisions())
	{
		FMemMark Mark(FMemStack::Get());
		TArray<AActor*, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> Actors;
		TArray<UPrimitiveComponent*, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> PrimitiveComponents;

		for (int32 AnimContextIndex = 0; AnimContextIndex < Num(); ++AnimContextIndex)
		{
			if (const UObject* AnimContext = GetAnimContext(AnimContextIndex))
			{
				AActor* Actor = const_cast<AActor*>(GetContextOwningActor(AnimContext, false));
				check(Actor);
				Actors.Add(Actor);
				PrimitiveComponents.Add(Cast<UPrimitiveComponent>(Actor->GetRootComponent()));
			}
		}

		for (int32 IndexA = 0; IndexA < Actors.Num(); ++IndexA)
		{
			UPrimitiveComponent* PrimitiveComponentA = PrimitiveComponents[IndexA];
			AActor* ActorA = Actors[IndexA];
			check(ActorA);

			for (int32 IndexB = IndexA + 1; IndexB < Actors.Num(); ++IndexB)
			{
				AActor* ActorB = Actors[IndexB];
				check(ActorB);

				UPrimitiveComponent* PrimitiveComponentB = PrimitiveComponents[IndexB];

				if (PrimitiveComponentA && !PrimitiveComponentA->GetMoveIgnoreActors().Contains(ActorB))
				{
					DisabledCollisions.Add({ ActorA, ActorB });
					PrimitiveComponentA->IgnoreActorWhenMoving(ActorB, true);
				}

				if (PrimitiveComponentB && !PrimitiveComponentB->GetMoveIgnoreActors().Contains(ActorA))
				{
					DisabledCollisions.Add({ ActorB, ActorA });
					PrimitiveComponentB->IgnoreActorWhenMoving(ActorA, true);
				}
			}
		}
	}
}

void FValidInteractionSearch::UndoDisableCollisions()
{
#if ENABLE_ANIM_DEBUG
	UE_MT_SCOPED_WRITE_ACCESS(ThreadSafeCounter);
#endif // ENABLE_ANIM_DEBUG

	for (const FDisabledCollisions::ElementType& DisabledCollision : DisabledCollisions)
	{
		if (AActor* ActorA = DisabledCollision.Key)
		{
			if (AActor* ActorB = DisabledCollision.Value)
			{
				if (UPrimitiveComponent* PrimitiveComponentA = Cast<UPrimitiveComponent>(ActorA->GetRootComponent()))
				{
					PrimitiveComponentA->IgnoreActorWhenMoving(ActorB, false);
				}
			}
		}
	}

	DisabledCollisions.Reset();
}

#if ENABLE_VISUAL_LOG
void FValidInteractionSearch::VLogContext(const FColor& Color) const
{
#if ENABLE_ANIM_DEBUG
	UE_MT_SCOPED_READ_ACCESS(ThreadSafeCounter);
#endif // ENABLE_ANIM_DEBUG

	if (FVisualLogger::IsRecording())
	{
		static const TCHAR* LogName = TEXT("PoseSearchInteraction");

		const int32 AnimContextsNum = Num();
		TArray<FVector, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> Locations;
		Locations.SetNum(AnimContextsNum);
		for (int32 Index = 0; Index < AnimContextsNum; ++Index)
		{
			if (const UObject* AnimContext = GetAnimContext(Index))
			{
				const AActor* ContextOwningActor = GetContextOwningActor(AnimContext);
				check(ContextOwningActor);
				// using ContextOwningActor location instead of skeletal mesh location since it's faster for UAF
				//Locations[Index] = GetContextLocation(AnimContext);
				Locations[Index] = ContextOwningActor->GetTransform().GetTranslation();
			}
		}

		for (int32 IndexA = 0; IndexA < AnimContextsNum; ++IndexA)
		{
			for (int32 IndexB = IndexA + 1; IndexB < AnimContextsNum; ++IndexB)
			{
				for (int32 IndexAll = 0; IndexAll < AnimContextsNum; ++IndexAll)
				{
					if (const UObject* AnimContext = GetAnimContext(IndexAll))
					{
						UE_VLOG_SEGMENT(AnimContext, LogName, Display, Locations[IndexA], Locations[IndexB], Color, TEXT(""));
					}
				}
			}
		}
	}
}
#endif // ENABLE_VISUAL_LOG

void FValidInteractionSearch::AddReferencedObjects(FReferenceCollector& Collector)
{
#if ENABLE_ANIM_DEBUG
	UE_MT_SCOPED_READ_ACCESS(ThreadSafeCounter);
#endif // ENABLE_ANIM_DEBUG

	FInteractionSearchContextBase::AddReferencedObjects(Collector);

	for (FDisabledCollision& DisabledCollision : DisabledCollisions)
	{
		Collector.AddReferencedObject(DisabledCollision.Key);
		Collector.AddReferencedObject(DisabledCollision.Value);
	}
}

// FInteractionSearchContext
///////////////////////////////////////////////////////////
void FValidInteractionSearches::Update(const UPoseSearchInteractionSubsystem* InteractionSubsystem)
{
#if ENABLE_ANIM_DEBUG
	UE_MT_SCOPED_WRITE_ACCESS(ThreadSafeCounter);
#endif //ENABLE_ANIM_DEBUG

	// avoiding ContextsToSearches.Reset() to preserve the array allocations in ContextsToSearches
	for (TPair<uint32, TArray<uint32>>& ContextToSearches : ContextsToSearches)
	{
		ContextToSearches.Value.Reset();
	}

	FMemMark Mark(FMemStack::Get());
	TMap<uint32, bool, TInlineSetAllocator<32, TMemStackSetAllocator<>>> ContinuingOrNewValidSearchContexts;
	for (const FInteractionIsland* Island : InteractionSubsystem->GetIslands())
	{
		if (Island->IsInitialized())
		{
			// analyzing ALL current tick interaction results
			for (const FInteractionSearchResult& SearchResult : Island->GetSearchResults())
			{
				const FInteractionSearchContext& SearchContext = Island->GetSearchContexts()[SearchResult.SearchIndex];
				const uint32 ValidSearchContextHash = SearchContext.GetHashForEquivalence();

				ContextsToSearches.FindOrAdd(SearchContext.GetContextHashForEquivalence()).Add(ValidSearchContextHash);

				if (FValidInteractionSearch* ContinuingValidInteractionSearch = Searches.Find(ValidSearchContextHash))
				{
					ContinuingOrNewValidSearchContexts.Add(ValidSearchContextHash) = true;
					ContinuingValidInteractionSearch->Update(SearchResult);
				}
				else
				{
					ContinuingOrNewValidSearchContexts.Add(ValidSearchContextHash) = false;
					Searches.Add(ValidSearchContextHash).Init(SearchContext).Update(SearchResult);
				}
			}
		}
	}

	// calling all the OnInteractionEnd first
	for (TMap<uint32, FValidInteractionSearch>::TIterator It = Searches.CreateIterator(); It; ++It)
	{
		if (!ContinuingOrNewValidSearchContexts.Find(It->Key))
		{
			OnInteractionEnd(It->Value, InteractionSubsystem);
			It.RemoveCurrent();
		}
	}

	// calling all the OnInteractionContinuing and OnInteractionStart after
	for (TMap<uint32, FValidInteractionSearch>::TIterator It = Searches.CreateIterator(); It; ++It)
	{
		const bool bIsContinuingValidSearchContexts = ContinuingOrNewValidSearchContexts.FindChecked(It->Key);
		if (bIsContinuingValidSearchContexts)
		{
			OnInteractionContinuing(It->Value, InteractionSubsystem);
		}
		else
		{
			OnInteractionStart(It->Value, InteractionSubsystem);
		}
	}
}

void FValidInteractionSearches::OnInteractionStart(FValidInteractionSearch& ValidInteractionSearch, const UPoseSearchInteractionSubsystem* InteractionSubsystem)
{
	check(InteractionSubsystem);

#if ENABLE_VISUAL_LOG
	ValidInteractionSearch.VLogContext(FColor::Blue);
#endif

	ValidInteractionSearch.DoDisableCollisions();

	InteractionSubsystem->OnInteractionStartedDelegate.Broadcast(ValidInteractionSearch);
}

void FValidInteractionSearches::OnInteractionContinuing(FValidInteractionSearch& ValidInteractionSearch, const UPoseSearchInteractionSubsystem* InteractionSubsystem)
{
	check(InteractionSubsystem);

#if ENABLE_VISUAL_LOG
	ValidInteractionSearch.VLogContext(FColor::Green);
#endif

	InteractionSubsystem->OnInteractionContinuingDelegate.Broadcast(ValidInteractionSearch);
}

void FValidInteractionSearches::OnInteractionEnd(FValidInteractionSearch& ValidInteractionSearch, const UPoseSearchInteractionSubsystem* InteractionSubsystem)
{
	check(InteractionSubsystem);

#if ENABLE_VISUAL_LOG
	ValidInteractionSearch.VLogContext(FColor::Black);
#endif // ENABLE_VISUAL_LOG

	ValidInteractionSearch.UndoDisableCollisions();

	InteractionSubsystem->OnInteractionEndedDelegate.Broadcast(ValidInteractionSearch);
}

const FValidInteractionSearch* FValidInteractionSearches::Get(const FInteractionSearchContext& SearchContext) const
{
#if ENABLE_ANIM_DEBUG
	UE_MT_SCOPED_READ_ACCESS(ThreadSafeCounter);
#endif // ENABLE_ANIM_DEBUG

	const uint32 HashForEquivalence = SearchContext.GetHashForEquivalence();
	return Searches.Find(HashForEquivalence);
}

bool FValidInteractionSearches::IsThereAnyValidSearchWithTheSameContexts(const FInteractionSearchContext& SearchContext) const
{
#if ENABLE_ANIM_DEBUG
	UE_MT_SCOPED_READ_ACCESS(ThreadSafeCounter);
#endif // ENABLE_ANIM_DEBUG

	const uint32 ContextHashForEquivalence = SearchContext.GetContextHashForEquivalence();
	if (const TArray<uint32>* HashesForEquivalence = ContextsToSearches.Find(ContextHashForEquivalence))
	{
		return !HashesForEquivalence->IsEmpty();
	}
	return false;
}

void FValidInteractionSearches::AddReferencedObjects(FReferenceCollector& Collector)
{
#if ENABLE_ANIM_DEBUG
	UE_MT_SCOPED_READ_ACCESS(ThreadSafeCounter);
#endif // ENABLE_ANIM_DEBUG

	for (TPair<uint32, FValidInteractionSearch>& SearchPair : Searches)
	{
		SearchPair.Value.AddReferencedObjects(Collector);
	}
}

// FInteractionSearchContext
///////////////////////////////////////////////////////////
void FInteractionSearchContext::Add(const UObject* AnimContext, const IPoseHistory* PoseHistory
	, const FRole Role, bool bWantsDisableCollisions, int32 TickPriority
#if ENABLE_ANIM_DEBUG
	, float InBroadPhaseRadius, float InBroadPhaseRadiusIncrementOnInteraction
#endif // ENABLE_ANIM_DEBUG
	)
{
	FInteractionSearchContextBase::Add(AnimContext, PoseHistory, Role, bWantsDisableCollisions);
	
	TickPriorities.Add(TickPriority);

#if ENABLE_ANIM_DEBUG
	DebugBroadPhaseRadius.Add(InBroadPhaseRadius);
	BroadPhaseRadiusIncrementOnInteraction.Add(InBroadPhaseRadiusIncrementOnInteraction);
#endif // ENABLE_ANIM_DEBUG
}

FPoseSearchContinuingProperties FInteractionSearchContext::GetContinuingProperties() const
{
	FPoseSearchContinuingProperties ContinuingProperties;
	ContinuingProperties.PlayingAsset = PlayingAsset;
	ContinuingProperties.PlayingAssetAccumulatedTime = PlayingAssetAccumulatedTime;
	ContinuingProperties.bIsPlayingAssetMirrored = bIsPlayingAssetMirrored;
	ContinuingProperties.PlayingAssetBlendParameters = PlayingAssetBlendParameters;
	ContinuingProperties.InterruptMode = InterruptMode;
	ContinuingProperties.bIsContinuingInteraction = bIsContinuingInteraction;
	ContinuingProperties.bIsContinuingContextInteraction = bIsContinuingContextInteraction;
	return ContinuingProperties;
}

#if ENABLE_ANIM_DEBUG
float FInteractionSearchContext::GetDebugBroadPhaseRadius(int32 Index) const
{
	if (bIsContinuingInteraction)
	{
		return DebugBroadPhaseRadius[Index] + BroadPhaseRadiusIncrementOnInteraction[Index];
	}
	
	return DebugBroadPhaseRadius[Index];
}
#endif // ENABLE_ANIM_DEBUG

void FInteractionSearchContext::SetContinuingProperties(float InPlayingAssetAccumulatedTime, const UObject* InPlayingAsset,
	bool bInIsPlayingAssetMirrored, const FVector& InPlayingAssetBlendParameters, EPoseSearchInterruptMode InInterruptMode)
{
	PlayingAssetAccumulatedTime = InPlayingAssetAccumulatedTime;
	PlayingAsset = InPlayingAsset;
	bIsPlayingAssetMirrored = bInIsPlayingAssetMirrored;
	PlayingAssetBlendParameters = InPlayingAssetBlendParameters;
	InterruptMode = InInterruptMode;
}

void FInteractionSearchContext::AddReferencedObjects(FReferenceCollector& Collector)
{
	FInteractionSearchContextBase::AddReferencedObjects(Collector);
	Collector.AddReferencedObject(PlayingAsset);
}

// FInteractionSearchResult
///////////////////////////////////////////////////////////
bool FInteractionSearchResult::operator==(const FInteractionSearchResult& Other) const
{
	// not checking SearchIndex, nor ActorRootTransforms, nor ActorRootBoneTransforms for equality
	return static_cast<const FSearchResult&>(*this) == static_cast<const FSearchResult&>(Other);
}

// FIslandPreTickFunction
///////////////////////////////////////////////////////////
void FInteractionIsland::FPreTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	// Called before any skeletal mesh component tick, when there aren't animation jobs flying. No need to FScopeLock Lock(&Mutex);
	// generating trajectories before running any of the skeletal mesh component ticks
	check(Island);

	if (Island->HasTickDependencies())
	{
		CheckInteractionThreadSafety(Island);

		for (const FInteractionSearchContext& SearchContext : Island->SearchContexts)
		{
			for (int32 Index = 0; Index < SearchContext.Num(); ++Index)
			{
				if (const UObject* AnimContext = SearchContext.GetAnimContext(Index))
				{
					if (const IPoseHistory* PoseHistory = SearchContext.GetPoseHistory(Index))
					{
						// since FInteractionIsland has a tick dependency with the USkeletalMeshComponent it's safe modify the IPoseHistory
						const_cast<IPoseHistory*>(PoseHistory)->GenerateTrajectory(AnimContext, DeltaTime);
					}
				}
			}
		}

#if ENABLE_ANIM_DEBUG
		if (Island->bPreTickFunctionExecuted)
		{
			// @todo: need to figure out why when creating a new island FPreTickFunction gets called twice (it's not a real issue rather than a performance hit)
			//		  use GVarPoseSearchInteractionCacheIslands = false to debug the issue (it destroys the islands every frame)
			UE_LOGF(LogPoseSearch, Log, "FInteractionIsland::FPreTickFunction::ExecuteTick, called twice before UPoseSearchInteractionSubsystem::Tick!");
		}
		else
		{
			if (Island->bPostTickFunctionExecuted)
			{
				UE_LOGF(LogPoseSearch, Error, "FInteractionIsland::FPreTickFunction::ExecuteTick, FPostTickFunction::ExecuteTick alreay run?!");
				Island->LogTickDependencies();
			}

			if (Island->bClosingTickFunctionExecuted)
			{
				UE_LOGF(LogPoseSearch, Error, "FInteractionIsland::FPreTickFunction::ExecuteTick, FClosingTickFunction::ExecuteTick alreay run?!");
				Island->LogTickDependencies();
			}

			Island->bPreTickFunctionExecuted = true;
		}
#endif // ENABLE_ANIM_DEBUG
	}
}

// FPostTickFunction
///////////////////////////////////////////////////////////
void FInteractionIsland::FPostTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	check(Island);

#if ENABLE_ANIM_DEBUG
	if (Island->HasTickDependencies())
	{
		CheckInteractionThreadSafety(Island);

		if (Island->bPostTickFunctionExecuted)
		{
			// @todo: need to figure out why when creating a new island FPostTickFunction gets called twice (it's not a real issue rather than a performance hit)
			//		  use GVarPoseSearchInteractionCacheIslands = false to debug the issue (it destroys the islands every frame)
			UE_LOGF(LogPoseSearch, Log, "FInteractionIsland::FPostTickFunction::ExecuteTick, called twice before UPoseSearchInteractionSubsystem::Tick!");
		}
		else
		{
			if (!Island->bPreTickFunctionExecuted)
			{
				UE_LOGF(LogPoseSearch, Error, "FInteractionIsland::FPostTickFunction::ExecuteTick, FPreTickFunction::ExecuteTick didn't run!");
				Island->LogTickDependencies();
			}

			if (Island->bClosingTickFunctionExecuted)
			{
				UE_LOGF(LogPoseSearch, Error, "FInteractionIsland::FPostTickFunction::ExecuteTick, FClosingTickFunction::ExecuteTick alreay run?!");
				Island->LogTickDependencies();
			}

			Island->bPostTickFunctionExecuted = true;
		}
	}
#endif // ENABLE_ANIM_DEBUG
}

// FClosingTickFunction
///////////////////////////////////////////////////////////
void FInteractionIsland::FClosingTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	check(Island);

	if (Island->HasTickDependencies())
	{
		CheckInteractionThreadSafety(Island);
		Island->UpdateConstraints(DeltaTime);

#if ENABLE_ANIM_DEBUG
		if (Island->bClosingTickFunctionExecuted)
		{
			// @todo: need to figure out why when creating a new island FClosingTickFunction gets called twice (it's not a real issue rather than a performance hit)
			//		  use GVarPoseSearchInteractionCacheIslands = false to debug the issue (it destroys the islands every frame)
			UE_LOGF(LogPoseSearch, Log, "FInteractionIsland::FClosingTickFunction::ExecuteTick, called twice before UPoseSearchInteractionSubsystem::Tick!");
		}
		else
		{
			if (!Island->bPreTickFunctionExecuted)
			{
				UE_LOGF(LogPoseSearch, Error, "FInteractionIsland::FClosingTickFunction::ExecuteTick, FPreTickFunction::ExecuteTick didn't run!");
				Island->LogTickDependencies();
			}

			if (!Island->bPostTickFunctionExecuted)
			{
				UE_LOGF(LogPoseSearch, Error, "FInteractionIsland::FClosingTickFunction::ExecuteTick, FPostTickFunction::ExecuteTick didn't run!");
				Island->LogTickDependencies();
			}

			Island->bClosingTickFunctionExecuted = true;
		}
#endif // ENABLE_ANIM_DEBUG
	}
}

// FInteractionIsland
///////////////////////////////////////////////////////////
FInteractionIsland::FInteractionIsland(ULevel* Level, UPoseSearchInteractionSubsystem* Subsystem)
{
	PreTickFunction.bAllowTickBatching = true;
	PreTickFunction.bRunOnAnyThread = true;
	PreTickFunction.Island = this;
	PreTickFunction.RegisterTickFunction(Level);

	PostTickFunction.bAllowTickBatching = true;
	PostTickFunction.bRunOnAnyThread = true;
	PostTickFunction.Island = this;
	PostTickFunction.RegisterTickFunction(Level);

	ClosingTickFunction.bAllowTickBatching = true;
	ClosingTickFunction.bRunOnAnyThread = true;
	ClosingTickFunction.Island = this;
	ClosingTickFunction.RegisterTickFunction(Level);

	InteractionSubsystem = Subsystem;
}

FInteractionIsland::~FInteractionIsland()
{
	Uninitialize(false);

	PreTickFunction.UnRegisterTickFunction();
	PostTickFunction.UnRegisterTickFunction();
	ClosingTickFunction.UnRegisterTickFunction();

	InteractionSubsystem = nullptr;
}

IInteractionIslandDependency* FInteractionIsland::FindCustomDependency(UActorComponent* InTickComponent)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	const int32 NumFeatures = ModularFeatures.GetModularFeatureImplementationCount(IInteractionIslandDependency::FeatureName);

	// Add pre-tick function dependencies
	for (int32 FeatureIndex = 0; FeatureIndex < NumFeatures; ++FeatureIndex)
	{
		if (IInteractionIslandDependency* IslandDependency = static_cast<IInteractionIslandDependency*>(ModularFeatures.GetModularFeatureImplementation(IInteractionIslandDependency::FeatureName, FeatureIndex)))
		{
			if (IslandDependency->CanMakeDependency(nullptr, InTickComponent))
			{
				return IslandDependency;
			}
		}
	}
	return nullptr;
}

void FInteractionIsland::AddTickDependencies(UActorComponent* TickActorComponent, bool bInIsMainActor)
{
	check(TickActorComponent);

	AActor* TickActor = TickActorComponent->GetOwner();
	check(TickActor);
			
	if (IInteractionIslandDependency* IslandDependency = FindCustomDependency(TickActorComponent))
	{
		if (const FTickFunction* TickActorComponentTickFunction = IslandDependency->FindTickFunction(TickActorComponent))
		{
			if (bInIsMainActor)
			{
				// PostTickFunction and PreTickFunction prerequisites should be empty since we haven't add the main actor tick function yet
				check(PostTickFunction.GetPrerequisites().IsEmpty());
				check(PreTickFunction.GetPrerequisites().IsEmpty());

				// adding to PreTickFunction all the tick dependencies TickActorComponent has (excluding the TickActor->PrimaryActorTick),
				// so it runs after all the tick dependencies of ALL the TickActorComponents in this FInteractionIsland
				for (const FTickPrerequisite& TickActorComponentPrerequisite : TickActorComponentTickFunction->GetPrerequisites())
				{
					check(TickActorComponentPrerequisite.PrerequisiteTickFunction);

					// making sure we're not adding the component nor the actor tick functions to the PreTickFunction
					check(TickActorComponentPrerequisite.PrerequisiteTickFunction != &TickActorComponent->PrimaryComponentTick);
					if (TickActorComponentPrerequisite.PrerequisiteTickFunction != &TickActor->PrimaryActorTick)
					{
						AddPrerequisite(PreTickFunction, TickActorComponentPrerequisite.PrerequisiteObject.Get(), *TickActorComponentPrerequisite.PrerequisiteTickFunction);
					}
				}

				// adding to TickActorComponent and owner actor TickActor dependencies to the PreTickFunction
				IslandDependency->AddSubsequent(InteractionSubsystem, PreTickFunction, TickActorComponent);
				AddPrerequisite(TickActor->PrimaryActorTick, InteractionSubsystem, PreTickFunction);

				// add PostTickFunction dependencies to TickActorComponent and owner actor
				IslandDependency->AddPrerequisite(InteractionSubsystem, PostTickFunction, TickActorComponent);
				AddPrerequisite(PostTickFunction, TickActor, TickActor->PrimaryActorTick);

				check(!bHasTickDependencies);
				bHasTickDependencies = true;
			}
			else
			{
				if (TickActor->CanEverTick())
				{
					// PostTickFunction should contain only the tick function to the main actor's ones (component and owner) if actor ticks
					check(PostTickFunction.GetPrerequisites().Num() == 2);
				}

				// adding to PreTickFunction all the tick dependencies TickActorComponent has (excluding the TickActor->PrimaryActorTick),
				// so it runs after all the tick dependencies of ALL the TickActorComponents in this FInteractionIsland
				// BUT excluding the main actor tick fuctions that are in PostTickFunction.GetPrerequisites()
				for (const FTickPrerequisite& TickActorComponentPrerequisite : TickActorComponentTickFunction->GetPrerequisites())
				{
					if (TickActorComponentPrerequisite.PrerequisiteTickFunction != &TickActor->PrimaryActorTick &&
						!PostTickFunction.GetPrerequisites().ContainsByPredicate([&TickActorComponentPrerequisite](const FTickPrerequisite& MainActorTickPrerequisite)
						{
							return TickActorComponentPrerequisite.PrerequisiteTickFunction == MainActorTickPrerequisite.Get();
						}))
					{
						check(TickActorComponentPrerequisite.PrerequisiteTickFunction);
						AddPrerequisite(PreTickFunction, TickActorComponentPrerequisite.PrerequisiteObject.Get(), *TickActorComponentPrerequisite.PrerequisiteTickFunction);
					}
				}

				// add PostTickFunction dependencies
				IslandDependency->AddSubsequent(InteractionSubsystem, PostTickFunction, TickActorComponent);
				AddPrerequisite(TickActor->PrimaryActorTick, InteractionSubsystem, PostTickFunction);

				// add ClosingTickFunction dependencies to TickActorComponent and owner actor
				IslandDependency->AddPrerequisite(InteractionSubsystem, ClosingTickFunction, TickActorComponent);
				AddPrerequisite(ClosingTickFunction, TickActor, TickActor->PrimaryActorTick);

				check(bHasTickDependencies);
			}

			// adding dependency between actor and UAF TickActorComponent so CBP runs before UAF parallel execution jobs
			IslandDependency->AddSubsequent(InteractionSubsystem, TickActor->PrimaryActorTick, TickActorComponent);
		}
		else
		{
			UE_LOGF(LogPoseSearch, Error, "FInteractionIsland::AddTickDependencies, error while retrieving the tick function for %ls", *TickActorComponent->GetName());
		}
	}
	else
	{
		if (bInIsMainActor)
		{
			// PostTickFunction and PreTickFunction prerequisites should be empty since we haven't add the main actor tick function yet
			check(PostTickFunction.GetPrerequisites().IsEmpty());
			check(PreTickFunction.GetPrerequisites().IsEmpty());

			// adding to PreTickFunction all the tick dependencies TickActorComponent has (excluding the TickActor->PrimaryActorTick),
			// so it runs after all the tick dependencies of ALL the TickActorComponents in this FInteractionIsland
			for (FTickPrerequisite& TickActorComponentPrerequisite : TickActorComponent->PrimaryComponentTick.GetPrerequisites())
			{
				check(TickActorComponentPrerequisite.PrerequisiteTickFunction);

				// making sure we're not adding the component nor the actor tick functions to the PreTickFunction
				check(TickActorComponentPrerequisite.PrerequisiteTickFunction != &TickActorComponent->PrimaryComponentTick);
				if (TickActorComponentPrerequisite.PrerequisiteTickFunction != &TickActor->PrimaryActorTick)
				{
					AddPrerequisite(PreTickFunction, TickActorComponentPrerequisite.PrerequisiteObject.Get(), *TickActorComponentPrerequisite.PrerequisiteTickFunction);
				}
			}

			// adding to TickActorComponent and owner actor TickActor dependencies to the PreTickFunction
			AddPrerequisite(TickActorComponent->PrimaryComponentTick, InteractionSubsystem, PreTickFunction);
			AddPrerequisite(TickActor->PrimaryActorTick, InteractionSubsystem, PreTickFunction);

			// add PostTickFunction function dependencies to TickActorComponent and owner actor
			AddPrerequisite(PostTickFunction, TickActorComponent, TickActorComponent->PrimaryComponentTick);
			AddPrerequisite(PostTickFunction, TickActor, TickActor->PrimaryActorTick);

			check(!bHasTickDependencies);
			bHasTickDependencies = true;
		}
		else
		{
			if (TickActor->CanEverTick())
			{
				// PostTickFunction should contain only the tick function to the main actor's ones (component and owner) if actor ticks
				check(PostTickFunction.GetPrerequisites().Num() == 2);
			}

			// adding to PreTickFunction all the tick dependencies TickActorComponent has (excluding the TickActor->PrimaryActorTick),
			// so it runs after all the tick dependencies of ALL the TickActorComponents in this FInteractionIsland
			// BUT excluding the main actor tick fuctions that are in PostTickFunction.GetPrerequisites()
			for (FTickPrerequisite& TickActorComponentPrerequisite : TickActorComponent->PrimaryComponentTick.GetPrerequisites())
			{
				if (TickActorComponentPrerequisite.PrerequisiteTickFunction != &TickActor->PrimaryActorTick &&
					!PostTickFunction.GetPrerequisites().ContainsByPredicate([&TickActorComponentPrerequisite](const FTickPrerequisite& MainActorTickPrerequisite)
					{
						return TickActorComponentPrerequisite.PrerequisiteTickFunction == MainActorTickPrerequisite.Get();
					}))
				{
					AddPrerequisite(PreTickFunction, TickActorComponentPrerequisite.PrerequisiteObject.Get(), *TickActorComponentPrerequisite.PrerequisiteTickFunction);
				}
			}

			// add PostTickFunction dependencies
			AddPrerequisite(TickActorComponent->PrimaryComponentTick, InteractionSubsystem, PostTickFunction);
			AddPrerequisite(TickActor->PrimaryActorTick, InteractionSubsystem, PostTickFunction);

			// add ClosingTickFunction dependencies to TickActorComponent and owner actor
			AddPrerequisite(ClosingTickFunction, TickActorComponent, TickActorComponent->PrimaryComponentTick);
			AddPrerequisite(ClosingTickFunction, TickActor, TickActor->PrimaryActorTick);

			check(bHasTickDependencies);
		}

		// adding prerequisite between actor and skeletal mesh component so CBP runs before ABP parallel execution jobs
		AddPrerequisite(TickActorComponent->PrimaryComponentTick, InteractionSubsystem, TickActor->PrimaryActorTick);

		// @todo: find a better way to do this!
		// adding prerequisite to the ClosingTickFunction to components having a tick dependency on TickActorComponent, so they can run AFTER the ClosingTickFunction
		for (UActorComponent* DependantTickActorComponent : TickActor->GetComponents())
		{
			if (DependantTickActorComponent)
			{
				if (DependantTickActorComponent->PrimaryComponentTick.GetPrerequisites().Contains(FTickPrerequisite(TickActorComponent, TickActorComponent->PrimaryComponentTick)))
				{
					PostClosingTickActorComponents.AddUnique(DependantTickActorComponent);
					AddPrerequisite(DependantTickActorComponent->PrimaryComponentTick, InteractionSubsystem, ClosingTickFunction);
				}
			}
		}
	}

#if ENABLE_ANIM_DEBUG
	if (GVarPoseSearchInteractionDiagnoseTickDependencies)
	{
		TSet<FTickFunction*> VisitedTickFunctions;
		bool bIsPreTickFunctionValid = ValidateTickDependenciesCycles(PreTickFunction, VisitedTickFunctions);
		check(VisitedTickFunctions.IsEmpty());
		bool bIsPostTickFunctionValid = ValidateTickDependenciesCycles(PostTickFunction, VisitedTickFunctions);
		check(VisitedTickFunctions.IsEmpty());
		bool bIsClosingTickFunctionValid = ValidateTickDependenciesCycles(ClosingTickFunction, VisitedTickFunctions);
		check(VisitedTickFunctions.IsEmpty());
		if (!bIsPreTickFunctionValid || !bIsPostTickFunctionValid || !bIsClosingTickFunctionValid)
		{
			// if this validation triggers here, FInteractionIsland is not respectint the already present dependencies, creating cycles
			UE_LOGF(LogPoseSearch, Error, "============== FInteractionIsland::AddTickDependencies ValidateTickDependencies failed! Analyze the log and tune the FPoseSearchInteractionAvailability::TickPriority ==============");
			LogTickDependencies();
		}
	}
#endif // ENABLE_ANIM_DEBUG
}

#if ENABLE_ANIM_DEBUG
void FInteractionIsland::ValidateTickDependenciesLeaks() const
{
	bool bAnyError = false;
	for (const TObjectPtr<UActorComponent>& TickActorComponentPtr : TickActorComponents)
	{
		if (const UActorComponent* TickActorComponent = TickActorComponentPtr.Get())
		{
			const AActor* TickActor = TickActorComponent->GetOwner();
			check(TickActor && TickActor->CanEverTick());

			// checking the TickActor
			for (const FTickPrerequisite& TickPrerequisite : TickActor->PrimaryActorTick.GetPrerequisites())
			{
				if (TickPrerequisite.PrerequisiteTickFunction == &PreTickFunction)
				{
					UE_LOGF(LogPoseSearch, Error, "============== FInteractionIsland::ValidateTickDependenciesLeaks failed! PreTickFunction leaked in %ls ==============", *TickActor->GetName());
					bAnyError = true;
				}

				if (TickPrerequisite.PrerequisiteTickFunction == &PostTickFunction)
				{
					UE_LOGF(LogPoseSearch, Error, "============== FInteractionIsland::ValidateTickDependenciesLeaks failed! PostTickFunction leaked in %ls ==============", *TickActor->GetName());
					bAnyError = true;
				}

				if (TickPrerequisite.PrerequisiteTickFunction == &ClosingTickFunction)
				{
					UE_LOGF(LogPoseSearch, Error, "============== FInteractionIsland::ValidateTickDependenciesLeaks failed! ClosingTickFunction leaked in %ls ==============", *TickActor->GetName());
					bAnyError = true;
				}
			}

			// checking the TickActorComponent
			for (const FTickPrerequisite& TickPrerequisite : TickActorComponent->PrimaryComponentTick.GetPrerequisites())
			{
				if (TickPrerequisite.PrerequisiteTickFunction == &PreTickFunction)
				{
					UE_LOGF(LogPoseSearch, Error, "============== FInteractionIsland::ValidateTickDependenciesLeaks failed! PreTickFunction leaked in %ls ==============", *TickActorComponent->GetName());
					bAnyError = true;
				}

				if (TickPrerequisite.PrerequisiteTickFunction == &PostTickFunction)
				{
					UE_LOGF(LogPoseSearch, Error, "============== FInteractionIsland::ValidateTickDependenciesLeaks failed! PostTickFunction leaked in %ls ==============", *TickActorComponent->GetName());
					bAnyError = true;
				}

				if (TickPrerequisite.PrerequisiteTickFunction == &ClosingTickFunction)
				{
					UE_LOGF(LogPoseSearch, Error, "============== FInteractionIsland::ValidateTickDependenciesLeaks failed! ClosingTickFunction leaked in %ls ==============", *TickActorComponent->GetName());
					bAnyError = true;
				}
			}
		}
	}

	if (bAnyError)
	{
		LogTickDependencies();
	}
}
#endif // ENABLE_ANIM_DEBUG

void FInteractionIsland::RemoveTickDependencies(bool bValidateTickDependencies)
{
	// Called by UPoseSearchInteractionSubsystem::Tick when there aren't animation jobs flying.
	check(IsInGameThread());

	check(TickActorComponents.Num() == IslandAnimContexts.Num());

	if (!bHasTickDependencies)
	{
#if ENABLE_ANIM_DEBUG
		if (bValidateTickDependencies && (bPreTickFunctionExecuted || bPostTickFunctionExecuted || bClosingTickFunctionExecuted))
		{
			if (bPreTickFunctionExecuted)
			{
				UE_LOGF(LogPoseSearch, Error, "FInteractionIsland::RemoveTickDependencies, unexpected FPreTickFunction::ExecuteTick run!");
			}
			else if (bPostTickFunctionExecuted)
			{
				UE_LOGF(LogPoseSearch, Error, "FInteractionIsland::RemoveTickDependencies, unexpected FPostTickFunction::ExecuteTick run!");
			}
			else
			{
				UE_LOGF(LogPoseSearch, Error, "FInteractionIsland::RemoveTickDependencies, unexpected FClosingTickFunction::ExecuteTick run!");
			}
				
			LogTickDependencies();
		}
#endif // ENABLE_ANIM_DEBUG
	}
	else
	{
#if ENABLE_ANIM_DEBUG
		if (bValidateTickDependencies && (!bPreTickFunctionExecuted || !bPostTickFunctionExecuted || !bClosingTickFunctionExecuted))
		{
			if (!bPreTickFunctionExecuted)
			{
				UE_LOGF(LogPoseSearch, Error, "FInteractionIsland::RemoveTickDependencies, expected FPreTickFunction::ExecuteTick didn't run!");
			}
			else if (!bPostTickFunctionExecuted)
			{
				UE_LOGF(LogPoseSearch, Error, "FInteractionIsland::RemoveTickDependencies, expected FPostTickFunction::ExecuteTick didn't run!");
			}
			else
			{
				UE_LOGF(LogPoseSearch, Error, "FInteractionIsland::RemoveTickDependencies, expected FClosingTickFunction::ExecuteTick didn't run!");
			}
			
			LogTickDependencies();
		}
#endif // ENABLE_ANIM_DEBUG

		// [undo] adding to PreTickFunction all the tick dependencies TickActorComponent has, so it runs after all the tick dependencies of ALL the TickActorComponents in this FInteractionIsland
		while (!PreTickFunction.GetPrerequisites().IsEmpty())
		{
			const FTickPrerequisite& PreTickFunctionPrerequisite = PreTickFunction.GetPrerequisites().Last();
			PreTickFunction.RemovePrerequisite(PreTickFunctionPrerequisite.PrerequisiteObject.Get(), *PreTickFunctionPrerequisite.PrerequisiteTickFunction);
		}

		bool bMainActor = true;
		for (TObjectPtr<UActorComponent>& TickActorComponentPtr : TickActorComponents)
		{
			if (UActorComponent* TickActorComponent = TickActorComponentPtr.Get())
			{
				AActor* TickActor = TickActorComponent->GetOwner();
				check(TickActor);

				if(IInteractionIslandDependency* IslandDependency = FindCustomDependency(TickActorComponent))
				{
					if (bMainActor)
					{
						// [undo] adding to TickActorComponent and owner actor TickActor dependencies to the PreTickFunction
						IslandDependency->RemoveSubsequent(InteractionSubsystem, PreTickFunction, TickActorComponent);
						TickActor->PrimaryActorTick.RemovePrerequisite(InteractionSubsystem, PreTickFunction);
						
						// [undo] add PostTickFunction dependencies to TickActorComponent and owner actor
						IslandDependency->RemovePrerequisite(InteractionSubsystem, PostTickFunction, TickActorComponent);
						PostTickFunction.RemovePrerequisite(TickActor, TickActor->PrimaryActorTick);
					}
					else
					{
						// [undo] add PostTickFunction dependencies
						IslandDependency->RemoveSubsequent(InteractionSubsystem, PostTickFunction, TickActorComponent);
						TickActor->PrimaryActorTick.RemovePrerequisite(InteractionSubsystem, PostTickFunction);
						
						// [undo] add ClosingTickFunction dependencies to TickActorComponent and owner actor
						IslandDependency->RemovePrerequisite(InteractionSubsystem, ClosingTickFunction, TickActorComponent);
						ClosingTickFunction.RemovePrerequisite(TickActor, TickActor->PrimaryActorTick);
					}

					// [undo] adding dependency between actor and UAF TickActorComponent so CBP runs before UAF parallel execution jobs
					IslandDependency->RemoveSubsequent(InteractionSubsystem, TickActor->PrimaryActorTick, TickActorComponent);
				}
				else
				{
					if (bMainActor)
					{
						// [undo] adding to TickActorComponent and owner actor TickActor dependencies to the PreTickFunction
						TickActorComponent->PrimaryComponentTick.RemovePrerequisite(InteractionSubsystem, PreTickFunction);
						TickActor->PrimaryActorTick.RemovePrerequisite(InteractionSubsystem, PreTickFunction);

						// [undo] add PostTickFunction function dependencies to TickActorComponent and owner actor
						PostTickFunction.RemovePrerequisite(TickActorComponent, TickActorComponent->PrimaryComponentTick);
						PostTickFunction.RemovePrerequisite(TickActor, TickActor->PrimaryActorTick);
					}
					else
					{
						// [undo] add PostTickFunction dependencies
						TickActorComponent->PrimaryComponentTick.RemovePrerequisite(InteractionSubsystem, PostTickFunction);
						TickActor->PrimaryActorTick.RemovePrerequisite(InteractionSubsystem, PostTickFunction);
						
						// [undo] add ClosingTickFunction dependencies to TickActorComponent and owner actor
						ClosingTickFunction.RemovePrerequisite(TickActorComponent, TickActorComponent->PrimaryComponentTick);
						ClosingTickFunction.RemovePrerequisite(TickActor, TickActor->PrimaryActorTick);
					}

					// [undo] adding prerequisite between actor and skeletal mesh component so CBP runs before ABP parallel execution jobs
					TickActorComponent->PrimaryComponentTick.RemovePrerequisite(InteractionSubsystem, TickActor->PrimaryActorTick);
				}
			}
			bMainActor = false;
		}

		// [undo] adding prerequisite to the ClosingTickFunction to components having a tick dependency on TickActorComponent, so they can run AFTER the ClosingTickFunction
		for (TObjectPtr<UActorComponent>& PostClosingTickActorComponent : PostClosingTickActorComponents)
		{
			if (PostClosingTickActorComponent)
			{
				PostClosingTickActorComponent->PrimaryComponentTick.RemovePrerequisite(InteractionSubsystem, ClosingTickFunction);
			}
		}

#if ENABLE_ANIM_DEBUG
		if (GVarPoseSearchInteractionDiagnoseTickDependencies)
		{
			ValidateTickDependenciesLeaks();
		}
#endif // ENABLE_ANIM_DEBUG

		if (!PreTickFunction.GetPrerequisites().IsEmpty())
		{
			UE_LOGF(LogPoseSearch, Warning, "FInteractionIsland::RemoveTickDependencies, unable to cleanly remove all the PreTickFunction prerequisites! (This could have happened if an actor got garbage collected or a BP got recompiled while PIE was paused)");
			PreTickFunction.GetPrerequisites().Reset();
		}

		if (!PostTickFunction.GetPrerequisites().IsEmpty())
		{
			UE_LOGF(LogPoseSearch, Warning, "FInteractionIsland::RemoveTickDependencies, unable to cleanly remove all the PostTickFunction prerequisites! (This could have happened if an actor got garbage collected or a BP got recompiled while PIE was paused)");
			PostTickFunction.GetPrerequisites().Reset();
		}

		if (!ClosingTickFunction.GetPrerequisites().IsEmpty())
		{
			UE_LOGF(LogPoseSearch, Warning, "FInteractionIsland::RemoveTickDependencies, unable to cleanly remove all the ClosingTickFunction prerequisites! (This could have happened if an actor got garbage collected or a BP got recompiled while PIE was paused)");
			ClosingTickFunction.GetPrerequisites().Reset();
		}

		PostClosingTickActorComponents.Reset();
		bHasTickDependencies = false;
	}

#if ENABLE_ANIM_DEBUG
	bPreTickFunctionExecuted = false;
	bPostTickFunctionExecuted = false;
	bClosingTickFunctionExecuted = false;
#endif // ENABLE_ANIM_DEBUG
}

void FInteractionIsland::InjectToActor(const UObject* AnimContext, bool bAddTickDependencies)
{
	check(IsInGameThread());

	// Called by UPoseSearchInteractionSubsystem::Tick when there aren't animation jobs flying. No need to FScopeLock Lock(&Mutex);
	if (AnimContext)
	{
#if ENABLE_ANIM_DEBUG
		if (bPreTickFunctionExecuted || bPostTickFunctionExecuted || bClosingTickFunctionExecuted)
		{
			if (bPreTickFunctionExecuted)
			{
				UE_LOGF(LogPoseSearch, Error, "FInteractionIsland::InjectToActor, unexpected FPreTickFunction::ExecuteTick run!");
			}
			else if (bPostTickFunctionExecuted)
			{
				UE_LOGF(LogPoseSearch, Error, "FInteractionIsland::InjectToActor, unexpected FPostTickFunction::ExecuteTick run!");
			}
			else
			{
				UE_LOGF(LogPoseSearch, Error, "FInteractionIsland::InjectToActor, unexpected FClosingTickFunction::ExecuteTick run!");
			}
				
			LogTickDependencies();
		}
#endif // ENABLE_ANIM_DEBUG

		if (UActorComponent* TickActorComponent = FindComponentForTickDependencies(AnimContext))
		{
			const bool bIsMainActor = IslandAnimContexts.IsEmpty();

			//	tick order: 
			//		ALL TickActorComponents prerequisites (ultimately we're looking to have UCharacterMovementComponent or UCharacterMoverComponent ticked) ->
			//			Island.PreTickFunction ->
			//				first injected TickActor AND TickActorComponent (USkeletalMeshComponent, or UAnimNextComponent) to have CBP running before and ABP / UAF on the same actor ->
			//					Island.PostTickFunction ->
			//						other TickActor(s) before TickActorComponent(s)
			TickActorComponents.AddUnique(TickActorComponent);
			IslandAnimContexts.AddUnique(AnimContext);

			// making sure that if we add a unique TickActorComponent, we add as well a unique PostTickComponent
			// (so we can remove them later on in a consistent fashion)
			check(TickActorComponents.Num() == IslandAnimContexts.Num());

			if (bAddTickDependencies)
			{
				AddTickDependencies(TickActorComponent, bIsMainActor);
			}
			else
			{
				check(!bHasTickDependencies);
			}
		}
	}
}

void FInteractionIsland::AddSearchContext(const FInteractionSearchContext& SearchContext)
{
#if DO_CHECK
	check(SearchContext.CheckForConsistency());
#endif
	check(IsInGameThread());
	SearchContexts.Add(SearchContext);
}

void FInteractionIsland::Uninitialize(bool bValidateTickDependencies)
{
#if ENABLE_ANIM_DEBUG
	if (GVarPoseSearchInteractionDiagnoseTickDependencies)
	{
		TSet<FTickFunction*> VisitedTickFunctions;
		bool bIsPreTickFunctionValid = ValidateTickDependenciesCycles(PreTickFunction, VisitedTickFunctions);
		check(VisitedTickFunctions.IsEmpty());
		bool bIsPostTickFunctionValid = ValidateTickDependenciesCycles(PostTickFunction, VisitedTickFunctions);
		check(VisitedTickFunctions.IsEmpty());
		bool bIsClosingTickFunctionValid = ValidateTickDependenciesCycles(ClosingTickFunction, VisitedTickFunctions);
		check(VisitedTickFunctions.IsEmpty());
		if (!bIsPreTickFunctionValid || !bIsPostTickFunctionValid || !bIsClosingTickFunctionValid)
		{
			// if this validation triggers here, some additional tick dependency outside FInteractionIsland has been injected witout respecting the already present dependencies, creating cycles
			UE_LOGF(LogPoseSearch, Error, "============== FInteractionIsland::Uninitialize ValidateTickDependencies failed! ==============");
			LogTickDependencies();
		}
	}
#endif // ENABLE_ANIM_DEBUG

	RemoveTickDependencies(bValidateTickDependencies);

	if (IsInitialized())
	{
		check(PostClosingTickActorComponents.IsEmpty());

		TickActorComponents.Reset();
		IslandAnimContexts.Reset();

		SearchContexts.Reset();
		SearchResults.Reset();
		bSearchPerfomed = false;
	}
	else
	{
		check(TickActorComponents.IsEmpty() && PostClosingTickActorComponents.IsEmpty() && IslandAnimContexts.IsEmpty() && SearchContexts.IsEmpty() && SearchResults.IsEmpty() && !bSearchPerfomed);
	}
}

bool FInteractionIsland::HasTickDependencies() const
{
	return bHasTickDependencies;
}

bool FInteractionIsland::IsInitialized() const
{
	return !SearchContexts.IsEmpty();
}

const UObject* FInteractionIsland::GetMainAnimContext() const
{
	return !IslandAnimContexts.IsEmpty() ? IslandAnimContexts[0].Get() : nullptr;
}

const AActor* FInteractionIsland::GetMainActor() const
{
	if (IsInitialized())
	{
		return GetContextOwningActor(GetMainAnimContext(), false);
	}
	return nullptr;
}

#if ENABLE_ANIM_DEBUG
void FInteractionIsland::LogTickDependencies(const TConstArrayView<UActorComponent*> TickActorComponents, int32 InteractionIslandIndex)
{
	check(IsInGameThread());

	for (UActorComponent* TickActorComponent : TickActorComponents)
	{
		if (TickActorComponent)
		{
			UE_LOGF(LogPoseSearch, Log, "============== %ls (Island %d) ==============", *TickActorComponent->GetOwner()->GetName(), InteractionIslandIndex);
			if (IInteractionIslandDependency* IslandDependency = FindCustomDependency(TickActorComponent))
			{
				// AnimNextComponent case
				if (const FTickFunction* TickActorComponentTickFunction = IslandDependency->FindTickFunction(TickActorComponent))
				{
					LogTickFunction(*const_cast<FTickFunction*>(TickActorComponentTickFunction), ENamedThreads::GameThread, true, 1);
				}
				else
				{
					UE_LOGF(LogPoseSearch, Error, "FInteractionIsland::LogTickDependencies, error while retrieving the tick function for to %ls", *TickActorComponent->GetName());
				}
			}
			else
			{
				// SkeletalMeshComponent / AnimInstance case
				LogTickFunction(TickActorComponent->PrimaryComponentTick, ENamedThreads::GameThread, true, 1);
			}
		}
		else
		{
			UE_LOGF(LogPoseSearch, Log, "============== !!!Missing Actor!!! (Island %d) ==============", InteractionIslandIndex);
		}
	}
}

void FInteractionIsland::LogTickDependencies() const
{
	const int32 InteractionIslandIndex = InteractionSubsystem->GetInteractionIslands().IndexOfByKey(this);

	if (IsInGameThread())
	{
		LogTickDependencies(TickActorComponents, InteractionIslandIndex);
		LogTickDependencies(PostClosingTickActorComponents, InteractionIslandIndex);
	}
	else
	{
		TArray<UActorComponent*> TickActorComponentsCopy = TickActorComponents;
		TArray<UActorComponent*> PostClosingTickActorComponentsCopy = PostClosingTickActorComponents;
		FFunctionGraphTask::CreateAndDispatchWhenReady([TickActorComponentsCopy, PostClosingTickActorComponentsCopy, InteractionIslandIndex]()
		{
			LogTickDependencies(TickActorComponentsCopy, InteractionIslandIndex);
			LogTickDependencies(PostClosingTickActorComponentsCopy, InteractionIslandIndex);
		}, TStatId(), nullptr, ENamedThreads::GameThread);
	}
}
#endif // ENABLE_ANIM_DEBUG

void FInteractionIsland::AddReferencedObjects(FReferenceCollector& Collector)
{
	TSet<TObjectPtr<AActor>, DefaultKeyFuncs<AActor*>, TInlineSetAllocator<32>> OwningTickActors;

	for (TObjectPtr<UActorComponent>& TickActorComponent : TickActorComponents)
	{
		if (TickActorComponent)
		{
			Collector.AddReferencedObject(TickActorComponent);
			// since AddReferencedObject may kill PostClosingTickActorComponent when recompiling blue prints
			if (TickActorComponent)
			{
				OwningTickActors.Add(TickActorComponent->GetOwner());
			}
		}
	}

	for (TObjectPtr<UActorComponent>& PostClosingTickActorComponent : PostClosingTickActorComponents)
	{
		if (PostClosingTickActorComponent)
		{
			Collector.AddReferencedObject(PostClosingTickActorComponent);
			// since AddReferencedObject may kill PostClosingTickActorComponent when recompiling blue prints
			if (PostClosingTickActorComponent)
			{
				OwningTickActors.Add(PostClosingTickActorComponent->GetOwner());
			}
		}
	}
	
	for (TObjectPtr<AActor>& OwningTickActor : OwningTickActors)
	{
		if (OwningTickActor)
		{
			Collector.AddReferencedObject(OwningTickActor);
		}
	}

	for (TObjectPtr<const UObject>& IslandAnimContext : IslandAnimContexts)
	{
		if (IslandAnimContext)
		{
			Collector.AddReferencedObject(IslandAnimContext);
		}
	}

	for (FInteractionSearchContext& SearchContext : SearchContexts)
	{
		SearchContext.AddReferencedObjects(Collector);
	}
}

bool FInteractionIsland::DoSearch_AnyThread(const UObject* AnimContext, FPoseSearchBlueprintResult& Result)
{
	check(AnimContext);

	// searches are performed only on the MainAnimContext / MainActor
	if (!bSearchPerfomed && AnimContext == GetMainAnimContext())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_UPoseSearchInteractionInteractionIsland_Search);

		FMemMark Mark(FMemStack::Get());

		TArray<FSearchResult, TMemStackAllocator<>> PoseSearchResults;
		TArray<FChooserEvaluationContext, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> Contexts;

		// SearchContexts are modified only by UPoseSearchInteractionSubsystem::Tick and constant otherwise, so it's safe to access them in a threaded environment without locks
		PoseSearchResults.SetNum(SearchContexts.Num());
		for (int32 SearchIndex = 0; SearchIndex < SearchContexts.Num(); ++SearchIndex)
		{
			FInteractionSearchContext& InteractionSearchContext = SearchContexts[SearchIndex];
			const UPoseSearchDatabase* Database = InteractionSearchContext.GetDatabase();
			check(Database && Database->Schema);
			
			const int32 NumRoles = InteractionSearchContext.Num();
			Contexts.Reset();
			Contexts.SetNum(NumRoles);

			bool bIsSearchContextValid = true;
			FSearchContext SearchContext(0.f, FFloatInterval(0.f, 0.f), FPoseSearchEvent());
			for (int32 RoleIndex = 0; RoleIndex < NumRoles; ++RoleIndex)
			{
				const UObject* SearchContextAnimContext = InteractionSearchContext.GetAnimContext(RoleIndex);
				check(SearchContextAnimContext);

				Contexts[RoleIndex].AddObjectParam(const_cast<UObject*>(SearchContextAnimContext));

				if (const IPoseHistory* PoseHistory = InteractionSearchContext.GetPoseHistory(RoleIndex))
				{
					SearchContext.AddRole(InteractionSearchContext.GetRole(RoleIndex), &Contexts[RoleIndex], PoseHistory);
				}
				else
				{
					bIsSearchContextValid = false;
					break;
				}
			}

			if (bIsSearchContextValid)
			{
				const UObject* AssetsToSearch[] = { Database };
				// @todo: we could perform multiple UPoseSearchLibrary::MotionMatch in parallel!
				FSearchResults_Single MotionMatchSearchResults;
				UPoseSearchLibrary::MotionMatch(SearchContext, AssetsToSearch, InteractionSearchContext.GetContinuingProperties(), MotionMatchSearchResults);
				PoseSearchResults[SearchIndex] = MotionMatchSearchResults.GetBestResult();
			}
		}

		InitSearchResults(SearchResults, PoseSearchResults, SearchContexts);
		bSearchPerfomed = true;
	}

	return GetResult_AnyThread(AnimContext, Result);
}

const FInteractionSearchResult* FInteractionIsland::GetInteractionSearchResult_AnyThread(const UObject* AnimContext, bool bCompareOwningActors, FRole& OutRole) const
{
	check(AnimContext);

	const AActor* Actor = bCompareOwningActors ? GetContextOwningActor(AnimContext, false) : nullptr;

	// looking for AnimContext in SearchResults to fill up Result
	for (const FInteractionSearchResult& SearchResult : SearchResults)
	{
		const FInteractionSearchContext& SearchContext = SearchContexts[SearchResult.SearchIndex];
		for (int32 AnimContextIndex = 0; AnimContextIndex < SearchContext.Num(); ++AnimContextIndex)
		{
			if (bCompareOwningActors)
			{
				if (GetContextOwningActor(SearchContext.GetAnimContext(AnimContextIndex), false) == Actor)
				{
					// we found our AnimContext: we can stop searching
					OutRole = SearchContext.GetRole(AnimContextIndex);
					return &SearchResult;
				}
			}
			else
			{
				if (SearchContext.GetAnimContext(AnimContextIndex) == AnimContext)
				{
					// we found our AnimContext: we can stop searching
					OutRole = SearchContext.GetRole(AnimContextIndex);
					return &SearchResult;
				}
			}
		}
	}

	OutRole = DefaultRole;
	return nullptr;
}

bool FInteractionIsland::GetResult_AnyThread(const UObject* AnimContext, FPoseSearchBlueprintResult& Result, bool bCompareOwningActors)
{
	check(AnimContext);

	FRole AnimContextRole;
	if (const FInteractionSearchResult* SearchResult = GetInteractionSearchResult_AnyThread(AnimContext, bCompareOwningActors, AnimContextRole))
	{
		// @todo: perhaps add a custom Result.InitFrom(SearchResult, 1.f) for MM interactions
		const UPoseSearchDatabase* Database = SearchResult->Database.Get();
		check(Database);

		const FSearchIndexAsset* SearchIndexAsset = SearchResult->GetSearchIndexAsset();
		check(SearchIndexAsset);

		const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAssetBase = Database->GetDatabaseAnimationAsset(SearchIndexAsset->GetSourceAssetIdx());
		check(DatabaseAnimationAssetBase);

		Result.SelectedAnim = DatabaseAnimationAssetBase->GetAnimationAsset();
		Result.SelectedTime = SearchResult->GetAssetTime();
		Result.bIsContinuingPoseSearch = SearchResult->bIsContinuingPoseSearch;
		Result.bLoop = SearchIndexAsset->IsLooping();
		Result.bIsMirrored = SearchIndexAsset->IsMirrored();
		Result.BlendParameters = SearchIndexAsset->GetBlendParameters();
		Result.SelectedDatabase = Database;
		Result.SearchCost = SearchResult->PoseCost;
		Result.bIsInteraction = true;
		Result.Role = AnimContextRole;

		// @todo: figuring out the WantedPlayRate
		Result.WantedPlayRate = 1.f;
		//if (Future.Animation && Future.IntervalTime > 0.f)
		//{
		//	if (const UPoseSearchFeatureChannel_PermutationTime* PermutationTimeChannel = Database->Schema->FindFirstChannelOfType<UPoseSearchFeatureChannel_PermutationTime>())
		//	{
		//		const FSearchIndex& SearchIndex = Database->GetSearchIndex();
		//		if (!SearchIndex.IsValuesEmpty())
		//		{
		//			TConstArrayView<float> ResultData = Database->GetSearchIndex().GetPoseValues(SearchResult.PoseIdx);
		//			const float ActualIntervalTime = PermutationTimeChannel->GetPermutationTime(ResultData);
		//			ProviderResult.WantedPlayRate = ActualIntervalTime / Future.IntervalTime;
		//		}
		//	}
		//}

		const FInteractionSearchContext& SearchContext = SearchContexts[SearchResult->SearchIndex];
		if (const UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(Result.SelectedAnim))
		{
			const int32 NumRoles = MultiAnimAsset->GetNumRoles();
			Result.ActorRootTransforms.SetNum(NumRoles);
			Result.ActorRootBoneTransforms.SetNum(NumRoles);
			Result.AnimContexts.SetNum(NumRoles);

			const FRoleToIndex InteractionSearchContextRoleToIndex = MakeRoleToIndex(SearchContext.GetRoles());

			for (int32 MultiAnimAssetRoleIndex = 0; MultiAnimAssetRoleIndex < NumRoles; ++MultiAnimAssetRoleIndex)
			{
				const FRole MultiAnimAssetRole = MultiAnimAsset->GetRole(MultiAnimAssetRoleIndex);
				if (MultiAnimAssetRole == Result.Role)
				{
					Result.RoleIndex = MultiAnimAssetRoleIndex;
				}

				if (const int32* InteractionSearchContextRoleIndex = InteractionSearchContextRoleToIndex.Find(MultiAnimAssetRole))
				{
					Result.ActorRootTransforms[MultiAnimAssetRoleIndex] = SearchResult->ActorRootTransforms[*InteractionSearchContextRoleIndex];
					Result.ActorRootBoneTransforms[MultiAnimAssetRoleIndex] = SearchResult->ActorRootBoneTransforms[*InteractionSearchContextRoleIndex];
					Result.AnimContexts[MultiAnimAssetRoleIndex] = SearchContext.GetAnimContext(*InteractionSearchContextRoleIndex);
				}
				else
				{
					Result.ActorRootTransforms[MultiAnimAssetRoleIndex] = FTransform::Identity;
					Result.ActorRootBoneTransforms[MultiAnimAssetRoleIndex] = FTransform::Identity;
					Result.AnimContexts[MultiAnimAssetRoleIndex] = nullptr;
				}
			}
		}
		else
		{
			// @todo: should we support trivial "interactions" with only a character defined using some other assets rather then UMultiAnimAsset?
			check(Result.AnimContexts.Num() == 1);

			Result.ActorRootTransforms = SearchResult->ActorRootTransforms;
			Result.ActorRootBoneTransforms = SearchResult->ActorRootBoneTransforms;
					
			Result.AnimContexts.SetNum(1);
			Result.AnimContexts[0] = SearchContext.GetAnimContext(0);

			Result.RoleIndex = 0;
		}

		return true;
	}

	Result = FPoseSearchBlueprintResult();
	return false;
}

void FInteractionIsland::UpdateConstraints(float DeltaTime)
{
	check(HasTickDependencies());
	check(InteractionSubsystem);
		
	for (FInteractionSearchResult& SearchResult : SearchResults)
	{
		const FInteractionSearchContext& SearchContext = SearchContexts[SearchResult.SearchIndex];

		// making sure nobody already called UpdateConstraints or we didn't reset the constraints...
		ensure(SearchResult.Constraints.IsEmpty());
	
		const UPoseSearchDatabase* Database = SearchResult.Database.Get();
		if (ensure(Database))
		{
			const FSearchIndexAsset* SearchIndexAsset = SearchResult.GetSearchIndexAsset();
			if (ensure(SearchIndexAsset))
			{
				const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAssetBase = Database->GetDatabaseAnimationAsset(SearchIndexAsset->GetSourceAssetIdx());
				if (ensure(DatabaseAnimationAssetBase))
				{
					if (const UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(DatabaseAnimationAssetBase->GetAnimationAsset()))
					{
						// adding the cached previous constraints stored into the InteractionSubsystem::ValidInteractionSearches to SearchResult.Constraints for 
						// UPoseSearchInteractionLibrary::UpdateConstraints to be able to reason about ramping up or down the FPoseSearchConstraint::DesiredReach
						SearchResult.Constraints = InteractionSubsystem->GetConstraints(SearchContext);
						UPoseSearchInteractionLibrary::UpdateConstraints(SearchResult.Constraints, MultiAnimAsset, DeltaTime, SearchResult.GetAssetTime(), SearchIndexAsset->IsMirrored(), SearchIndexAsset->GetBlendParameters(), SearchContext.GetAnimContexts(), SearchContext.GetRoles());
					}
				}
			}
		}
	}
}

bool FInteractionIsland::GetConstraint(const UObject* AnimContext, FName SocketName, float& OutDesiredReach, FTransform& OutTransform, bool bCompareOwningActors) const
{
	FRole AnimContextRole;
	if (const FInteractionSearchResult* SearchResult = GetInteractionSearchResult_AnyThread(AnimContext, bCompareOwningActors, AnimContextRole))
	{
		for (const FPoseSearchConstraint& Constraint : SearchResult->Constraints)
		{
			if (const FTransform* SocketTransform = Constraint.GetSocketTransform(AnimContextRole, SocketName))
			{
				OutDesiredReach = Constraint.DesiredReach;
				OutTransform = *SocketTransform;
				return true;
			}
		}
	}

	OutDesiredReach = 0.f;
	OutTransform = FTransform::Identity;
	return false;
}

} // namespace UE::PoseSearch
