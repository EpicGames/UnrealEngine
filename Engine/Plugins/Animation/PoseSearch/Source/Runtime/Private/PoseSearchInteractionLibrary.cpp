// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchInteractionLibrary.h"
#include "Animation/AnimMontage.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/World.h"
#include "GameFramework/Pawn.h"
#include "PoseSearch/AnimNode_PoseSearchHistoryCollector.h"
#include "PoseSearch/MultiAnimAsset.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchInteractionSubsystem.h"
#include "PoseSearch/PoseSearchInteractionUtils.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchInteractionLibrary)

FPoseSearchBlueprintResult UPoseSearchInteractionLibrary::MotionMatchInteraction_Pure(TArray<FPoseSearchInteractionAvailability> Availabilities, const UObject* AnimContext, FName PoseHistoryName, bool bValidateResultAgainstAvailabilities)
{
	FPoseSearchBlueprintResult Result;
	MotionMatchInteraction(Result, Availabilities, AnimContext, PoseHistoryName, nullptr, bValidateResultAgainstAvailabilities);
	return Result;
}

FPoseSearchBlueprintResult UPoseSearchInteractionLibrary::MotionMatchInteraction(TArray<FPoseSearchInteractionAvailability> Availabilities, const UObject* AnimContext, FName PoseHistoryName, bool bValidateResultAgainstAvailabilities)
{
	FPoseSearchBlueprintResult Result;
	MotionMatchInteraction(Result, Availabilities, AnimContext, PoseHistoryName, nullptr, bValidateResultAgainstAvailabilities);
	return Result;
}

void UPoseSearchInteractionLibrary::MotionMatchInteraction(FPoseSearchBlueprintResult& Result, const TArrayView<const FPoseSearchInteractionAvailability> Availabilities, const UObject* AnimContext, FName PoseHistoryName, const UE::PoseSearch::IPoseHistory* PoseHistory, bool bValidateResultAgainstAvailabilities)
{
	if (UPoseSearchInteractionSubsystem* InteractionSubsystem = UPoseSearchInteractionSubsystem::GetSubsystem_AnyThread(AnimContext))
	{
		InteractionSubsystem->Query_AnyThread(Availabilities, AnimContext, Result, PoseHistoryName, PoseHistory, bValidateResultAgainstAvailabilities);
	}
	else
	{
		Result = FPoseSearchBlueprintResult();
	}
}

void UPoseSearchInteractionLibrary::MotionMatchInteraction(FPoseSearchBlueprintResult& InOutResult, const TArrayView<const FPoseSearchInteractionAvailability> Availabilities, const UObject* AnimContext, FName PoseHistoryName, const UE::PoseSearch::IPoseHistory* PoseHistory, bool bValidateResultAgainstAvailabilities, bool bKeepInteractionAlive, float BlendTime, float DeltaTime)
{
	using namespace UE::PoseSearch;
	
	if (!Availabilities.IsEmpty())
	{
		CheckInteractionThreadSafety(AnimContext);

		FPoseSearchBlueprintResult SearchResult;
		UPoseSearchInteractionLibrary::MotionMatchInteraction(SearchResult, Availabilities, AnimContext, PoseHistoryName, PoseHistory, bValidateResultAgainstAvailabilities);
		check(SearchResult.ActorRootTransforms.Num() == SearchResult.ActorRootBoneTransforms.Num());

		if (SearchResult.SelectedAnim)
		{
			InOutResult = SearchResult;
			return;
		}
	}

	// performing the regular single character motion matching search in case there's no MM interaction
	if (!InOutResult.bIsInteraction)
	{
		// do nothing
		return;
	}
	
	if (!bKeepInteractionAlive)
	{
		InOutResult = FPoseSearchBlueprintResult();
		return;
	}

	// checking if the kept alive interaction has reached the end of animation
	if (!InOutResult.SelectedDatabase || INDEX_NONE == InOutResult.SelectedDatabase->GetPoseIndex(InOutResult.SelectedAnim.Get(), InOutResult.SelectedTime, InOutResult.bIsMirrored, InOutResult.BlendParameters))
	{
		InOutResult = FPoseSearchBlueprintResult();
		return;
	}

	// letting the interaction animation run until its length minus blend time
	// (to avoid having to blend from a frozen animation that reached its end for the entire duration of the blend)
	const UAnimationAsset* AnimationAssetForRole = InOutResult.GetAnimationAssetForRole();
	if (!AnimationAssetForRole || InOutResult.SelectedTime >= (AnimationAssetForRole->GetPlayLength() - BlendTime))
	{
		InOutResult = FPoseSearchBlueprintResult();
		return;
	}

	// we're keeping alive only the animation part of the search result
	InOutResult.ActorRootTransforms.Reset();
	InOutResult.ActorRootBoneTransforms.Reset();
	InOutResult.AnimContexts.Reset();
	InOutResult.SelectedTime += DeltaTime;
}

void UPoseSearchInteractionLibrary::CalculateFullAlignedTransforms(const FPoseSearchBlueprintResult& Result, float TimeOffset, bool bWarpUsingRootBone, TArray<FTransform>& OutFullAlignedTransforms)
{
	if (!Result.AnimContexts.IsEmpty() && Result.SelectedAnim && Result.SelectedAnim->IsA<UMultiAnimAsset>())
	{
		OutFullAlignedTransforms.SetNum(Result.AnimContexts.Num());
		UE::PoseSearch::CalculateFullAlignedTransforms(Result, Result.SelectedTime, TimeOffset, bWarpUsingRootBone, OutFullAlignedTransforms);
	}
	else
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::CalculateFullAlignedTransforms failed!");
		OutFullAlignedTransforms.Reset();
	}
}

void UPoseSearchInteractionLibrary::CalculateFullAlignedTransform(const FPoseSearchBlueprintResult& Result, float TimeOffset, bool bWarpUsingRootBone, FTransform& OutFullAlignedTransform)
{
	if (Result.AnimContexts.IsValidIndex(Result.RoleIndex) && Result.SelectedAnim && Result.SelectedAnim->IsA<UMultiAnimAsset>())
	{
		TArray<FTransform, TInlineAllocator<UE::PoseSearch::PreallocatedRolesNum>> FullAlignedTransforms;
		FullAlignedTransforms.SetNum(Result.AnimContexts.Num());
		UE::PoseSearch::CalculateFullAlignedTransforms(Result, Result.SelectedTime, TimeOffset, bWarpUsingRootBone, FullAlignedTransforms);
		OutFullAlignedTransform = FullAlignedTransforms[Result.RoleIndex];
	}
	else
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::CalculateFullAlignedTransform failed!");
		OutFullAlignedTransform.SetIdentity();
	}
}

void UPoseSearchInteractionLibrary::CalculateNoAlignedTransform(const FPoseSearchBlueprintResult& Result, float TimeOffset, bool bWarpUsingRootBone, FTransform& OutNoAlignedTransform)
{
	using namespace UE::PoseSearch;

	if (!Result.ActorRootTransforms.IsValidIndex(Result.RoleIndex))
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::CalculateNoAlignedTransform failed!");
		OutNoAlignedTransform.SetIdentity();
		return;
	}

	const UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(Result.SelectedAnim);
	if (!MultiAnimAsset)
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::CalculateNoAlignedTransform failed!");
		OutNoAlignedTransform.SetIdentity();
		return;
	}

	const UAnimationAsset* AnimationAsset = MultiAnimAsset->GetAnimationAsset(Result.Role);
	if (!AnimationAsset)
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::CalculateNoAlignedTransform failed!");
		OutNoAlignedTransform.SetIdentity();
		return;
	}

	check(Result.ActorRootBoneTransforms.Num() == Result.ActorRootTransforms.Num());

	const FAnimationAssetSampler Sampler(AnimationAsset, FTransform::Identity, Result.BlendParameters);
	const FTransform SelectedTimeRootTransform = Sampler.ExtractRootTransform(Result.SelectedTime);
	const FTransform OffsettedTimeRootTransform = Sampler.ExtractRootTransform(Result.SelectedTime + TimeOffset);

	const FTransform DeltaRootTransform = OffsettedTimeRootTransform.GetRelativeTransform(SelectedTimeRootTransform);
	const FTransform ActorRootTransform = bWarpUsingRootBone ? Result.ActorRootBoneTransforms[Result.RoleIndex] * Result.ActorRootTransforms[Result.RoleIndex] : Result.ActorRootTransforms[Result.RoleIndex];

	OutNoAlignedTransform = DeltaRootTransform * ActorRootTransform;
}

FPoseSearchContinuingProperties UPoseSearchInteractionLibrary::GetMontageContinuingProperties(UAnimInstance* AnimInstance)
{
	FPoseSearchContinuingProperties ContinuingProperties;
	if (const FAnimMontageInstance* AnimMontageInstance = AnimInstance->GetActiveMontageInstance())
	{
		ContinuingProperties.PlayingAsset = AnimMontageInstance->Montage;
		ContinuingProperties.PlayingAssetAccumulatedTime = AnimMontageInstance->DeltaTimeRecord.GetPrevious();
	}
	return ContinuingProperties;
}

void UPoseSearchInteractionLibrary::GetMotionMatchInteractionConstraint(const UObject* AnimContext, FName SocketName, float& OutDesiredReach, FTransform& OutTransform, bool& OutIsValid, bool bCompareOwningActors)
{
	if (UPoseSearchInteractionSubsystem* InteractionSubsystem = UPoseSearchInteractionSubsystem::GetSubsystem_AnyThread(AnimContext))
	{
		OutIsValid = InteractionSubsystem->GetConstraint(AnimContext, SocketName, OutDesiredReach, OutTransform, bCompareOwningActors);
	}
	else
	{
		OutDesiredReach = 0.f;
		OutTransform = FTransform::Identity;
		OutIsValid = false;
	}
}

void UPoseSearchInteractionLibrary::MotionMatchMulti(
	TArray<FPoseSearchMotionMatchMultiQuery> MotionMatchMultiQueries,
	const FName PoseHistoryName,
	const FPoseSearchContinuingProperties ContinuingProperties,
	TArray<FPoseSearchBlueprintResult>& Results)
{
	using namespace UE::PoseSearch;

	Results.Reset();

	FMemMark Mark(FMemStack::Get());

	typedef TSet<FRole, DefaultKeyFuncs<FRole>, TInlineSetAllocator<PreallocatedRolesNum, TMemStackSetAllocator<>>> FRoleSet;
	typedef TSet<const UObject*, DefaultKeyFuncs<const UObject*>, TInlineSetAllocator<32, TMemStackSetAllocator<>>> FAnimContextSet;
	typedef TPair<const UObject*, FRole> FRoledAnimContext;

	TSet<const UPoseSearchDatabase*, DefaultKeyFuncs<const UPoseSearchDatabase*>, TInlineSetAllocator<32, TMemStackSetAllocator<>>> UniqueDatabases;
	FAnimContextSet UniqueAnimContexts;
	FRoleSet DatabaseRoles;
	FRoleSet UniqueRoles;

	TMap<const UObject*, const FPoseHistory*, TInlineSetAllocator<32, TMemStackSetAllocator<>>> AnimContextToPoseHistory;
	TArray<FRoledAnimContext> RoledAnimContexts;

	struct FSearch
	{
		const UPoseSearchDatabase* Database = nullptr;

		TArray<const IPoseHistory*, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> PoseHistories;
		TArray<const UObject*, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> AnimContexts;
		TArray<FRole, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> Roles;

		FSearchResult Result;
	};

	// analyzing input MotionMatchMultiQueries to compile RoledAnimContexts and AnimContextToPoseHistory for each database.
	// then generating Searches using all the valid permutations for RoledAnimContexts
	TArray<FSearch, TInlineAllocator<16, TMemStackAllocator<>>> Searches;
	for (const FPoseSearchMotionMatchMultiQuery& MotionMatchMultiQuery : MotionMatchMultiQueries)
	{
		RoledAnimContexts.Reset();

		if (!MotionMatchMultiQuery.Database)
		{
			UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::MotionMatchMulti - null MotionMatchMultiQuery.Database not supported");
			return;
		}

		bool bDatabaseAlreadyInSet = false;
		UniqueDatabases.Add(MotionMatchMultiQuery.Database, &bDatabaseAlreadyInSet);
		if (bDatabaseAlreadyInSet)
		{
			UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::MotionMatchMulti - found unsupported duplicated database '%ls' in MotionMatchMultiQuery", *MotionMatchMultiQuery.Database->GetName());
			return;
		}

		const TArray<FPoseSearchRoledSkeleton>& RoledSkeletons = MotionMatchMultiQuery.Database->Schema->GetRoledSkeletons();
		for (const FPoseSearchRoledSkeleton& RoledSkeleton : RoledSkeletons)
		{
			DatabaseRoles.Add(RoledSkeleton.Role);
		}

		UniqueAnimContexts.Reset();
		for (const FPoseSearchAnimContextRoles& AnimContextRoles : MotionMatchMultiQuery.AnimContextsRoles)
		{
			if (!AnimContextRoles.AnimContext)
			{
				UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::MotionMatchMulti - null AnimContextRoles.AnimContext not supported");
				return;
			}

			bool bAnimContextAlreadyInSet = false;
			UniqueAnimContexts.Add(AnimContextRoles.AnimContext, &bAnimContextAlreadyInSet);
			if (bAnimContextAlreadyInSet)
			{
				UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::MotionMatchMulti - found unsupported duplicated AnimContext '%ls' for database '%ls' in MotionMatchMultiQuery", *AnimContextRoles.AnimContext->GetName(), *MotionMatchMultiQuery.Database->GetName());
				return;
			}

			UAnimInstance* AnimInstance = Cast<UAnimInstance>(AnimContextRoles.AnimContext);
			if (!AnimInstance)
			{
				UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::MotionMatchMulti - provided AnimContext '%ls' is not an UAnimInstance. We currently support only UAnimInstance(s). UAF support is still WIP", *AnimContextRoles.AnimContext->GetName());
				return;
			}

			if (!AnimInstance->CurrentSkeleton)
			{
				UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::MotionMatchMulti - AnimInstance '%ls' CurrentSkeleton is null!", *AnimContextRoles.AnimContext->GetName());
				return;
			}

			const FPoseHistory*& PoseHistory = AnimContextToPoseHistory.Add(AnimContextRoles.AnimContext);
			if (!PoseHistory)
			{
				// @todo: improve this approach, perhaps promoting UAnimInstance::GetProxyOnAnyThread from protected to public
				// making sure there're no flying animation tasks when on game thread by calling GetProxyOnAnyThread so it's safe to access animinstance variables etc
				class UFinishFlyingAnimInstanceTasks : public UAnimInstance
				{
				public:
					static void Execute(UAnimInstance* AnimInstance)
					{
						static_cast<UFinishFlyingAnimInstanceTasks*>(AnimInstance)->GetProxyOnAnyThread<FAnimInstanceProxy>();
					}
				};
				UFinishFlyingAnimInstanceTasks::Execute(AnimInstance);

				if (const FAnimNode_PoseSearchHistoryCollector_Base* PoseHistoryNode = UPoseSearchLibrary::FindPoseHistoryNode(PoseHistoryName, AnimInstance))
				{
					PoseHistory = PoseHistoryNode->GetPoseHistoryPtr();
					if (!PoseHistory)
					{
						UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::MotionMatchMulti - Couldn't find a valid pose history in the pose history node '%ls' for AnimContext '%ls'", *PoseHistoryName.ToString(), *AnimContextRoles.AnimContext->GetName());
						return;
					}
				}
				else
				{
					UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::MotionMatchMulti - Couldn't find pose history node with name '%ls' for AnimContext '%ls'", *PoseHistoryName.ToString(), *AnimContextRoles.AnimContext->GetName());
					return;
				}
			}

			UniqueRoles.Reset();
			for (const FRole& Role : AnimContextRoles.Roles)
			{
				if (!DatabaseRoles.Contains(Role))
				{
					UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::MotionMatchMulti - found unsupported Role '%ls' requested by AnimContext '%ls' for database '%ls' in MotionMatchMultiQuery", *Role.ToString(), *AnimContextRoles.AnimContext->GetName(), *MotionMatchMultiQuery.Database->GetName());
					return;
				}

				bool bRoleAlreadyInSet = false;
				UniqueRoles.Add(Role, &bRoleAlreadyInSet);
				if (bRoleAlreadyInSet)
				{
					UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::MotionMatchMulti - found duplicated Role '%ls' for AnimContext '%ls' for database '%ls' in MotionMatchMultiQuery", *Role.ToString(), *AnimContextRoles.AnimContext->GetName(), *MotionMatchMultiQuery.Database->GetName());
					return;
				}

				RoledAnimContexts.Add({ AnimContextRoles.AnimContext, Role });
			}
		}

		// generating all the valid permutations for RoledAnimContexts to compile all the possible Searches. Using AnimContextToPoseHistory to map contexts to their pose history
		const int32 CombinationCardinality = RoledSkeletons.Num();
		GenerateCombinations(RoledAnimContexts.Num(), CombinationCardinality,
			// Combination is an array of indexes in RoledAnimContexts: 0 <= Combination[i] < RoledAnimContexts.Num()
			[&RoledAnimContexts, &Searches, &MotionMatchMultiQuery, &AnimContextToPoseHistory](const TConstArrayView<int32> Combination)
			{
				// CombinationCardinality represents the number of roles as well as the number interacting AnimContext(s) (ultimately number of Characters involved in the interaction)
				const int32 CombinationCardinality = Combination.Num();

				FRoleSet CombinationRoles;
				FAnimContextSet CombinationAnimContexts;
				for (int32 CombinationIndex = 0; CombinationIndex < CombinationCardinality; ++CombinationIndex)
				{
					const FRoledAnimContext& RoledAnimContext = RoledAnimContexts[Combination[CombinationIndex]];
					CombinationAnimContexts.Add(RoledAnimContext.Key);
					CombinationRoles.Add(RoledAnimContext.Value);
				}

				// do we have all the roles and are all the AnimContext(s) unique?
				// Trying to avoid matching one character with itself covering for multiple roles, or arrin a search without ALL the roles
				if (CombinationAnimContexts.Num() == CombinationCardinality && CombinationRoles.Num() == CombinationCardinality)
				{
					FSearch& Search = Searches.AddDefaulted_GetRef();
					Search.Database = MotionMatchMultiQuery.Database;

					for (int32 CombinationIndex = 0; CombinationIndex < CombinationCardinality; ++CombinationIndex)
					{
						const FRoledAnimContext& RoledAnimContext = RoledAnimContexts[Combination[CombinationIndex]];
						Search.AnimContexts.Add(RoledAnimContext.Key);
						Search.Roles.Add(RoledAnimContext.Value);
						Search.PoseHistories.Add(AnimContextToPoseHistory[RoledAnimContext.Key]);
					}
				}
			});
	}

	if (Searches.IsEmpty())
	{
		UE_LOGF(LogPoseSearch, Warning, "UPoseSearchInteractionLibrary::MotionMatchMulti - no searches to perform? double check your inputs");
		return;
	}

	// performing all MotionMatch Searches in parallel
	ParallelFor(Searches.Num(), [&Searches, &ContinuingProperties](const int32 SearchIndex)
		{
			FSearch& Search = Searches[SearchIndex];
			const TArrayView<const UObject*> AssetsToSearch = MakeArrayView((const UObject**)&Search.Database, 1);
			Search.Result = UPoseSearchLibrary::MotionMatch(Search.AnimContexts, Search.Roles, Search.PoseHistories, AssetsToSearch, ContinuingProperties, FPoseSearchFutureProperties(), FPoseSearchEvent());
		}, ParallelForFlags);

	// sorting the searches:
	// - highest number of roles (interacting characters) first  
	// - best score otherwise (lowest SearchA.Result.PoseCost)
	// - invalid searches last (to avoid invalid searches with more roles to win against valid searches with less roles)
	Searches.StableSort([](const FSearch& SearchA, const FSearch& SearchB)
		{
			const bool IsValidA = SearchA.Result.IsValid();
			const bool IsValidB = SearchB.Result.IsValid();

			if (IsValidA && !IsValidB)
			{
				return true;
			}

			if (!IsValidA && IsValidB)
			{
				return false;
			}

			if (!IsValidA && !IsValidB)
			{
				return true;
			}

			const int32 NumRolesA = SearchA.Roles.Num();
			const int32 NumRolesB = SearchB.Roles.Num();

			if (NumRolesA > NumRolesB)
			{
				return true;
			}

			if (NumRolesA < NumRolesB)
			{
				return false;
			}

			return SearchA.Result.PoseCost < SearchB.Result.PoseCost;
		});

	// populating Results by allocating AnimContexts starting by the best searches
	// @todo: we should add differnt policies, like minimizing the overall score of the searches, or others..
	FStackAssetSet VisitedAnimContexts;
	for (const FSearch& Search : Searches)
	{
		if (!Search.Result.IsValid())
		{
			// only invalid results from now on, since Searches is sorted. We can break the loop
			break;
		}

		if (VisitedAnimContexts.ContainsAnyOf(Search.AnimContexts))
		{
			// AnimContext has already been assigned to a different interaction. skipping this search
			continue;
		}

		const FSearchIndexAsset* SearchIndexAsset = Search.Result.GetSearchIndexAsset();
		if (!SearchIndexAsset)
		{
			UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::MotionMatchMulti - null search FSearchIndexAsset");
			continue;
		}

		const UPoseSearchDatabase* Database = Search.Result.Database.Get();
		check(Database);

		const FPoseSearchDatabaseAnimationAssetBase* DatabaseAsset = Database->GetDatabaseAnimationAsset(SearchIndexAsset->GetSourceAssetIdx());
		if (!ensure(DatabaseAsset))
		{
			UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::MotionMatchMulti - null search FPoseSearchDatabaseAnimationAssetBase");
			continue;
		}

		UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(DatabaseAsset->GetAnimationAsset());
		if (!ensure(MultiAnimAsset))
		{
			UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::MotionMatchMulti - null search UMultiAnimAsset");
			continue;
		}

		// populating VisitedAnimContexts to invalidate subsequent searches containing Search.AnimContexts
		VisitedAnimContexts.Append(Search.AnimContexts);

		const int32 NumRoles = MultiAnimAsset->GetNumRoles();

		// adding the first result for this search
		FPoseSearchBlueprintResult& FirstResult = Results.AddDefaulted_GetRef();

		FirstResult.SelectedAnim = MultiAnimAsset;
		FirstResult.SelectedTime = Search.Result.GetAssetTime();
		FirstResult.bIsContinuingPoseSearch = Search.Result.bIsContinuingPoseSearch;
		FirstResult.WantedPlayRate = 1.f;
		FirstResult.bLoop = SearchIndexAsset->IsLooping();
		FirstResult.bIsMirrored = SearchIndexAsset->IsMirrored();
		FirstResult.BlendParameters = SearchIndexAsset->GetBlendParameters();
		FirstResult.SelectedDatabase = Database;
		FirstResult.SearchCost = Search.Result.PoseCost;
		FirstResult.Role = Search.Roles[0];
		FirstResult.bIsInteraction = true;
		FirstResult.ActorRootTransforms.SetNum(NumRoles);
		FirstResult.ActorRootBoneTransforms.SetNum(NumRoles);
		FirstResult.AnimContexts.SetNum(NumRoles);
				
		const FRoleToIndex RoleToIndex = MakeRoleToIndex(Search.Roles);

		for (int32 MultiAnimAssetRoleIndex = 0; MultiAnimAssetRoleIndex < NumRoles; ++MultiAnimAssetRoleIndex)
		{
			const FRole MultiAnimAssetRole = MultiAnimAsset->GetRole(MultiAnimAssetRoleIndex);
			const int32 RoleIndex = RoleToIndex.FindChecked(MultiAnimAssetRole);
			const IPoseHistory* PoseHistory = Search.PoseHistories[RoleIndex];
			check(PoseHistory);

			const USkeleton* Skeleton = GetContextSkeleton(Search.AnimContexts[RoleIndex], false);

			PoseHistory->GetTransformAtTime(0.f, FirstResult.ActorRootTransforms[MultiAnimAssetRoleIndex], Skeleton, ComponentSpaceIndexType, WorldSpaceIndexType);
			PoseHistory->GetTransformAtTime(0.f, FirstResult.ActorRootBoneTransforms[MultiAnimAssetRoleIndex], Skeleton, RootBoneIndexType, ComponentSpaceIndexType);

			FirstResult.AnimContexts[MultiAnimAssetRoleIndex] = Search.AnimContexts[RoleIndex];
		}

		FirstResult.Role = MultiAnimAsset->GetRole(0);
		FirstResult.RoleIndex = 0;

		// adding the other results for the other roles for this search
		for (int32 MultiAnimAssetRoleIndex = 1; MultiAnimAssetRoleIndex < NumRoles; ++MultiAnimAssetRoleIndex)
		{
			FPoseSearchBlueprintResult& OtherResult = Results.AddDefaulted_GetRef();
			OtherResult = FirstResult;
			OtherResult.Role = MultiAnimAsset->GetRole(MultiAnimAssetRoleIndex);
			OtherResult.RoleIndex = MultiAnimAssetRoleIndex;
		}
	}
}

void UPoseSearchInteractionLibrary::UpdateConstraints(UPARAM(ref) TArray<FPoseSearchConstraint>& InOutConstraints, float DeltaTime, const UMultiAnimAsset* MultiAnimAsset, const TArray<UObject*> AnimContexts, float SelectedTime, bool bIsMirrored, FVector BlendParameters)
{
	using namespace UE::PoseSearch;

	const int32 NumRoles = AnimContexts.Num();
	TArray<FRole, TInlineAllocator<PreallocatedRolesNum>> Roles;
	Roles.SetNum(NumRoles);

	if (MultiAnimAsset)
	{
		if (AnimContexts.Num() != MultiAnimAsset->GetNumRoles())
		{
			UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::UpdateConstraints - AnimContexts.Num() != MultiAnimAsset->GetNumRoles()");
			return;
		}

		for (int32 RoleIndex = 0; RoleIndex < NumRoles; ++RoleIndex)
		{
			Roles[RoleIndex] = MultiAnimAsset->GetRole(RoleIndex);
		}
	}
	else
	{
		for (int32 RoleIndex = 0; RoleIndex < NumRoles; ++RoleIndex)
		{
			Roles[RoleIndex] = DefaultRole;
		}
	}

	TArray<TObjectPtr<const UObject>> AnimContextPtrs(AnimContexts);
	UPoseSearchInteractionLibrary::UpdateConstraints(InOutConstraints, MultiAnimAsset, DeltaTime, SelectedTime, bIsMirrored, BlendParameters, AnimContextPtrs, Roles);
}

void UPoseSearchInteractionLibrary::GetConstraint(const TArray<FPoseSearchConstraint>& Constraints, FName AnimContextRole, FName SocketName, float& OutDesiredReach, FTransform& OutTransform, bool& IsValid)
{
	for (const FPoseSearchConstraint& Constraint : Constraints)
	{
		if (const FTransform* SocketTransform = Constraint.GetSocketTransform(AnimContextRole, SocketName))
		{
			OutDesiredReach = Constraint.DesiredReach;
			OutTransform = *SocketTransform;
			IsValid = true;
			return;
		}
	}

	OutDesiredReach = 0.f;
	OutTransform = FTransform::Identity;
	IsValid = false;
}

// right now we suppose that ALL the constraints are from the same MultiAnimAsset
void UPoseSearchInteractionLibrary::UpdateConstraints(TArray<FPoseSearchConstraint>& InOutConstraints, const UMultiAnimAsset* MultiAnimAsset, float DeltaTime, float AssetTime, 
	bool bIsMirrored, const FVector& BlendParameters, const TConstArrayView<TObjectPtr<const UObject>> AnimContexts, const TConstArrayView<UE::PoseSearch::FRole> Roles)
{
	using namespace UE::PoseSearch;

	if (!MultiAnimAsset)
	{
		InOutConstraints.Reset();
		return;
	}

	const int32 NumAnimContexts = AnimContexts.Num();
	if (!ensure(NumAnimContexts == Roles.Num()))
	{
		InOutConstraints.Reset();
		return;
	}

	FMemMark Mark(FMemStack::Get());

	TArray<bool, TInlineAllocator<16, TMemStackAllocator<>>> VisitedConstraints;
	VisitedConstraints.SetNumZeroed(InOutConstraints.Num());

	TArray<FPoseSearchConstraint, TInlineAllocator<16, TMemStackAllocator<>>> NewConstraints;

	// extracting the contraints for this SearchContext from MultiAnimAsset
	FAnimNotifyContext PreAllocatedNotifyContext;
	for (int32 AnimContextIndex = 0; AnimContextIndex < NumAnimContexts; ++AnimContextIndex)
	{
		const FRole& Role = Roles[AnimContextIndex];
		const UAnimationAsset* Animation = MultiAnimAsset->GetAnimationAsset(Role);

		if (!Animation)
		{
			UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::UpdateConstraints aborting because of invalid animation asset for Role '%ls' in MultiAnimAsset '%ls'", *Role.ToString(), *MultiAnimAsset->GetName());
			InOutConstraints.Reset();
			return;
		}

		// initializing FAnimationAssetSampler only for extracting AnimNotifyStates
		FAnimationAssetSampler Sampler(Animation, FTransform::Identity, FVector::ZeroVector, FAnimationAssetSampler::DefaultRootTransformSamplingRate, false, false);
		Sampler.ExtractAnimNotifyEvents(AssetTime, PreAllocatedNotifyContext, [AssetTime, DeltaTime, &InOutConstraints, &VisitedConstraints, &NewConstraints](const FAnimNotifyEvent* AnimNotifyEvent)
			{
				if (const UAnimNotifyState_PoseSearchConstraint* ConstraintNotifyState = Cast<const UAnimNotifyState_PoseSearchConstraint>(AnimNotifyEvent->NotifyStateClass))
				{
					// subtracting ConstraintNotifyState->CoolDownTime from the validity of this UAnimNotifyState_PoseSearchConstraint
					const float NotifyEndTime = AnimNotifyEvent->GetEndTriggerTime();
					if (AssetTime <= (NotifyEndTime - ConstraintNotifyState->CoolDownTime))
					{
						const int32 ConstraintIndex = InOutConstraints.IndexOfByPredicate([ConstraintNotifyState](const FPoseSearchConstraint& Other)
							{
								return Other.HasTheSameRoledSockets(*ConstraintNotifyState);
							});

						if (ConstraintIndex != INDEX_NONE)
						{
							VisitedConstraints[ConstraintIndex] = true;
							FPoseSearchConstraint& OldConstraint = InOutConstraints[ConstraintIndex];

							// @todo: for now we just ignore conflicting constraints, but we should probably blend OldConstraint into NewConstraint :/
							//        perhaps keeping the min or RampUpTime, to make the constraint as following the fastest one to get into full DesiredReach, and the max CoolDownTime to fade off with the slowest rate
							//        not really sure what to do about TranslationWeight and RotationWeight.. maybe slowly blending them to the newer one?
							if (OldConstraint.RampUpTime > UE_KINDA_SMALL_NUMBER)
							{
								OldConstraint.DesiredReach = FMath::Min(1.f, OldConstraint.DesiredReach + (DeltaTime / OldConstraint.RampUpTime));
							}
							else
							{
								OldConstraint.DesiredReach = 1.f;
							}
						}
						else
						{
							// @todo: for now we also ignore duplicate constraints, and perhaps we should use a similar strategy of blending the constraints
							if (!NewConstraints.FindByPredicate([ConstraintNotifyState](const FPoseSearchConstraint& Other)
							{
								return Other.HasTheSameRoledSockets(*ConstraintNotifyState);
							}))
							{
								FPoseSearchConstraint& NewConstraint = NewConstraints.AddDefaulted_GetRef();
								NewConstraint.Initialize(ConstraintNotifyState);
								if (NewConstraint.RampUpTime <= UE_KINDA_SMALL_NUMBER)
								{
									NewConstraint.DesiredReach = 1.f;
								}
							}
						}
					}
				}
				return true;
			});
	}

	// evaluating non visited constraints to either update their DesiredReach or remove them from InOutConstraints
	// (visited constraints have their DesiredReach already updated)
	for (int32 OldConstraintIndex = InOutConstraints.Num() - 1; OldConstraintIndex >= 0; --OldConstraintIndex)
	{
		if (!VisitedConstraints[OldConstraintIndex])
		{
			FPoseSearchConstraint& OldConstraint = InOutConstraints[OldConstraintIndex];
			const float DesiredReach = OldConstraint.CoolDownTime > UE_KINDA_SMALL_NUMBER ? OldConstraint.DesiredReach - (DeltaTime / OldConstraint.CoolDownTime) : 0.f;

			if (DesiredReach > UE_KINDA_SMALL_NUMBER)
			{
				// keeping OldConstraint alive since it still has some reach
				OldConstraint.DesiredReach = DesiredReach;
			}
			else
			{
				// no reach. constraints is no longer necessary
				InOutConstraints.RemoveAt(OldConstraintIndex);
			}
		}
	}

	// adding all the new constraints
	InOutConstraints.Append(NewConstraints);

	// process all the collected constraints
	if (!InOutConstraints.IsEmpty())
	{
		TArray<FTransform, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> ComponentToWorldTransforms;
		TArray<TConstArrayView<FTransform>, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> AnimContextsComponentSpaceTransforms;
		TArray<FBoneContainer, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> BoneContainers;
		TArray<FCompactPose, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> Poses;
		TArray<FCSPose<FCompactPose>, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> ComponentSpacePoses;
		TArray<const USkeletalMeshComponent*, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> SkeletalMeshComponents;

		ComponentToWorldTransforms.SetNum(NumAnimContexts);
		AnimContextsComponentSpaceTransforms.SetNum(NumAnimContexts);
		BoneContainers.SetNum(NumAnimContexts);
		Poses.SetNum(NumAnimContexts);
		ComponentSpacePoses.SetNum(NumAnimContexts);
		SkeletalMeshComponents.SetNum(NumAnimContexts);

		FRoleToIndex RoleToIndex;
		for (int32 AnimContextIndex = 0; AnimContextIndex < NumAnimContexts; ++AnimContextIndex)
		{
			const UObject* AnimContext = AnimContexts[AnimContextIndex];
			if (ensure(AnimContext))
			{
				const USkeletalMeshComponent* SkeletalMeshComponent = GetContextSkeletalMeshComponent(AnimContext, false);
				if (ensure(SkeletalMeshComponent))
				{
					SkeletalMeshComponents[AnimContextIndex] = SkeletalMeshComponent;

					// @todo: do we need this logic about URO etc? if we're executing this is because we update the mesh, so we probably don't need any interpolation...
					const bool bUROInSync = SkeletalMeshComponent->ShouldUseUpdateRateOptimizations() && SkeletalMeshComponent->AnimUpdateRateParams != nullptr;
					const bool bUsingExternalInterpolation = SkeletalMeshComponent->IsUsingExternalInterpolation();

					const TArray<FTransform>& CachedComponentSpaceTransforms = SkeletalMeshComponent->GetCachedComponentSpaceTransforms();
					const bool bArraySizesMatch = CachedComponentSpaceTransforms.Num() == SkeletalMeshComponent->GetComponentSpaceTransforms().Num();

					ComponentToWorldTransforms[AnimContextIndex] = SkeletalMeshComponent->GetComponentTransform();
					AnimContextsComponentSpaceTransforms[AnimContextIndex] = (bUROInSync || bUsingExternalInterpolation) && bArraySizesMatch ? CachedComponentSpaceTransforms : SkeletalMeshComponent->GetComponentSpaceTransforms();

					// @todo: check FAnimNode_CopyPoseFromMesh::PreUpdate to see how to GetAnimCurves and GetCustomAttributes
					const FRole& Role = Roles[AnimContextIndex];
					RoleToIndex.Add(Role) = AnimContextIndex;

					const UAnimationAsset* Animation = MultiAnimAsset->GetAnimationAsset(Role);
					if (ensure(Animation))
					{
						const FTransform Origin = MultiAnimAsset->GetOrigin(Role);

						FAnimationAssetSampler Sampler(Animation, Origin, BlendParameters, FAnimationAssetSampler::DefaultRootTransformSamplingRate, true, false);
						const FTransform RootTransform = Sampler.ExtractRootTransform(AssetTime);

						// @todo add support for mirroring if needed
						//if (MirrorDataTables.IsValidIndex(ItemIndex) && MirrorDataTables[ItemIndex])
						//{
						//	const FMirrorDataCache MirrorDataCache(MirrorDataTables[ItemIndex]);
						//	AssetRootBoneTransforms[ItemIndex] = MirrorDataCache.MirrorTransform(AssetRootBoneTransforms[ItemIndex]);
						//}

						// array containing the bone index of the root bone (0)
						const USkeleton* Skeleton = Animation->GetSkeleton();
						if (ensure(Skeleton && SkeletalMeshComponent->GetSkeletalMeshAsset()->GetSkeleton() == Skeleton))
						{
							const FReferenceSkeleton& ReferenceSkeleton = Skeleton->GetReferenceSkeleton();

							// @todo: refine this BoneIndices / cache bone containers
							const int32 NumBones = ReferenceSkeleton.GetNum();
							TArray<uint16, TMemStackAllocator<>> BoneIndices;
							BoneIndices.SetNumUninitialized(NumBones);
							for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
							{
								BoneIndices[BoneIndex] = BoneIndex;
							}

							// extracting the pose, containing only the root bone from the Sampler 
							BoneContainers[AnimContextIndex].InitializeTo(BoneIndices, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *Skeleton);
							Poses[AnimContextIndex].SetBoneContainer(&BoneContainers[AnimContextIndex]);
							Sampler.ExtractPose(AssetTime, Poses[AnimContextIndex]);

							// making sure the animation root bone transform is Identity, so we can confuse the root with the root BONE transform and preserve performances!
							FTransform& RootBoneTransform = Poses[AnimContextIndex][FCompactPoseBoneIndex(0)];
							if (!RootBoneTransform.Equals(FTransform::Identity))
							{
								const FVector Pos = RootBoneTransform.GetTranslation();
								const FRotator Rot(RootBoneTransform.GetRotation());
								UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::UpdateConstraints unsupported non identity root bone in %ls at time %f Pos(%f, %f, %f), Rot(%f, %f, %f)", *Animation->GetName(), AssetTime, Pos.X, Pos.Y, Pos.Z, Rot.Pitch, Rot.Yaw, Rot.Roll);
							}
							RootBoneTransform = Sampler.ExtractRootTransform(AssetTime);
							ComponentSpacePoses[AnimContextIndex].InitPose(Poses[AnimContextIndex]);

#if ENABLE_VISUAL_LOG
							if (FVisualLogger::IsRecording())
							{
								static const TCHAR* LogName = TEXT("PoseSearchInteraction_Constraints");

								for (int32 BoneIndex = 0; BoneIndex < ReferenceSkeleton.GetNum(); ++BoneIndex)
								{
									const int32 ParentBoneIndex = ReferenceSkeleton.GetParentIndex(BoneIndex);
									if (ParentBoneIndex != INDEX_NONE)
									{
										const FVector CurrentBoneWorldLocation = ComponentToWorldTransforms[AnimContextIndex].TransformPosition(AnimContextsComponentSpaceTransforms[AnimContextIndex][BoneIndex].GetTranslation());
										const FVector CurrentParentBoneWorldLocation = ComponentToWorldTransforms[AnimContextIndex].TransformPosition(AnimContextsComponentSpaceTransforms[AnimContextIndex][ParentBoneIndex].GetTranslation());

										const FVector AnimBoneWorldLocation = ComponentSpacePoses[AnimContextIndex].GetComponentSpaceTransform(FCompactPoseBoneIndex(BoneIndex)).GetTranslation();
										const FVector AnimParentBoneWorldLocation = ComponentSpacePoses[AnimContextIndex].GetComponentSpaceTransform(FCompactPoseBoneIndex(ParentBoneIndex)).GetTranslation();

										for (int32 DrawAnimContextIndex = 0; DrawAnimContextIndex < NumAnimContexts; ++DrawAnimContextIndex)
										{
											const UObject* DrawAnimContext = AnimContexts[DrawAnimContextIndex];
											const FColor Color = DrawAnimContext == AnimContext ? FColor::Red : FColor::Blue;

											// current characters poses
											UE_VLOG_SEGMENT(DrawAnimContext, LogName, VeryVerbose, CurrentBoneWorldLocation, CurrentParentBoneWorldLocation, Color, TEXT(""));

											// sampled poses from UAnimationAsset
											UE_VLOG_SEGMENT(DrawAnimContext, LogName, VeryVerbose, AnimBoneWorldLocation, AnimParentBoneWorldLocation, Color, TEXT(""));
										}
									}
								}
							}
#endif //ENABLE_VISUAL_LOG
						}
					}
				}
			}
		}

		for (TArray<FPoseSearchConstraint>::TIterator ConstraintIt = InOutConstraints.CreateIterator(); ConstraintIt; ++ConstraintIt)
		{
			FPoseSearchConstraint& Constraint = *ConstraintIt;
			const int32* FromSocketRoleIndex = RoleToIndex.Find(Constraint.FromSocketRole);
			const int32* ToSocketRoleIndex = RoleToIndex.Find(Constraint.ToSocketRole);

			if (!FromSocketRoleIndex || !ToSocketRoleIndex)
			{
#if !NO_LOGGING
				if (!FromSocketRoleIndex)
				{
					UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::UpdateConstraints, Failed to find Role '%ls' used in a PoseSearchConstraint in asset '%ls'", *Constraint.FromSocketRole.ToString(), *MultiAnimAsset->GetName());
				}

				if (!ToSocketRoleIndex)
				{
					UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::UpdateConstraints, Failed to find Role '%ls' used in a PoseSearchConstraint in asset '%ls'", *Constraint.ToSocketRole.ToString(), *MultiAnimAsset->GetName());
				}
#endif // !NO_LOGGING
				ConstraintIt.RemoveCurrent();
			}
			else
			{
				int32 FromBoneIndex = INDEX_NONE;
				int32 ToBoneIndex = INDEX_NONE;

				FTransform FromSocketLocalTransform(FTransform::Identity);
				FTransform ToSocketLocalTransform(FTransform::Identity);

				if (!SkeletalMeshComponents[*FromSocketRoleIndex]->GetSocketInfoByName(Constraint.FromSocketName, FromSocketLocalTransform, FromBoneIndex))
				{
					FromBoneIndex = SkeletalMeshComponents[*FromSocketRoleIndex]->GetBoneIndex(Constraint.FromSocketName);
				}

				if (!SkeletalMeshComponents[*ToSocketRoleIndex]->GetSocketInfoByName(Constraint.ToSocketName, ToSocketLocalTransform, ToBoneIndex))
				{
					ToBoneIndex = SkeletalMeshComponents[*ToSocketRoleIndex]->GetBoneIndex(Constraint.ToSocketName);
				}

				if (FromBoneIndex == INDEX_NONE || ToBoneIndex == INDEX_NONE)
				{
#if !NO_LOGGING
					if (FromBoneIndex == INDEX_NONE)
					{
						UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::UpdateConstraints, Failed to find Socket '%ls' from SkeletalMeshComponent '%ls' for Role '%ls'", *Constraint.FromSocketName.ToString(), *GetNameSafe(SkeletalMeshComponents[*FromSocketRoleIndex]), *Constraint.FromSocketRole.ToString());
					}

					if (ToBoneIndex == INDEX_NONE)
					{
						UE_LOGF(LogPoseSearch, Error, "UPoseSearchInteractionLibrary::UpdateConstraints, Failed to find Socket '%ls' from SkeletalMeshComponent '%ls' for Role '%ls'", *Constraint.ToSocketName.ToString(), *GetNameSafe(SkeletalMeshComponents[*ToSocketRoleIndex]), *Constraint.ToSocketRole.ToString());
					}
#endif // !NO_LOGGING
					ConstraintIt.RemoveCurrent();
				}
				else
				{
					const FTransform CurrentFromTransform = FromSocketLocalTransform * (AnimContextsComponentSpaceTransforms[*FromSocketRoleIndex][FromBoneIndex] * ComponentToWorldTransforms[*FromSocketRoleIndex]);
					const FTransform CurrentToTransform = ToSocketLocalTransform * (AnimContextsComponentSpaceTransforms[*ToSocketRoleIndex][ToBoneIndex] * ComponentToWorldTransforms[*ToSocketRoleIndex]);

					const FTransform AnimFromTransform = FromSocketLocalTransform * (ComponentSpacePoses[*FromSocketRoleIndex].GetComponentSpaceTransform(FCompactPoseBoneIndex(FromBoneIndex)));
					const FTransform AnimToTransform = ToSocketLocalTransform * (ComponentSpacePoses[*ToSocketRoleIndex].GetComponentSpaceTransform(FCompactPoseBoneIndex(ToBoneIndex)));

					const FTransform WantedCurrentToTransform = AnimToTransform.GetRelativeTransform(AnimFromTransform) * CurrentFromTransform;
					const FTransform WantedCurrentFromTransform = AnimFromTransform.GetRelativeTransform(AnimToTransform) * CurrentToTransform;

					Constraint.FromSocketTransform.SetTranslation(FMath::Lerp(WantedCurrentFromTransform.GetTranslation(), CurrentFromTransform.GetTranslation(), Constraint.TranslationWeight));
					Constraint.FromSocketTransform.SetRotation(FQuat::Slerp(WantedCurrentFromTransform.GetRotation(), CurrentFromTransform.GetRotation(), Constraint.RotationWeight));
					Constraint.FromSocketTransform.SetScale3D(CurrentFromTransform.GetScale3D());

					Constraint.ToSocketTransform.SetTranslation(FMath::Lerp(WantedCurrentToTransform.GetTranslation(), CurrentToTransform.GetTranslation(), 1.f - Constraint.TranslationWeight));
					Constraint.ToSocketTransform.SetRotation(FQuat::Slerp(WantedCurrentToTransform.GetRotation(), CurrentToTransform.GetRotation(), 1.f - Constraint.RotationWeight));
					Constraint.ToSocketTransform.SetScale3D(CurrentToTransform.GetScale3D());

#if WITH_EDITORONLY_DATA && ENABLE_ANIM_DEBUG && ENABLE_VISUAL_LOG
					if (FVisualLogger::IsRecording())
					{
						static const TCHAR* LogName = TEXT("PoseSearchInteraction_Constraints");

						for (int32 DrawAnimContextIndex = 0; DrawAnimContextIndex < NumAnimContexts; ++DrawAnimContextIndex)
						{
							const UObject* DrawAnimContext = AnimContexts[DrawAnimContextIndex];

							// current characters poses
							UE_VLOG_SEGMENT(DrawAnimContext, LogName, VeryVerbose, CurrentFromTransform.GetTranslation(), CurrentToTransform.GetTranslation(), FColor::Yellow, TEXT(""));

							// sampled poses from UAnimationAsset
							UE_VLOG_SEGMENT(DrawAnimContext, LogName, VeryVerbose, AnimFromTransform.GetTranslation(), AnimToTransform.GetTranslation(), FColor::Yellow, TEXT(""));

							// wanted transforms
							UE_VLOG_SEGMENT(DrawAnimContext, LogName, VeryVerbose, CurrentFromTransform.GetTranslation(), WantedCurrentToTransform.GetTranslation(), FColor::Green, TEXT(""));
							UE_VLOG_SEGMENT(DrawAnimContext, LogName, VeryVerbose, CurrentToTransform.GetTranslation(), WantedCurrentFromTransform.GetTranslation(), FColor::Green, TEXT(""));

							UE_VLOG_SEGMENT(DrawAnimContext, LogName, VeryVerbose, CurrentFromTransform.GetTranslation(), Constraint.FromSocketTransform.GetTranslation(), FColor::Orange, TEXT(""));
							UE_VLOG_SEGMENT(DrawAnimContext, LogName, VeryVerbose, CurrentToTransform.GetTranslation(), Constraint.ToSocketTransform.GetTranslation(), FColor::Orange, TEXT(""));

							const FColor DesiredReachColor = (FLinearColor::Red * (1.f - Constraint.DesiredReach) + FLinearColor::Green * Constraint.DesiredReach).ToFColor(true);
							UE_VLOG_SPHERE(DrawAnimContext, LogName, Display, Constraint.FromSocketTransform.GetTranslation(), 2.f, DesiredReachColor, TEXT(""));
							UE_VLOG_SPHERE(DrawAnimContext, LogName, Display, Constraint.ToSocketTransform.GetTranslation(), 2.f, DesiredReachColor, TEXT(""));
						}
					}
#endif // WITH_EDITORONLY_DATA && ENABLE_ANIM_DEBUG && ENABLE_VISUAL_LOG
				}
			}
		}
	}
}