// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchInteractionSubsystem.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/World.h"
#include "PoseSearch/AnimNode_PoseSearchHistoryCollector.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchInteractionIsland.h"
#include "PoseSearch/PoseSearchInteractionUtils.h"
#include "PoseSearch/PoseSearchRole.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearch/PoseSearchSettings.h"
#include "PoseSearch/Trace/PoseSearchTraceLogger.h"
#include "Util/GridIndexing2.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchInteractionSubsystem)

namespace UE::PoseSearch
{
enum EGenerateAnimContextInfosMethod
{
	BruteForce,
	UniqueCommonRoleFilter,
	HashGrid
};
static int32 GVarPoseSearchInteractionGenerateAnimContextInfosMethod = HashGrid;
#if !NO_CVARS
static FAutoConsoleVariableRef CVarPoseSearchInteractionGenerateAnimContextInfosMethod(TEXT("a.PoseSearchInteraction.GenerateAnimContextInfosMethod"), GVarPoseSearchInteractionGenerateAnimContextInfosMethod,
	TEXT("GenerateAnimContextInfosMethod: 0) brute force, all vs all no partitioning, 1) Sort AnimContextInfos by unique common role filter, it improves broadphase performance when actors publish availabilities with a single role filter, 2) use hash grid partitioning"));

static bool GVarPoseSearchInteractionEnabled = true;
static FAutoConsoleVariableRef CVarPoseSearchInteractionEnabled(TEXT("a.PoseSearchInteraction.Enabled"), GVarPoseSearchInteractionEnabled, TEXT("Enable/Disable Pose Search Interaction"));

static bool GVarPoseSearchInteractionCacheIslands = true;
static FAutoConsoleVariableRef CVarPoseSearchInteractionCacheIslands(TEXT("a.PoseSearchInteraction.CacheIslands"), GVarPoseSearchInteractionCacheIslands, TEXT("Cache Pose Search Interaction Islands for future reuse instead of destrying them"));

static bool GVarPoseSearchInteractionLoglandsTickDependencies = false;
static FAutoConsoleVariableRef CVarPoseSearchInteractionLoglandsTickDependencies(TEXT("a.PoseSearchInteraction.LoglandsTickDependencies"), GVarPoseSearchInteractionLoglandsTickDependencies, TEXT("Log islands tick dependencies"));
#endif // !NO_CVARS

struct FAnimContextInfo
{
	void Init(const FInteractionAnimContextAvailabilities& InAnimContextAvailabilities)
	{
		check(InAnimContextAvailabilities.AnimContext && !InAnimContextAvailabilities.Availabilities.IsEmpty());
		AnimContextAvailabilities = &InAnimContextAvailabilities;
		
		// using ContextOwningActor location instead of skeletal mesh location since it's faster for UAF
		//Location = GetContextLocation(InAnimContextAvailabilities.AnimContext, false);
		const AActor* ContextOwningActor = GetContextOwningActor(InAnimContextAvailabilities.AnimContext);
		check(ContextOwningActor);
		Location = ContextOwningActor->GetTransform().GetLocation();

		AvailabilitiesMaxBroadPhaseRadius = 0.f;
		for (const FInteractionAvailability& Availability : AnimContextAvailabilities->Availabilities)
		{
			// @todo: optimize the AvailabilitiesMaxBroadPhaseRadius, since adding Availability.BroadPhaseRadiusIncrementOnInteraction is required ONLY if AnimContext is already part of an interaction
			AvailabilitiesMaxBroadPhaseRadius = FMath::Max(AvailabilitiesMaxBroadPhaseRadius, Availability.BroadPhaseRadius + Availability.BroadPhaseRadiusIncrementOnInteraction);
		}
	}

	// returns the common FRole used as RolesFilter if it's only one and consistent across all the AnimContextAvailabilities->Availabilities
	// used to categorize FAnimContextInfo(s) in different buckets to speed up the broed phase calculation
	const FRole* GetUniqueCommonRoleFilter() const
	{
		const FRole* UniqueCommonRoleFilter = nullptr;
		for (const FInteractionAvailability& Availability : AnimContextAvailabilities->Availabilities)
		{
			if (Availability.RolesFilter.Num() == 1)
			{
				if (!UniqueCommonRoleFilter)
				{
					UniqueCommonRoleFilter = &Availability.RolesFilter[0];
				}
				else if (*UniqueCommonRoleFilter != Availability.RolesFilter[0])
				{
					UniqueCommonRoleFilter = nullptr;
					break;
				}
			}
			else
			{
				UniqueCommonRoleFilter = nullptr;
				break;
			}
		}

		return UniqueCommonRoleFilter;
	}

	const FVector& GetLocation() const
	{
		return Location;
	}

	FVector2f GetLocation2f() const
	{
		return FVector2f(Location.X, Location.Y);
	}

	float GetAvailabilitiesMaxBroadPhaseRadius() const
	{
		return AvailabilitiesMaxBroadPhaseRadius;
	}
	
	void AddNearbyAnimContextInfo(const FAnimContextInfo& NearbyAnimContextInfo)
	{
		NearbyAnimContextInfos.Emplace(&NearbyAnimContextInfo);
	}

	const UObject* GetAnimContext() const
	{
		return AnimContextAvailabilities->AnimContext;
	}

	const TArray<FInteractionAvailability>& GetAvailabilities() const
	{
		return AnimContextAvailabilities->Availabilities;
	}

	typedef TArray<const FAnimContextInfo*, TInlineAllocator<8, TMemStackAllocator<>>> FNearbyAnimContextInfos;
	const FNearbyAnimContextInfos& GetNearbyAnimContextInfos() const
	{
		return NearbyAnimContextInfos;
	}

private:
	const FInteractionAnimContextAvailabilities* AnimContextAvailabilities = nullptr;

	// cached AnimContext location
	FVector Location = FVector::ZeroVector;
	float AvailabilitiesMaxBroadPhaseRadius = 0.f;
	FNearbyAnimContextInfos NearbyAnimContextInfos;
};
struct FAnimContextInfos : TArray<FAnimContextInfo, TMemStackAllocator<>> {};

typedef TArray<const UPoseSearchDatabase*, TInlineAllocator<32, TMemStackAllocator<>>> FDatabasesPerTag;
struct FTagToDatabases : TMap<FName, FDatabasesPerTag, TInlineSetAllocator<8, TMemStackSetAllocator<>>> {};

// FInteractionAvailability
///////////////////////////////////////////////
FString FInteractionAvailability::GetPoseHistoryName() const
{
	if (PoseHistory)
	{
		return "HistoryProvider";
	}
	return PoseHistoryName.ToString();
}

const UE::PoseSearch::IPoseHistory* FInteractionAvailability::GetPoseHistory(const UObject* AnimContext) const
{
	using namespace UE::PoseSearch;

	if (PoseHistory)
	{
		return PoseHistory;
	}
		
	if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(AnimContext))
	{
		if (const FAnimNode_PoseSearchHistoryCollector_Base* PoseSearchHistoryCollector = UPoseSearchLibrary::FindPoseHistoryNode(PoseHistoryName, AnimInstance))
		{
			if (const FPoseHistory* PoseHistoryFromCollector = PoseSearchHistoryCollector->GetPoseHistoryPtr())
			{
				return PoseHistoryFromCollector;
			}
		}
	}

	UE_LOGF(LogPoseSearch, Error, "FInteractionAvailability::GetPoseHistory couldn't find a pose history for AnimContext %ls with name %ls", *GetNameSafe(AnimContext), *PoseHistoryName.ToString());
	return nullptr;
}

} // namespace UE::PoseSearch

// UPoseSearchInteractionSubsystem
///////////////////////////////////////////////
UE::PoseSearch::FInteractionIsland& UPoseSearchInteractionSubsystem::CreateIsland()
{
	return *Islands.Add_GetRef(new UE::PoseSearch::FInteractionIsland(ToRawPtr(GetWorld()->PersistentLevel), this));
}

void UPoseSearchInteractionSubsystem::DestroyIsland(int32 Index)
{
	delete Islands[Index];
	Islands.RemoveAt(Index);
}

UE::PoseSearch::FInteractionIsland& UPoseSearchInteractionSubsystem::GetAvailableIsland()
{
	using namespace UE::PoseSearch;

	for (FInteractionIsland* Island : Islands)
	{
		if (!Island->IsInitialized())
		{
			return *Island;
		}
	}

	return CreateIsland();
}

void UPoseSearchInteractionSubsystem::DestroyAllIslands()
{
	for (int32 IslandIndex = Islands.Num() - 1; IslandIndex >= 0; --IslandIndex)
	{
		DestroyIsland(IslandIndex);
	}
}

void UPoseSearchInteractionSubsystem::RegenerateAllIslands(float DeltaSeconds)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UPoseSearchInteractionSubsystem_RegenerateAllIslands);

	using namespace UE::PoseSearch;

	check(IsInGameThread());

	FMemMark Mark(FMemStack::Get());

	// generating all the possible interaction tuples of AnimContext(s) with roles and pose histories (defined in FInteractionSearchContext)
	FInteractionSearchContexts SearchContexts;
	GenerateSearchContexts(DeltaSeconds, SearchContexts);

#if ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
	// drawing the current frame islands to be consistent with the search, before regenerating the islands with the newly published availabilities
	DebugDrawIslands();
#endif // ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
	
#if ENABLE_ANIM_DEBUG
	DebugLogTickDependencies();
#endif // ENABLE_ANIM_DEBUG

#if !NO_CVARS
	if (!GVarPoseSearchInteractionCacheIslands)
	{
		// not caching the islands. Destroy them all!
		DestroyAllIslands();
	}
	else
#endif // !NO_CVARS
	{
		for (FInteractionIsland* Island : Islands)
		{
			CheckInteractionThreadSafety(Island);
			Island->Uninitialize(true);
		}
	}

	struct FInteractionSearchContextGroup
	{
		typedef TPair<const UObject*, int32> FAnimContextToTickPriorityPair;
		typedef TArray<FAnimContextToTickPriorityPair, TInlineAllocator<16, TMemStackAllocator<>>> FAnimContextToTickPriority;
		typedef TArray<int32, TInlineAllocator<16, TMemStackAllocator<>>> FSearchContextsIndices;

	private:
		typedef TMap<const UObject*, int32, TInlineSetAllocator<16, TMemStackSetAllocator<>>> FAnimContextToTickPriorityMap;

		// all the AnimContexts in this group with their TickPriority
		FAnimContextToTickPriorityMap AnimContextToTickPriorityMap;

		// sorted pair contained in AnimContextToTickPriorityMap calcualted during Finalize
		FAnimContextToTickPriority AnimContextToTickPriority;

		// indexes to the searchcontexts assigned to this group
		FSearchContextsIndices SearchContextsIndices;

	public:
		const FAnimContextToTickPriority& GetAnimContextToTickPriority() const
		{
			// did we call Finalize?
			check(AnimContextToTickPriority.Num() == AnimContextToTickPriorityMap.Num());
			return AnimContextToTickPriority;
		}

		const FSearchContextsIndices& GetSearchContextsIndices() const
		{
			// did we call Finalize?
			check(AnimContextToTickPriority.Num() == AnimContextToTickPriorityMap.Num());
			return SearchContextsIndices;
		}

		bool Contains(const FInteractionSearchContext& SearchContext) const
		{
			for (int32 AnimContextIndex = 0; AnimContextIndex < SearchContext.Num(); ++AnimContextIndex)
			{
				if (AnimContextToTickPriorityMap.Find(SearchContext.GetAnimContext(AnimContextIndex)))
				{
					return true;
				}
			}
			return false;
		}

		void Add(const FInteractionSearchContext& SearchContext, int32 SearchContextIndex)
		{
			for (int32 AnimContextIndex = 0; AnimContextIndex < SearchContext.Num(); ++AnimContextIndex)
			{
				if (const UObject* AnimContext = SearchContext.GetAnimContext(AnimContextIndex))
				{
					if (int32* TickPriority = AnimContextToTickPriorityMap.Find(AnimContext))
					{
						*TickPriority = FMath::Max(*TickPriority, SearchContext.GetTickPriority(AnimContextIndex));
					}
					else
					{
						AnimContextToTickPriorityMap.Add(AnimContext) = SearchContext.GetTickPriority(AnimContextIndex);
					}
				}
			}

			SearchContextsIndices.Add(SearchContextIndex);
		}

		void Merge(const FInteractionSearchContextGroup& SearchContextGroup)
		{
			for (const FAnimContextToTickPriorityPair& AnimContextToTickPriorityPair : SearchContextGroup.AnimContextToTickPriorityMap)
			{
				if (int32* TickPriority = AnimContextToTickPriorityMap.Find(AnimContextToTickPriorityPair.Key))
				{
					*TickPriority = FMath::Max(*TickPriority, AnimContextToTickPriorityPair.Value);
				}
				else
				{
					AnimContextToTickPriorityMap.Add(AnimContextToTickPriorityPair.Key) = AnimContextToTickPriorityPair.Value;
				}
			}

			for (int32 SearchContextsIndex : SearchContextGroup.SearchContextsIndices)
			{
				SearchContextsIndices.Add(SearchContextsIndex);
			}
		}

		// sorting by TickPriority first and by AnimContext in case of same TickPriority
		void Finalize(const FInteractionSearchContexts& SearchContexts)
		{
			SearchContextsIndices.Sort([&SearchContexts](int32 A, int32 B)
				{
					return SearchContexts[A].GetAnimContext(0) < SearchContexts[B].GetAnimContext(0);
				});

			AnimContextToTickPriority.Reset();
			AnimContextToTickPriority.Reserve(AnimContextToTickPriorityMap.Num());
			for (const FInteractionSearchContextGroup::FAnimContextToTickPriorityPair& AnimContextToTickPriorityPair : AnimContextToTickPriorityMap)
			{
				AnimContextToTickPriority.Add(AnimContextToTickPriorityPair);
			}
			AnimContextToTickPriority.Sort(
				[](const FInteractionSearchContextGroup::FAnimContextToTickPriorityPair& A, const FInteractionSearchContextGroup::FAnimContextToTickPriorityPair& B)
				{
					if (B.Value != A.Value)
					{
						return B.Value < A.Value;
					}

					return A.Key < B.Key;
				});
		}
	};

	struct FInteractionSearchContextGroups
	{
	private:
		// grouping SearchContexts AnimContext(s) in FInteractionSearchContextGroup(s). We'll create as many interaction islands as many groups
		TArray<FInteractionSearchContextGroup, TInlineAllocator<PreallocatedSearchesNum, TMemStackAllocator<>>> Groups;
		TArray<int32, TInlineAllocator<PreallocatedSearchesNum, TMemStackAllocator<>>> SortedGroupsIndexes;

	public:
		int32 Num() const
		{
			check(SortedGroupsIndexes.Num() == Groups.Num());
			return SortedGroupsIndexes.Num();
		}

		const FInteractionSearchContextGroup& operator[](int32 Index) const
		{
			return Groups[SortedGroupsIndexes[Index]];
		}

		FInteractionSearchContextGroups(const FInteractionSearchContexts& SearchContexts)
		{
			for (int32 SearchContextIndex = 0; SearchContextIndex < SearchContexts.Num(); ++SearchContextIndex)
			{
				// evaluating where to place SearchContext..
				const FInteractionSearchContext& SearchContext = SearchContexts[SearchContextIndex];

				int32 MainGroupIndex = INDEX_NONE;
				for (int32 GroupIndex = 0; GroupIndex < Groups.Num();)
				{
					// ..if Groups[GroupIndex] contains ANY of the AnimContexts from SearchContext..
					if (Groups[GroupIndex].Contains(SearchContext))
					{
						if (MainGroupIndex == INDEX_NONE)
						{
							// ..we add SearchContext to Groups[GroupIndex] 
							// and set MainGroupIndex to GroupIndex to know what is the group containing SearchContext, so..
							MainGroupIndex = GroupIndex;
							Groups[MainGroupIndex].Add(SearchContext, SearchContextIndex);
							++GroupIndex;
						}
						else
						{
							// ..in case SearchContext has already being inserted in MainGroupIndex group 
							// we merge the newly found Groups[GroupIndex] to Groups[MainGroupIndex]
							// (containing another of the the AnimContexts)
							Groups[MainGroupIndex].Merge(Groups[GroupIndex]);
							Groups.RemoveAt(GroupIndex);
						}
					}
					else
					{
						++GroupIndex;
					}
				}
				if (MainGroupIndex == INDEX_NONE)
				{
					Groups.AddDefaulted_GetRef().Add(SearchContext, SearchContextIndex);
				}
			}

			// finalizing the Groups first to sort their internal data structures to be able to then sort the SortedGroupsIndexes
			for (FInteractionSearchContextGroup& Group : Groups)
			{
				Group.Finalize(SearchContexts);
			}

			SortedGroupsIndexes.SetNumUninitialized(Groups.Num());
			for (int32 GroupIndex = 0; GroupIndex < Groups.Num(); ++GroupIndex)
			{
				SortedGroupsIndexes[GroupIndex] = GroupIndex;
			}

			SortedGroupsIndexes.Sort([this, &SearchContexts](int32 A, int32 B)
				{
					const int32 SearchContextsIndexA = Groups[A].GetSearchContextsIndices()[0];
					const int32 SearchContextsIndexB = Groups[B].GetSearchContextsIndices()[0];

					const UObject* AnimContextA = SearchContexts[SearchContextsIndexA].GetAnimContext(0);
					const UObject* AnimContextB = SearchContexts[SearchContextsIndexB].GetAnimContext(0);

					return AnimContextA < AnimContextB;
				});
		}
	};

	FInteractionSearchContextGroups SearchContextGroups(SearchContexts);
	for (int32 SearchContextGroupIndex = 0; SearchContextGroupIndex < SearchContextGroups.Num(); ++SearchContextGroupIndex)
	{
		// @todo: search for the most suitable island to reuse to avoid having to Uninitialize/RemoveTickDependencies and InjectToActor right away
		FInteractionIsland& Island = GetAvailableIsland();
		CheckInteractionThreadSafety(&Island);

		// initializing the island with its assigned SearchContexts
		bool bAreTickDependenciesRequired = false;
		check(Island.GetSearchContexts().IsEmpty());
		
		const FInteractionSearchContextGroup& SearchContextGroup = SearchContextGroups[SearchContextGroupIndex];
		for (int32 SearchContextsIndex : SearchContextGroup.GetSearchContextsIndices())
		{
			const FInteractionSearchContext& SearchContext = SearchContexts[SearchContextsIndex];
			// if there're at least two AnimContext(s) potentially interacting with each other 
			// (where the search involves 2+ characters) tick dependencies are required to be thread safe
			bAreTickDependenciesRequired |= SearchContext.Num() > 1;
			Island.AddSearchContext(SearchContext);
		}

		// injecting tick dependencies between island AnimContext following their TickPriorities, 
		// so the AnimContext with the highest TickPriority will be elected as "Main Actor", being evaluated, 
		// and performing all the island searches, before any other Actor in the same island\
		// (that'll end up using the cached search results in a multithread manner)
		for (const FInteractionSearchContextGroup::FAnimContextToTickPriorityPair& SortedByTickPriorityAnimContext : SearchContextGroup.GetAnimContextToTickPriority())
		{
			Island.InjectToActor(SortedByTickPriorityAnimContext.Key, bAreTickDependenciesRequired);
		}
	}
}

#if DO_CHECK
bool UPoseSearchInteractionSubsystem::ValidateAllIslands() const
{
	using namespace UE::PoseSearch;

	TSet<const UActorComponent*> TickActorComponents;

	typedef TSet<const UObject*> FIslandAnimContexts;
	TArray<FIslandAnimContexts> IslandsAnimContexts;

	const int32 NumIslands = Islands.Num();
	IslandsAnimContexts.Reserve(NumIslands);

	bool bAlreadyInSet = false;
	for (const FInteractionIsland* Island : Islands)
	{
		for (const UActorComponent* TickActorComponent : Island->GetTickActorComponents())
		{
			TickActorComponents.Add(TickActorComponent, &bAlreadyInSet);
			if (bAlreadyInSet)
			{
				return false;
			}
		}

		FIslandAnimContexts& IslandAnimContexts = IslandsAnimContexts.AddDefaulted_GetRef();
		for (const FInteractionSearchContext& SearchContext : Island->GetSearchContexts())
		{
			for (int32 AnimContextIndex = 0; AnimContextIndex < SearchContext.Num(); ++AnimContextIndex)
			{
				if (const UObject* AnimContext = SearchContext.GetAnimContext(AnimContextIndex))
				{
					IslandAnimContexts.Add(AnimContext);
				}
			}
		}
	}

	for (int32 IslandIndex = 0; IslandIndex < NumIslands; ++IslandIndex)
	{
		for (const UObject* AnimContext : IslandsAnimContexts[IslandIndex])
		{
			for (int32 OtherIslandIndex = 0; OtherIslandIndex < NumIslands; ++OtherIslandIndex)
			{
				if (IslandIndex != OtherIslandIndex)
				{
					if (IslandsAnimContexts[OtherIslandIndex].Find(AnimContext))
					{
						// AnimContext is shared between multiple islands. it'd cause multi threadind issues!
						return false;
					}
				}
			}
		}				
	}

	return true;
}
#endif // DO_CHECK

UE::PoseSearch::FInteractionIsland* UPoseSearchInteractionSubsystem::FindIsland(const UObject* AnimContext, bool bCompareOwningActors)
{
	using namespace UE::PoseSearch;

	if (AnimContext)
	{
		if (bCompareOwningActors)
		{
			const AActor* Actor = GetContextOwningActor(AnimContext, false);

			for (FInteractionIsland* Island : Islands)
			{
				for (const UObject* IslandAnimContext : Island->GetIslandAnimContexts())
				{ 
					if (GetContextOwningActor(IslandAnimContext, false) == Actor)
					{
						return Island;
					}
				}
			}
		}
		else
		{
			for (FInteractionIsland* Island : Islands)
			{
				if (Island->GetIslandAnimContexts().Contains(AnimContext))
				{
					return Island;
				}
			}
		}
	}
	return nullptr;
}

UPoseSearchInteractionSubsystem* UPoseSearchInteractionSubsystem::GetSubsystem_AnyThread(const UObject* AnimContext)
{
	if (AnimContext)
	{
		return GetSubsystem_AnyThread(AnimContext->GetWorld());
	}
	return nullptr;
}

UPoseSearchInteractionSubsystem* UPoseSearchInteractionSubsystem::GetSubsystem_AnyThread(const UWorld* World)
{
	if (World)
	{
		// We expect the subsystem to be already created from the GameThread.
		// We don't create the subsystem from any thread
		if (World->HasSubsystem<UPoseSearchInteractionSubsystem>())
		{
			return World->GetSubsystem<UPoseSearchInteractionSubsystem>();
		}
	}
	return nullptr;
}

void UPoseSearchInteractionSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	check(IsInGameThread());
	Super::Initialize(Collection);

	const UPoseSearchSettings& Settings = UPoseSearchSettings::Get();
	AnimContextsAvailabilitiesBuffer.SetNum(Settings.AvailabilitiesBufferSize);
}

void UPoseSearchInteractionSubsystem::Deinitialize()
{
	ValidInteractionSearches.Update(this);
	DestroyAllIslands();
	Super::Deinitialize();
}

void UPoseSearchInteractionSubsystem::AddAvailabilities(const TArrayView<const FPoseSearchInteractionAvailability> Availabilities, const UObject* AnimContext, FName PoseHistoryName, const UE::PoseSearch::IPoseHistory* PoseHistory)
{
	using namespace UE::PoseSearch;

	check(AnimContext && AnimContext->GetWorld() && AnimContext->GetWorld() == GetWorld());

	if (!Availabilities.IsEmpty())
	{
		const int32 ReservedAnimContextsAvailabilitiesIndex = AnimContextsAvailabilitiesIndex.Add(1);

		if (ReservedAnimContextsAvailabilitiesIndex < AnimContextsAvailabilitiesBuffer.Num())
		{
			FInteractionAnimContextAvailabilities& AnimContextAvailabilities = AnimContextsAvailabilitiesBuffer[ReservedAnimContextsAvailabilitiesIndex];
			AnimContextAvailabilities.AnimContext = AnimContext;
			AnimContextAvailabilities.Availabilities.Reset();
			for (const FPoseSearchInteractionAvailability& Availability : Availabilities)
			{
				AnimContextAvailabilities.Availabilities.AddDefaulted_GetRef().Init(Availability, PoseHistoryName, PoseHistory);
			}
		}
	}
}

void UPoseSearchInteractionSubsystem::GenerateAnimContextInfosAndTagToDatabases(UE::PoseSearch::FAnimContextInfos& AnimContextInfos, UE::PoseSearch::FTagToDatabases& TagToDatabases) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UPoseSearchInteractionSubsystem_GenerateAnimContextInfos);

	using namespace UE::Geometry;
	using namespace UE::PoseSearch;

	const UWorld* SubsystemWorld = GetWorld();
	TConstArrayView<FInteractionAnimContextAvailabilities> AnimContextsAvailabilities = MakeArrayView(AnimContextsAvailabilitiesBuffer.GetData(), AnimContextsAvailabilitiesNum);
	
#if DO_CHECK
	check(SubsystemWorld);
	check(AnimContextInfos.IsEmpty() && TagToDatabases.IsEmpty());
#endif // DO_CHECK

	for (const FInteractionAnimContextAvailabilities& AnimContextAvailabilities : AnimContextsAvailabilities)
	{
#if DO_CHECK
		check(AnimContextAvailabilities.AnimContext);
		const UWorld* AnimContextWorld = AnimContextAvailabilities.AnimContext->GetWorld();
		check(AnimContextWorld == SubsystemWorld);
		check(!AnimContextAvailabilities.Availabilities.IsEmpty());
#endif // DO_CHECK
		
		// adding AnimContext to AnimContextsWithAvailabilities only if at least one availability has a valid database or has a valid tag
		for (const FInteractionAvailability& InteractionAvailability : AnimContextAvailabilities.Availabilities)
		{
			bool bAnyValidAvailability = false;
			if (const UPoseSearchDatabase* Database = InteractionAvailability.Database.Get())
			{
				check(Database->Schema);
				if (InteractionAvailability.IsTagValid())
				{
					TagToDatabases.FindOrAdd(InteractionAvailability.Tag).AddUnique(InteractionAvailability.Database);
				}
			}
		}
	}

	auto EvaluateForNearbyAnimContextInfo = [&AnimContextInfos](int32 AnimContextInfoIndexA, int32 AnimContextInfoIndexB)
		{
			// performs broad phase analysis checking if at least one of the Availabilities associated to AnimContext can interact with OtherAnimContextInfo.
			const FVector DeltaLocation = AnimContextInfos[AnimContextInfoIndexA].GetLocation() - AnimContextInfos[AnimContextInfoIndexB].GetLocation();
			const float DistanceSquared = DeltaLocation.SquaredLength();
			const float MaxDistance = FMath::Min(AnimContextInfos[AnimContextInfoIndexA].GetAvailabilitiesMaxBroadPhaseRadius(), AnimContextInfos[AnimContextInfoIndexB].GetAvailabilitiesMaxBroadPhaseRadius());
			const float MaxDistanceSquared = MaxDistance * MaxDistance;
			if (DistanceSquared <= MaxDistanceSquared)
			{
				// the AnimContext of AnimContextInfos[AnimContextInfoIndexA] can potentially interact with the one from AnimContextInfos[AnimContextInfoIndexB]:
				// linking AnimContextInfos[AnimContextInfoIndexA] to AnimContextInfos[AnimContextInfoIndexB] and vice versa to keep track of this when evaluating the broad phase.
				// Since AnimContextInfos does't reallocate anymore, it's safe to store pointers to internal elements of the array!
				AnimContextInfos[AnimContextInfoIndexA].AddNearbyAnimContextInfo(AnimContextInfos[AnimContextInfoIndexB]);
				AnimContextInfos[AnimContextInfoIndexB].AddNearbyAnimContextInfo(AnimContextInfos[AnimContextInfoIndexA]);
			}
		};

	const int32 NumAnimContextInfos = AnimContextsAvailabilities.Num();
	AnimContextInfos.SetNum(NumAnimContextInfos);
	for (int32 AnimContextInfoIndex = 0; AnimContextInfoIndex < NumAnimContextInfos; ++AnimContextInfoIndex)
	{
		AnimContextInfos[AnimContextInfoIndex].Init(AnimContextsAvailabilities[AnimContextInfoIndex]);
	}

	switch (GVarPoseSearchInteractionGenerateAnimContextInfosMethod)
	{
	default:
	case EGenerateAnimContextInfosMethod::BruteForce:
	{
		// solving the broad phase using the AnimContextInfos
		for (int32 AnimContextInfoIndexA = 0; AnimContextInfoIndexA < NumAnimContextInfos - 1; ++AnimContextInfoIndexA)
		{
			for (int32 AnimContextInfoIndexB = AnimContextInfoIndexA + 1; AnimContextInfoIndexB < NumAnimContextInfos; ++AnimContextInfoIndexB)
			{
				EvaluateForNearbyAnimContextInfo(AnimContextInfoIndexA, AnimContextInfoIndexB);
			}
		}
		break;
	}
	
	case EGenerateAnimContextInfosMethod::UniqueCommonRoleFilter:
	{
		typedef TMap<const FRole, int32, TInlineSetAllocator<16, TMemStackSetAllocator<>>> FCommonRoleFilterToIndex;
		FCommonRoleFilterToIndex CommonRoleFilterToIndex;

		typedef TArray<TArray<int32, TMemStackAllocator<>>, TInlineAllocator<16, TMemStackAllocator<>>> FRoledAnimContextInfoIndexes;
		FRoledAnimContextInfoIndexes RoledAnimContextInfoIndexes;

		// adding RoledAnimContextInfoIndexes[0] containing all the indexes for those AnimContextInfos returning nullptr from GetUniqueCommonRoleFilter
		RoledAnimContextInfoIndexes.AddDefaulted();

		for (int32 AnimContextInfoIndex = 0; AnimContextInfoIndex < NumAnimContextInfos; ++AnimContextInfoIndex)
		{
			if (const FRole* UniqueCommonRoleFilter = AnimContextInfos[AnimContextInfoIndex].GetUniqueCommonRoleFilter())
			{
				if (int32* UniqueCommonRoleFilterIndex = CommonRoleFilterToIndex.Find(*UniqueCommonRoleFilter))
				{
					RoledAnimContextInfoIndexes[*UniqueCommonRoleFilterIndex].Add(AnimContextInfoIndex);
				}
				else
				{
					CommonRoleFilterToIndex.Add(*UniqueCommonRoleFilter) = RoledAnimContextInfoIndexes.Num();
					RoledAnimContextInfoIndexes.AddDefaulted_GetRef().Add(AnimContextInfoIndex);
				}
			}
			else
			{
				RoledAnimContextInfoIndexes[0].Add(AnimContextInfoIndex);
			}
		}

		// RoledAnimContextInfoIndexes[0] is a special case containing all those AnimContextInfos with more than just one unique common filter 
		// (GetUniqueCommonRoleFilter returned null), so we need to check all the possible pairs between those AnimContextInfos.
		// we're calling it first, so we can then sort RoledAnimContextInfoIndexes (losing reference to who's the first item) to speed up execution
		for (int32 IndexA = 0; IndexA < RoledAnimContextInfoIndexes[0].Num(); ++IndexA)
		{
			const int32 AnimContextInfoIndexA = RoledAnimContextInfoIndexes[0][IndexA];
			for (int32 IndexB = IndexA + 1; IndexB < RoledAnimContextInfoIndexes[0].Num(); ++IndexB)
			{
				const int32 AnimContextInfoIndexB = RoledAnimContextInfoIndexes[0][IndexB];
				EvaluateForNearbyAnimContextInfo(AnimContextInfoIndexA, AnimContextInfoIndexB);
			}
		}

		// checking all the Ith RoledAnimContextInfoIndexes against each other to find NearbyAnimContextInfo
		if (RoledAnimContextInfoIndexes.Num() > 1)
		{
			// sorting RoledAnimContextInfoIndexes by the internal arrays cardinality to minimize the number of calls 
			// of the inner 'for(s)' of the following nested 'for' statements
			RoledAnimContextInfoIndexes.Sort([](const TArray<int32, TMemStackAllocator<>>& A, const TArray<int32, TMemStackAllocator<>>& B)
				{
					return A.Num() < B.Num();
				});

			// obviously CommonRoleFilterToIndex is now invalid. Resetting it here just for clarity
			CommonRoleFilterToIndex.Reset();

			for (int32 IndexA = 0; IndexA < RoledAnimContextInfoIndexes.Num() - 1; ++IndexA)
			{
				for (int32 AnimContextInfoIndexA : RoledAnimContextInfoIndexes[IndexA])
				{
					for (int32 IndexB = IndexA + 1; IndexB < RoledAnimContextInfoIndexes.Num(); ++IndexB)
					{
						for (int32 AnimContextInfoIndexB : RoledAnimContextInfoIndexes[IndexB])
						{
							EvaluateForNearbyAnimContextInfo(AnimContextInfoIndexA, AnimContextInfoIndexB);
						}
					}
				}
			}
		}
		break;
	}

	case EGenerateAnimContextInfosMethod::HashGrid:
	{
		float GlobalMaxRadius = 0.f;
		for (int32 AnimContextInfoIndex = 0; AnimContextInfoIndex < NumAnimContextInfos; ++AnimContextInfoIndex)
		{
			GlobalMaxRadius = FMath::Max(GlobalMaxRadius, AnimContextInfos[AnimContextInfoIndex].GetAvailabilitiesMaxBroadPhaseRadius());
		}

		// creating an hash grid
		TScaleGridIndexer2<float> Indexer;
		// using a minimum cell size of twice the GlobalMaxRadius to minimize the number of cells to search for neighbors
		// AnimContextInfos (max of 4 cells), and clamping it to 1cm to avoid division by zero inside TScaleGridIndexer2
		Indexer.CellSize = FMath::Max(1.f, GlobalMaxRadius * 2.f);

		typedef TMultiMap<FVector2i, int, TMemStackSetAllocator<>> FHashGrid;
		FHashGrid Hash;
		Hash.Reserve(AnimContextsAvailabilitiesNum);

		for (int32 AnimContextInfoIndex = 0; AnimContextInfoIndex < NumAnimContextInfos; ++AnimContextInfoIndex)
		{
			const FVector2i idx = Indexer.ToGrid(AnimContextInfos[AnimContextInfoIndex].GetLocation2f());
			Hash.Add(idx, AnimContextInfoIndex);
		}

		// no need to evaluate the last AnimContextInfoIndex since the condition OtherAnimContextInfoIndex > AnimContextInfoIndex will fail anyways
		for (int32 AnimContextInfoIndex = 0; AnimContextInfoIndex < NumAnimContextInfos - 1; ++AnimContextInfoIndex)
		{
			const float Radius = AnimContextInfos[AnimContextInfoIndex].GetAvailabilitiesMaxBroadPhaseRadius();
			const FVector& Location = AnimContextInfos[AnimContextInfoIndex].GetLocation();
			const FVector2f QueryPoint(Location.X, Location.Y);

			const FVector2i min_idx = Indexer.ToGrid(QueryPoint - Radius * FVector2f::One());
			const FVector2i max_idx = Indexer.ToGrid(QueryPoint + Radius * FVector2f::One());

			const float RadiusSquared = Radius * Radius;

			for (int yi = min_idx.Y; yi <= max_idx.Y; yi++)
			{
				for (int xi = min_idx.X; xi <= max_idx.X; xi++)
				{
					const FVector2i idx(xi, yi);
					for (FHashGrid::TConstKeyIterator It = Hash.CreateConstKeyIterator(idx); It; ++It)
					{
						const int32 OtherAnimContextInfoIndex = It->Value;
						// to avoid adding duplicates we evaluate only pairs of indexes where OtherAnimContextInfoIndex > AnimContextInfoIndex
						if (OtherAnimContextInfoIndex > AnimContextInfoIndex)
						{
							EvaluateForNearbyAnimContextInfo(AnimContextInfoIndex, OtherAnimContextInfoIndex);
						}
					}
				}
			}
		}
		break;
	}
	}
}

void UPoseSearchInteractionSubsystem::GenerateSearchContexts(float DeltaSeconds, UE::PoseSearch::FInteractionSearchContexts& SearchContexts) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UPoseSearchInteractionSubsystem_GenerateSearchContexts);

	using namespace UE::PoseSearch;

	check(SearchContexts.IsEmpty());

	struct FRoledAnimContextInfo
	{
		FRoledAnimContextInfo(const FInteractionAvailability& InAvailability, const FAnimContextInfo& InAnimContextInfo, const FRole InRole, const IPoseHistory* InPoseHistory, const UPoseSearchDatabase& InDatabase)
			: Availability(&InAvailability)
			, AnimContextInfo(&InAnimContextInfo)
			, Role(InRole)
			, PoseHistory(InPoseHistory)
			, Database(&InDatabase)
			, BroadPhaseRadiusSquared(InAvailability.BroadPhaseRadius)
			, IncrementedBroadPhaseRadiusSquared(InAvailability.BroadPhaseRadius + InAvailability.BroadPhaseRadiusIncrementOnInteraction)
		{
			BroadPhaseRadiusSquared *= BroadPhaseRadiusSquared;
			IncrementedBroadPhaseRadiusSquared *= IncrementedBroadPhaseRadiusSquared;
		}

		bool operator==(const FRoledAnimContextInfo& Other) const
		{
			// no need to check for BroadPhaseRadiusSquared and IncrementedBroadPhaseRadiusSquared since they are derived values from Availability
			return
				Availability == Other.Availability &&
				AnimContextInfo == Other.AnimContextInfo &&
				Role == Other.Role &&
				PoseHistory == Other.PoseHistory &&
				Database == Other.Database;
		}

		bool operator<(const FRoledAnimContextInfo& Other) const
		{
			if (AnimContextInfo != Other.AnimContextInfo)
			{
				const UObject* AnimContext = AnimContextInfo->GetAnimContext();
				const UObject* OtherAnimContext = Other.AnimContextInfo->GetAnimContext();
				if (AnimContext != OtherAnimContext)
				{
					return AnimContext < OtherAnimContext;
				}

				return AnimContextInfo < Other.AnimContextInfo;
			}

			if (Availability != Other.Availability)
			{
				return Availability < Other.Availability;
			}

			if (Role != Other.Role)
			{
				return Role.FastLess(Other.Role);
			}

			if (PoseHistory != Other.PoseHistory)
			{
				return PoseHistory < Other.PoseHistory;
			}

			return Database < Other.Database;
		}

		// Availability that spawned this FRoledAnimContextInfo
		const FInteractionAvailability* Availability = nullptr;
		// AnimContextInfo containing all the information regarding the AnimContext that spawned this FRoledAnimContextInfo,
		// including all the availabilities associated to the AnimContext as well as all the other AnimContext(s) it can potentially interact with
		const FAnimContextInfo* AnimContextInfo = nullptr;
		FRole Role = DefaultRole;
		const IPoseHistory* PoseHistory = nullptr;
		const UPoseSearchDatabase* Database = nullptr;

		float BroadPhaseRadiusSquared = 0.f;
		float IncrementedBroadPhaseRadiusSquared = 0.f;
	};
	
	struct FRoledAnimContextInfos
	{
	private:
		typedef TArray<FRoledAnimContextInfo, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> FInternalData;

		FInternalData InternalData;
		TArray<bool, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> PresentRoleIndexes;

	public:
		FInternalData::SizeType Num() const
		{
			return InternalData.Num();
		}

		const FRoledAnimContextInfo& operator[](int32 Index)
		{
			return InternalData[Index];
		}

		bool HasAllRoles() const
		{
			for (bool bPresentRoleIndex : PresentRoleIndexes)
			{
				if (!bPresentRoleIndex)
				{
					return false;
				}
			}
			return true;
		}

		// sort RoledAnimContextInfos to generate deterministic SearchContext across multiple frames!
		void SortAndRemoveDuplicates()
		{
			if (InternalData.Num() <= 1)
			{
				return;
			}

			InternalData.Sort();

			int32 PrevIndex = 0;
			for (int32 Index = 1; Index < InternalData.Num(); ++Index)
			{
				if (InternalData[PrevIndex] != InternalData[Index])
				{
					++PrevIndex;

					if (PrevIndex != Index)
					{
						InternalData[PrevIndex] = MoveTemp(InternalData[Index]);
					}
				}
			}

			++PrevIndex;
			InternalData.RemoveAt(PrevIndex, Num() - PrevIndex, EAllowShrinking::No);
		}

		void AddRoledAnimContextInfos(const FInteractionAvailability& Availability, const FAnimContextInfo& AnimContextInfo, const IPoseHistory* PoseHistory, const UPoseSearchDatabase& Database)
		{
			const UPoseSearchSchema* Schema = Database.Schema;
			check(Schema);
			const TArray<FPoseSearchRoledSkeleton>& RoledSkeletons = Schema->GetRoledSkeletons();

			if (PresentRoleIndexes.IsEmpty())
			{
				check(InternalData.IsEmpty());
				PresentRoleIndexes.SetNumZeroed(RoledSkeletons.Num());
			}
			else
			{
				check(InternalData.IsEmpty() || InternalData[0].Database == &Database);
				check(Schema->GetRoledSkeletons().Num() == PresentRoleIndexes.Num());
			}

			if (Availability.RolesFilter.IsEmpty())
			{
				// adding ALL the possible roles from the database:
				for (int32 RoledSkeletonIndex = 0; RoledSkeletonIndex < RoledSkeletons.Num(); ++RoledSkeletonIndex)
				{
					InternalData.Emplace(Availability, AnimContextInfo, RoledSkeletons[RoledSkeletonIndex].Role, PoseHistory, Database);
					PresentRoleIndexes[RoledSkeletonIndex] = true;
				}
			}
			else
			{
				for (const FRole& Role : Availability.RolesFilter)
				{
					int32 RoledSkeletonIndex = 0;
					while (RoledSkeletonIndex < RoledSkeletons.Num())
					{
						if (RoledSkeletons[RoledSkeletonIndex].Role == Role)
					{
							InternalData.Emplace(Availability, AnimContextInfo, Role, PoseHistory, Database);
							PresentRoleIndexes[RoledSkeletonIndex] = true;
							break;
						}
						++RoledSkeletonIndex;
					}
#if !NO_LOGGING
					if (RoledSkeletonIndex == RoledSkeletons.Num())
					{
						UE_LOGF(LogPoseSearch, Warning, "UPoseSearchInteractionSubsystem::GenerateSearchContexts unsupported Role %ls for Database %ls", *Role.ToString(), *Database.GetName());
					}
#endif // !NO_LOGGING
				}
				}
			}

		void Reset()
		{
			InternalData.Reset();
			PresentRoleIndexes.Reset();
		}
	};

	// visits the FAnimContextInfos recursively to identify groups of nearby AnimContextInfo(s), relying on FAnimContextInfo::NearbyAnimContextInfos information.
	// it calls OnNewAnimAnimContextInfoFound on every new FAnimContextInfo found/visited, and OnDoneGroupingAnimContexts once reaches the end of the current group AnimContext(s).
	// it's then restart calling OnNewAnimAnimContextInfoFound in case there are still unvisited FAnimContextInfo(s), untill it visited ALL the FAnimContextInfo(s) in the input FAnimContextInfos
	struct FAnimContextInfoVisitor
	{
		FAnimContextInfoVisitor(
			const FAnimContextInfos& AnimContextInfos,
			TFunctionRef<void(const FAnimContextInfo&)> OnNewAnimAnimContextInfoFound,
			TFunctionRef<void()> OnDoneGroupingAnimContexts)
		{
			for (const FAnimContextInfo& AnimContextInfo : AnimContextInfos)
			{
				if (AnimContextInfo.GetNearbyAnimContextInfos().IsEmpty())
				{
					check(!VisitedAnimContextInfos.Find(&AnimContextInfo));
					// no need to add this context to the VisitedAnimContextInfos since it's isolated!
					OnNewAnimAnimContextInfoFound(AnimContextInfo);
					OnDoneGroupingAnimContexts();
				}
				else if(!VisitedAnimContextInfos.Find(&AnimContextInfo))
				{
					// starting the evaluation of a new set of grouped AnimContext(s)

					// processing the AnimContextsAvailabilities of the current AnimContextArray to fill up a map of Databases pointing to an array of all the AnimContexts with related roles
					VisitRecursively(AnimContextInfo, OnNewAnimAnimContextInfoFound);

					OnDoneGroupingAnimContexts();
				}
			}
		}

	private:
		void VisitRecursively(const FAnimContextInfo& AnimContextInfoToVisit, TFunctionRef<void(const FAnimContextInfo&)> OnNewAnimAnimContextInfoFound)
		{
			check(!AnimContextInfoToVisit.GetNearbyAnimContextInfos().IsEmpty());

			bool bIsAlreadyInSet;
			VisitedAnimContextInfos.FindOrAdd(&AnimContextInfoToVisit, &bIsAlreadyInSet);

			if (!bIsAlreadyInSet)
			{
				OnNewAnimAnimContextInfoFound(AnimContextInfoToVisit);

				for (const FAnimContextInfo* NearbyAnimContextInfo : AnimContextInfoToVisit.GetNearbyAnimContextInfos())
				{
					check(NearbyAnimContextInfo);
					VisitRecursively(*NearbyAnimContextInfo, OnNewAnimAnimContextInfoFound);
				}
			}
		}

		TSet<const FAnimContextInfo*, DefaultKeyFuncs<const FAnimContextInfo*>, TInlineSetAllocator<32, TMemStackSetAllocator<>>> VisitedAnimContextInfos;
	};

	// caching AnimContext(s) locations, max broad phase radiuses (squared) and collect relations of possible interactions between AnimContext(s)
	// (stored in FAnimContextInfo::NearbyAnimContextInfos::Index) as fast broad phase evaluation refined later on during FInteractionSearchContexts generation
	// and generating a mapping between availabilities Tag(s) to availabilities published databases
	FAnimContextInfos AnimContextInfos;
	FTagToDatabases TagToDatabases;
	GenerateAnimContextInfosAndTagToDatabases(AnimContextInfos, TagToDatabases);

	struct FDatabaseToRoledAnimContextInfos
	{
	private:
		typedef TMap<const UPoseSearchDatabase*, FRoledAnimContextInfos, TInlineSetAllocator<PreallocatedSearchesNum, TMemStackSetAllocator<>>> FInternalMap;
		FInternalMap InternalMap;

	public:
		typedef FInternalMap::TIterator TIterator;

		void AddRoledAnimContextInfos(const FInteractionAvailability& Availability, const FAnimContextInfo& AnimContextInfo, const IPoseHistory* PoseHistory, const UPoseSearchDatabase* Database)
		{
			check(Database && Database->Schema);
			FRoledAnimContextInfos& RoledAnimContextInfos = InternalMap.FindOrAdd(Database);
			RoledAnimContextInfos.AddRoledAnimContextInfos(Availability, AnimContextInfo, PoseHistory, *Database);
		}

		TIterator CreateIterator()
		{
			return InternalMap.CreateIterator();
		}

		void Reset()
		{
			// preserving the FRoledAnimContextInfos allocations, since reenumerating the few databases we have around with empty InternalMap is quite fast
			for (TPair<const UPoseSearchDatabase*, FRoledAnimContextInfos>& Pair : InternalMap)
			{
				Pair.Value.Reset();
			}
		}

	} DatabaseToRoledAnimContextInfos;

	// visiting ALL the AnimContexts in AnimContextInfos and relying on the cached information to refine potential interactions
	FAnimContextInfoVisitor AnimContextInfoVisitor(AnimContextInfos,
		// OnNewAnimContextFound: called when the FAnimContextInfoVisitor finds a new AnimContext that can be grouped in the current group of possibly interacting AnimContext(s)
		[&TagToDatabases, &DatabaseToRoledAnimContextInfos](const FAnimContextInfo& AnimContextInfo)
		{
			// analyzing all the Availability(s) associated with this AnimContext and eventually generate the associated FRoledAnimContextInfos,
			// inserted in a per database sorted data structure (DatabaseToRoledAnimContextInfos)
			for (const FInteractionAvailability& Availability : AnimContextInfo.GetAvailabilities())
			{
				if (const IPoseHistory* PoseHistory = Availability.GetPoseHistory(AnimContextInfo.GetAnimContext()))
				{
					if (const UPoseSearchDatabase* Database = Availability.Database.Get())
					{
						check(Database->Schema);
						DatabaseToRoledAnimContextInfos.AddRoledAnimContextInfos(Availability, AnimContextInfo, PoseHistory, Database);
					}
					else if (Availability.IsTagValid())
					{
						// since Database is null, but this availability has a valid Tag, we're looking for valid databases by Availability.Tag
						if (const FDatabasesPerTag* DatabasesPerTag = TagToDatabases.Find(Availability.Tag))
						{
							check(!DatabasesPerTag->IsEmpty());
							for (const UPoseSearchDatabase* DatabaseFromTag : *DatabasesPerTag)
							{
								DatabaseToRoledAnimContextInfos.AddRoledAnimContextInfos(Availability, AnimContextInfo, PoseHistory, DatabaseFromTag);
							}
						}
						else
						{
							//@todo: should we add a verbose LOG here? not sure since it'd be very spammy...

							// this is a valid condition we shouldn't log: for example when the "main character" is loaded and publishing availabilities with a valid tag and null database,
							// looking for other NPC / seconday characters to interact with, but they are not present of didn't publish any availability
						}
					}
					else
					{
						UE_LOGF(LogPoseSearch, Log, "UPoseSearchInteractionSubsystem::GenerateSearchContexts null Availability.Database (with invalid Availability.Tag)");
					}
				}
			}
		},
		// OnDoneGroupingAnimContexts: called when the FAnimContextInfoVisitor reaches the end of a group of possibly interacting AnimContext(s)
		[this, &DatabaseToRoledAnimContextInfos, &SearchContexts, DeltaSeconds]()
		{
			// for each database now we try to create all the possible combinations of the roled anim instances
			// for example, given a database set up with assets for 2 characters interactions with roles RoleA and RoleB
			// and 2 anim instances, all of them willing to partecipate in the 2 characters interaction with both roles RoleA and RoleB:
			// CharA could be taking RoleA and RoleB,
			// CharB could be taking RoleA and RoleB,
			// we generate all the combinations from the array of options:
			// CharA/RoleA, CharA/RoleB, CharB/RoleA, CharB/RoleB
			//
			// and we prune the invalid tuples:
			//
			// CharA/RoleA - CharA/RoleB -> invalid because of same CharA
			// CharA/RoleA - CharB/RoleA -> invalid because of same RoleA
			// CharA/RoleA - CharB/RoleB -> VALID!
			//
			// CharA/RoleB - CharB/RoleA -> VALID!
			// CharA/RoleB - CharB/RoleB -> invalid because of same RoleB
			//
			// CharB/RoleA - CharB/RoleB -> invalid because of same CharB

			for (FDatabaseToRoledAnimContextInfos::TIterator It = DatabaseToRoledAnimContextInfos.CreateIterator(); It; ++It)
			{
				FRoledAnimContextInfos& RoledAnimContextInfos = It->Value;
				if (RoledAnimContextInfos.HasAllRoles())
				{
					const UPoseSearchDatabase* Database = It->Key;
					check(Database->Schema);
					const TArray<FPoseSearchRoledSkeleton>& RoledSkeletons = Database->Schema->GetRoledSkeletons();
					const int32 CombinationCardinality = RoledSkeletons.Num();

					// sort RoledAnimContextInfos to generate deterministic SearchContext across multiple frames!
					RoledAnimContextInfos.SortAndRemoveDuplicates();

					GenerateCombinations(RoledAnimContextInfos.Num(), CombinationCardinality,
						// Combination is an array of indexes in RoledAnimContextInfos: 0 <= Combination[i] < RoledAnimContextInfos.Num()
						[this, Database, &RoledSkeletons, &RoledAnimContextInfos, &SearchContexts, DeltaSeconds](const TConstArrayView<int32> Combination)
						{
							// CombinationCardinality represents the number of roles as well as the number interacting AnimContext(s) (ultimately number of Characters involved in the interaction)
							const int32 CombinationCardinality = Combination.Num();

							for (int32 CombinationIndex = 0; CombinationIndex < CombinationCardinality; ++CombinationIndex)
							{
								const FRoledAnimContextInfo& RoledAnimContextInfo = RoledAnimContextInfos[Combination[CombinationIndex]];

								// since CombinationCardinality is usually small 2-3, it's faster to check linearly for duplicated animcontexts rather than holding them in a set
								const UObject* CombinationIndexAnimContext = RoledAnimContextInfo.AnimContextInfo->GetAnimContext();
								for (int32 NextCombinationIndex = CombinationIndex + 1; NextCombinationIndex < CombinationCardinality; ++NextCombinationIndex)
								{
									const FRoledAnimContextInfo& NextRoledAnimContextInfo = RoledAnimContextInfos[Combination[NextCombinationIndex]];
									const UObject* NextCombinationIndexAnimContext = NextRoledAnimContextInfo.AnimContextInfo->GetAnimContext();

									if (CombinationIndexAnimContext == NextCombinationIndexAnimContext)
									{
										// we have a duplicate AnimContext. this combination is NOT valid
										return false;
									}
								}
							}

							FInteractionSearchContext SearchContext;
							SearchContext.SetDatabase(Database);

							// setting up a FRoledAnimContextInfo in RoledAnimContextInfos describing 
							// this potential interaction properties about how to perform the search
							for (int32 CombinationIndex = 0; CombinationIndex < CombinationCardinality; ++CombinationIndex)
							{
								const int32 RoledAnimContextIndex = Combination[CombinationIndex];
								const FRoledAnimContextInfo& RoledAnimContextInfo = RoledAnimContextInfos[RoledAnimContextIndex];

								check(RoledAnimContextInfo.PoseHistory && RoledAnimContextInfo.Availability);
								SearchContext.Add(RoledAnimContextInfo.AnimContextInfo->GetAnimContext(), RoledAnimContextInfo.PoseHistory, RoledAnimContextInfo.Role
									, RoledAnimContextInfo.Availability->bDisableCollisions, RoledAnimContextInfo.Availability->TickPriority
#if ENABLE_ANIM_DEBUG
									, RoledAnimContextInfo.Availability->BroadPhaseRadius, RoledAnimContextInfo.Availability->BroadPhaseRadiusIncrementOnInteraction
#endif // ENABLE_ANIM_DEBUG
								);
							}

							// does SearchContext cover all the roles required by this interaction?
							for (const FPoseSearchRoledSkeleton& RoledSkeleton : RoledSkeletons)
							{
								// CombinationCardinality is usually 2-3, so we can search the SearchContext.Roles array for duplicates without requiring a faster container like TSet
								if (!SearchContext.GetRoles().Contains(RoledSkeleton.Role))
								{
									return false;
								}
							}

							// looking for a preexisting valid interaction resembling SearchContext
							SearchContext.CacheHashesForEquivalence();

							// looking for a previous valid interaction with the SearchContext properties in ValidInteractionSearches 
							// (it's using SearchContext.CacheHashesForEquivalence to speed up the search)
							const FValidInteractionSearch* ValidInteractionSearch = ValidInteractionSearches.Get(SearchContext);
							const bool bInIsContinuingInteraction = ValidInteractionSearch != nullptr;

							SearchContext.SetContinuingInteraction(bInIsContinuingInteraction);

							// checking if this combination is valid for the Database:
							for (int32 CombinationIndex = 0; CombinationIndex < CombinationCardinality; ++CombinationIndex)
							{
								const int32 RoledAnimContextIndex = Combination[CombinationIndex];
								const FRoledAnimContextInfo& RoledAnimContextInfo = RoledAnimContextInfos[RoledAnimContextIndex];

								// checking the narrow phase! if RoledAnimContextInfo can interact with OtherRoledAnimContextInfo
								for (int32 OtherCombinationIndex = CombinationIndex + 1; OtherCombinationIndex < CombinationCardinality; ++OtherCombinationIndex)
								{
									const int32 OtherRoledAnimContextIndex = Combination[OtherCombinationIndex];
									const FRoledAnimContextInfo& OtherRoledAnimContextInfo = RoledAnimContextInfos[OtherRoledAnimContextIndex];

									const FVector DeltaLocation = RoledAnimContextInfo.AnimContextInfo->GetLocation() - OtherRoledAnimContextInfo.AnimContextInfo->GetLocation();
									const float DistanceSquared = DeltaLocation.SquaredLength();

									float MaxDistanceSquared;
									if (bInIsContinuingInteraction)
									{
										MaxDistanceSquared = FMath::Min(RoledAnimContextInfo.IncrementedBroadPhaseRadiusSquared, OtherRoledAnimContextInfo.IncrementedBroadPhaseRadiusSquared);
									}
									else
									{
										MaxDistanceSquared = FMath::Min(RoledAnimContextInfo.BroadPhaseRadiusSquared, OtherRoledAnimContextInfo.BroadPhaseRadiusSquared);
									}

									// if any of the RoledAnimContextInfo cannot interact with any OtherRoledAnimContextInfo the inteaction cannot happen!
									if (DistanceSquared > MaxDistanceSquared)
									{
										return false;
									}
								}
							}

							// Populating Continuing properties by using the ValidInteractionSearch->SearchResult
							if (bInIsContinuingInteraction)
							{
								const FInteractionSearchResult& PreviousSearchResult = ValidInteractionSearch->GetSearchResult();
								if (const UE::PoseSearch::FSearchIndexAsset* SearchIndexAsset = PreviousSearchResult.GetSearchIndexAsset())
								{
									if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAsset = PreviousSearchResult.Database->GetDatabaseAnimationAsset(SearchIndexAsset->GetSourceAssetIdx()))
									{
										check(SearchIndexAsset->GetToRealTimeFactor() > UE_KINDA_SMALL_NUMBER);
										// in case DatabaseAsset->GetAnimationAsset() is a blendspace, PreviousSearchResult.AssetTime is a normalized time in the interval [0,1] 
										// so we need to convert the delta time in seconds to the asset normalized time before integrating PreviousSearchResult.AssetTime

										const float NormalizedDeltaTime = DeltaSeconds / SearchIndexAsset->GetToRealTimeFactor();
										const float PlayingAssetAccumulatedTime = PreviousSearchResult.GetAssetTime() + NormalizedDeltaTime;

										// @todo: populate SearchContext.InterruptMode
										EPoseSearchInterruptMode InterruptMode = EPoseSearchInterruptMode::DoNotInterrupt;
										SearchContext.SetContinuingProperties(PlayingAssetAccumulatedTime, DatabaseAsset->GetAnimationAsset(),
											SearchIndexAsset->IsMirrored(), SearchIndexAsset->GetBlendParameters(), InterruptMode);
									}
								}

								check(ValidInteractionSearches.IsThereAnyValidSearchWithTheSameContexts(SearchContext));
								SearchContext.SetContinuingContextInteraction(true);
							}
							else
							{
								const bool bIsThereAnyValidSearchWithTheSameContexts = ValidInteractionSearches.IsThereAnyValidSearchWithTheSameContexts(SearchContext);
								SearchContext.SetContinuingContextInteraction(bIsThereAnyValidSearchWithTheSameContexts);
							}

							SearchContexts.Emplace(SearchContext);
							return true;
						});
				}
			}

			// done using DatabaseToRoledAnimContextInfos. clearing up for the next group of AnimContext(s)
			DatabaseToRoledAnimContextInfos.Reset();
		});
}

bool UPoseSearchInteractionSubsystem::ShouldRegenerateAllIslands()
{
	using namespace UE::PoseSearch;

	const int32 AnimContextsAvailabilitiesIndexValue = AnimContextsAvailabilitiesIndex.GetValue();
	AnimContextsAvailabilitiesNum = AnimContextsAvailabilitiesBuffer.Num();

	if (AnimContextsAvailabilitiesIndexValue > AnimContextsAvailabilitiesNum)
	{
		UE_LOGF(LogPoseSearch, Warning, "UPoseSearchInteractionSubsystem::ShouldRegenerateAllIslands not enough space to add more availabilities locklessly. It'll be adjusted automatically, but some availability requests has been lost this frame [capacity %d / requests %d]. To void this warning set the initial budget for 'Edit/ProjectSettings/Plugins/Pose Search/Availabilities Buffer Size' to %d. ", AnimContextsAvailabilitiesNum, AnimContextsAvailabilitiesIndexValue, AnimContextsAvailabilitiesIndexValue);
		AnimContextsAvailabilitiesBuffer.SetNum(AnimContextsAvailabilitiesIndexValue);
	}
	else
	{
		AnimContextsAvailabilitiesNum = AnimContextsAvailabilitiesIndexValue;
	}

	if (AnimContextsAvailabilitiesNum <= 0)
	{
		for (FInteractionIsland* Island : Islands)
		{
			if (Island->IsInitialized())
			{
				return true;
			}
		}

		// nothing to do. returning false for a subsequent early out in UPoseSearchInteractionSubsystem::Tick
		return false;
	}
	
	return true;
}

void UPoseSearchInteractionSubsystem::Tick(float DeltaSeconds)
{
	using namespace UE::PoseSearch;

	QUICK_SCOPE_CYCLE_COUNTER(STAT_UPoseSearchInteractionSubsystem_Tick);

	check(IsInGameThread());

	// If we're in an AutoRTFM Transaction, exit the TickableObject Tick (which is top-level) and run again outside the transaction
	// This is temporary because AutoRTFM does not have a ThreadSafeCounter; we will eventually need to solve this as we do derference
	// UObjects inside this function.
	if (AutoRTFM::IsTransactional())
	{
		AutoRTFM::OnCommit([WeakThis = TWeakObjectPtr(this), DeltaSeconds]()
		{
			if (WeakThis.IsValid())
			{
				WeakThis->Tick(DeltaSeconds);
			}
		});

		return;
	}

	Super::Tick(DeltaSeconds);

	FMemMark Mark(FMemStack::Get());
	ValidInteractionSearches.Update(this);

	if (ShouldRegenerateAllIslands())
	{
		RegenerateAllIslands(DeltaSeconds);

#if DO_CHECK
		check(ValidateAllIslands());
#endif
	}

	AnimContextsAvailabilitiesIndex.Reset();
}

TStatId UPoseSearchInteractionSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UPoseSearchInteractionSubsystem, STATGROUP_Tickables);
}

void UPoseSearchInteractionSubsystem::Query_AnyThread(const TArrayView<const FPoseSearchInteractionAvailability> Availabilities, const UObject* AnimContext, 
	FPoseSearchBlueprintResult& Result, FName PoseHistoryName, const UE::PoseSearch::IPoseHistory* PoseHistory, bool bValidateResultAgainstAvailabilities)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UPoseSearchInteractionSubsystem_Query_AnyThread);

	using namespace UE::PoseSearch;

	Result = FPoseSearchBlueprintResult();

#if !NO_CVARS
	if (!GVarPoseSearchInteractionEnabled)
	{
		return;
	}
#endif // !NO_CVARS

	// if we find AnimContext in an island, we perform ALL the Island motion matching searches.
	if (FInteractionIsland* Island = FindIsland(AnimContext))
	{
		Island->DoSearch_AnyThread(AnimContext, Result);

		if (bValidateResultAgainstAvailabilities && Result.SelectedAnim)
		{
			bool bResultValidated = false;

			for (const FPoseSearchInteractionAvailability& Availability : Availabilities)
			{
				const bool bIsDatabaseValidates = (Availability.IsTagValid() && !Availability.Database) || (Availability.Database == Result.SelectedDatabase);
				if (bIsDatabaseValidates &&	(Availability.RolesFilter.IsEmpty() || Availability.RolesFilter.Contains(Result.Role)))
				{
					bResultValidated = true;
					break;
				}
			}

			if (!bResultValidated)
			{
				Result = FPoseSearchBlueprintResult();
			}
		}
	}

	// queuing the availabilities for the next frame Query_AnyThread
	AddAvailabilities(Availabilities, AnimContext, PoseHistoryName, PoseHistory);
}

void UPoseSearchInteractionSubsystem::GetResult_AnyThread(const UObject* AnimContext, FPoseSearchBlueprintResult& Result, bool bCompareOwningActors)
{
	using namespace UE::PoseSearch;

	if (FInteractionIsland* Island = FindIsland(AnimContext, bCompareOwningActors))
	{
		Island->GetResult_AnyThread(AnimContext, Result, bCompareOwningActors);
	}
	else
	{
		Result = FPoseSearchBlueprintResult();
	}
}

const TConstArrayView<FPoseSearchConstraint> UPoseSearchInteractionSubsystem::GetConstraints(const UE::PoseSearch::FInteractionSearchContext& SearchContext) const
{
	using namespace UE::PoseSearch;

	if (const FValidInteractionSearch* ValidInteractionSearch = ValidInteractionSearches.Get(SearchContext))
	{
		return ValidInteractionSearch->GetSearchResult().Constraints;
	}
	return TConstArrayView<FPoseSearchConstraint>();
}

bool UPoseSearchInteractionSubsystem::GetConstraint(const UObject* AnimContext, FName SocketName, float& OutDesiredReach, FTransform& OutTransform, bool bCompareOwningActors)
{
	using namespace UE::PoseSearch;

	if (const FInteractionIsland* Island = FindIsland(AnimContext, bCompareOwningActors))
	{
		return Island->GetConstraint(AnimContext, SocketName, OutDesiredReach, OutTransform, bCompareOwningActors);
	}
	
	OutDesiredReach = 0.f;
	OutTransform = FTransform::Identity;
	return false;
}

void UPoseSearchInteractionSubsystem::AddReferencedObjects(UObject* This, FReferenceCollector& Collector)
{
	using namespace UE::PoseSearch;

	Super::AddReferencedObjects(This, Collector);

	UPoseSearchInteractionSubsystem* Subsystem = CastChecked<UPoseSearchInteractionSubsystem>(This);

	for (FInteractionAnimContextAvailabilities& AnimContextAvailabilities : Subsystem->AnimContextsAvailabilitiesBuffer)
	{
		Collector.AddReferencedObject(AnimContextAvailabilities.AnimContext);
		for (FInteractionAvailability& Availability : AnimContextAvailabilities.Availabilities)
		{
			Collector.AddReferencedObject(Availability.Database);
		}
	}

	for (FInteractionIsland* Island : Subsystem->Islands)
	{
		Island->AddReferencedObjects(Collector);
	}

	Subsystem->ValidInteractionSearches.AddReferencedObjects(Collector);
}

#if ENABLE_ANIM_DEBUG
void UPoseSearchInteractionSubsystem::DebugDrawIslands() const
{
#if ENABLE_VISUAL_LOG

	using namespace UE::PoseSearch;

	check(IsInGameThread());

	const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(PoseSearchChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	static const FColor Colors[] =
	{
		FColor::White,
		FColor::Black,
		FColor::Red,
		FColor::Green,
		FColor::Blue,
		FColor::Yellow,
		FColor::Cyan,
		FColor::Magenta,
		FColor::Orange,
		FColor::Purple,
		FColor::Turquoise,
		FColor::Silver,
		FColor::Emerald
	};
	static const int32 NumColors = sizeof(Colors) / sizeof(Colors[0]);
	int32 CurrentColorIndex = 0;

	TArray<const UObject*, TInlineAllocator<256>> AllAnimContexts;
	for (const FInteractionIsland* Island : Islands)
	{
		for (const UObject* IslandAnimContext : Island->GetIslandAnimContexts())
		{
			if (IslandAnimContext)
			{
				AllAnimContexts.Add(IslandAnimContext);
			}
		}
	}

	for (const FInteractionIsland* Island : Islands)
	{
		if (Island->IsInitialized())
		{
			const FColor& Color = Colors[CurrentColorIndex];

			for (const FInteractionSearchContext& SearchContext : Island->GetSearchContexts())
			{
				for (int32 Index = 0; Index < SearchContext.Num(); ++Index)
				{
					if (const UObject* AnimContext = SearchContext.GetAnimContext(Index))
					{
						const float MaxBroadPhaseRadius = SearchContext.GetDebugBroadPhaseRadius(Index);
						if (MaxBroadPhaseRadius > UE_SMALL_NUMBER)
						{
							const FTransform& Transform = GetContextTransform(AnimContext);
							static const TCHAR* LogName = TEXT("PoseSearchInteraction");
								
							for (const UObject* IslandAnimContext : AllAnimContexts)
							{
								UE_VLOG_CIRCLE(IslandAnimContext, LogName, Display, Transform.GetLocation(), FVector::UpVector, MaxBroadPhaseRadius, Color, TEXT(""));
							}

							if (!Island->HasTickDependencies())
							{
								const FVector ForwardAxisStart = Transform.TransformPosition(FVector::ForwardVector * MaxBroadPhaseRadius);
								const FVector ForwardAxisEnd = Transform.TransformPosition(FVector::ForwardVector * -MaxBroadPhaseRadius);

								const FVector LeftAxisStart = Transform.TransformPosition(FVector::LeftVector * MaxBroadPhaseRadius);
								const FVector LeftAxisEnd = Transform.TransformPosition(FVector::LeftVector * -MaxBroadPhaseRadius);

								for (const UObject* IslandAnimContext : AllAnimContexts)
								{
									UE_VLOG_SEGMENT(IslandAnimContext, LogName, Display, ForwardAxisStart, ForwardAxisEnd, Color, TEXT(""));
									UE_VLOG_SEGMENT(IslandAnimContext, LogName, Display, LeftAxisStart, LeftAxisEnd, Color, TEXT(""));
								}
							}
						}
					}
				}
			}

			CurrentColorIndex = (CurrentColorIndex + 1) % NumColors;
		}
	}
#endif // ENABLE_VISUAL_LOG
}

void UPoseSearchInteractionSubsystem::DebugLogTickDependencies() const
{
#if !NO_CVARS
	using namespace UE::PoseSearch;

	if (GVarPoseSearchInteractionLoglandsTickDependencies)
	{
		UE_LOGF(LogPoseSearch, Log, "==================================================================");
		for (const FInteractionIsland* Island : Islands)
		{
			if (Island->IsInitialized())
			{
				Island->LogTickDependencies();
			}
		}
	}
#endif // !NO_CVARS
}
#endif // ENABLE_ANIM_DEBUG
