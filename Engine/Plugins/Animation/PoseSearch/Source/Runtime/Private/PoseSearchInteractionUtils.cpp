// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchInteractionUtils.h"
#include "PoseSearch/MultiAnimAsset.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchInteractionLibrary.h"
#include "PoseSearch/PoseSearchSchema.h"

namespace UE::PoseSearch
{
	
int32 GetRoleIndex(const UMultiAnimAsset* MultiAnimAsset, const FRole& Role)
{
	check(MultiAnimAsset);
	const int32 NumRoles = MultiAnimAsset->GetNumRoles();
	for (int32 MultiAnimAssetRoleIndex = 0; MultiAnimAssetRoleIndex < NumRoles; ++MultiAnimAssetRoleIndex)
	{
		if (MultiAnimAsset->GetRole(MultiAnimAssetRoleIndex) == Role)
		{
			return MultiAnimAssetRoleIndex;
		}
	}
	return INDEX_NONE;
}

FRoleToIndex MakeRoleToIndex(const UMultiAnimAsset* MultiAnimAsset)
{
	check(MultiAnimAsset);
	const int32 NumRoles = MultiAnimAsset->GetNumRoles();

	FRoleToIndex RoleToIndex;
	RoleToIndex.Reserve(NumRoles);
	for (int32 MultiAnimAssetRoleIndex = 0; MultiAnimAssetRoleIndex < NumRoles; ++MultiAnimAssetRoleIndex)
	{
		RoleToIndex.Add(MultiAnimAsset->GetRole(MultiAnimAssetRoleIndex)) = MultiAnimAssetRoleIndex;
	}
	return RoleToIndex;
}

void CalculateFullAlignedTransforms(const FPoseSearchBlueprintResult& Result, float Time, float TimeOffset, bool bWarpUsingRootBone, TArrayView<FTransform> OutFullAlignedTransforms)
{
	const UMultiAnimAsset* MultiAnimAsset = CastChecked<UMultiAnimAsset>(Result.SelectedAnim);
	const int32 NumRoles = MultiAnimAsset->GetNumRoles();
	
	check(OutFullAlignedTransforms.Num() == NumRoles);
	check(Result.ActorRootTransforms.Num() == Result.ActorRootBoneTransforms.Num());
	check(Result.ActorRootTransforms.Num() == Result.AnimContexts.Num());
	check(Result.SelectedDatabase != nullptr && Result.SelectedDatabase->Schema != nullptr);

	TConstArrayView<FTransform> ActorTransforms = Result.ActorRootTransforms;
	
	// if bWarpUsingRootBone we set ActorTransforms as the actors root bone world transforms instead of the root transforms
	TArray<FTransform, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> WorldActorRootBoneTransforms;
	if (bWarpUsingRootBone)
	{
		WorldActorRootBoneTransforms.SetNum(NumRoles);
		for (int32 Index = 0; Index < NumRoles; ++Index)
		{
			WorldActorRootBoneTransforms[Index] = Result.ActorRootBoneTransforms[Index] * Result.ActorRootTransforms[Index];
		}

		ActorTransforms = WorldActorRootBoneTransforms;
	}

	TArray<const UMirrorDataTable*, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> MirrorDataTables;
	if (Result.bIsMirrored)
	{
		MirrorDataTables.SetNum(NumRoles);
		for (int32 MultiAnimAssetRoleIndex = 0; MultiAnimAssetRoleIndex < NumRoles; ++MultiAnimAssetRoleIndex)
		{
			const FRole Role = MultiAnimAsset->GetRole(MultiAnimAssetRoleIndex);
			const FPoseSearchRoledSkeleton* RoledSkeleton = Result.SelectedDatabase->Schema->GetRoledSkeleton(Role);
			check(RoledSkeleton);
			MirrorDataTables[MultiAnimAssetRoleIndex] = RoledSkeleton->MirrorDataTable;
		}
	}

	MultiAnimAsset->CalculateWarpTransforms(Time, TimeOffset, ActorTransforms, OutFullAlignedTransforms, MirrorDataTables);
}

FTransform CalculateDeltaAlignment(const FTransform& MeshWithoutOffset, const FTransform& MeshWithOffset, const FTransform& FullAlignedTransform, float WarpingRotationRatio, float WarpingTranslationRatio)
{
	// calculating the NoDeltaAlignment as the delta transform that brings the actor to original mesh transform.
	const FTransform NoDeltaAlignment = MeshWithoutOffset.GetRelativeTransform(MeshWithOffset);

	// calculating the FullDeltaAlignment as the delta transform that brings the actor to its full aligned transform.
	const FTransform FullDeltaAlignment = FullAlignedTransform.GetRelativeTransform(MeshWithOffset);

	// calculating the DeltaAlignment as blend between the NoDeltaAlignment and the FullDeltaAlignment: how much the character need to move to get to the desired alignment
	const FTransform DeltaAlignment(FMath::Lerp(NoDeltaAlignment.GetRotation(), FullDeltaAlignment.GetRotation(), FMath::Clamp(WarpingRotationRatio, 0.f, 1.f)),
		FMath::Lerp(NoDeltaAlignment.GetTranslation(), FullDeltaAlignment.GetTranslation(), FMath::Clamp(WarpingTranslationRatio, 0.f, 1.f)), FVector::OneVector);

	// NoTe: keep in mind MeshWithoutOffset, MeshWithOffset, and FullAlignedTransform are relative to the previous execution frame so we still need to 
	//		 extract and and apply the current animation root motion transform to get to the current frame full aligned transform.
	return DeltaAlignment;
}

} // UE::PoseSearch
