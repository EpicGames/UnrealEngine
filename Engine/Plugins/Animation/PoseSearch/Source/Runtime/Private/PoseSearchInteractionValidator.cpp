// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchInteractionValidator.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchInteractionSubsystem.h"
#include "GameFramework/Actor.h"

namespace UE::PoseSearch
{
#if ENABLE_ANIM_DEBUG

FInteractionValidator::FInteractionValidator(const UObject* AnimContext)
	: ValidatingAnimContext(AnimContext)
	, ValidatingIsland(nullptr)
{
	check(AnimContext);
	UPoseSearchInteractionSubsystem* InteractionSubsystem = UPoseSearchInteractionSubsystem::GetSubsystem_AnyThread(AnimContext);
	if (!InteractionSubsystem)
	{
		return;
	}

	ValidatingIsland = InteractionSubsystem->FindIsland(AnimContext);
	if (!ValidatingIsland)
	{
		return;
	}

	const AActor* Actor = GetContextOwningActor(AnimContext, false);
	check(Actor);

	const AActor* MainActor = ValidatingIsland->GetMainActor();
	if (!MainActor)
	{
		UE_LOGF(LogPoseSearch, Error, "FInteractionValidator invalid MainActor! How did he die? Did you rebuild the animation blue print while PIE was running?");
		ValidatingIsland->LogTickDependencies();
		return;
	}

	if (Actor == MainActor)
	{
		const int32 PreviousCounter = ValidatingIsland->InteractionIslandThreadSafeCounter.Set(-1);
		if (PreviousCounter != 0)
		{
			UE_LOGF(LogPoseSearch, Error, "Non thread safe call! Why is there any other actor running while we schedule the MainActor (%ls)? %d", *MainActor->GetName(), PreviousCounter);
			ValidatingIsland->LogTickDependencies();
		}
	}
	else
	{
		// not the main actor
		const int32 PreviousCounter = ValidatingIsland->InteractionIslandThreadSafeCounter.Add(1);
		if (PreviousCounter < 0)
		{
			UE_LOGF(LogPoseSearch, Error, "Non thread safe call! The MainActor (%ls) is running! Nobody else (%ls) in the same island should be running! %d", *MainActor->GetName(), *Actor->GetName(),  PreviousCounter);
			ValidatingIsland->LogTickDependencies();
		}
	}
}

FInteractionValidator::FInteractionValidator(FInteractionIsland* InValidatingIsland)
	: ValidatingAnimContext(nullptr)
	, ValidatingIsland(InValidatingIsland)
{
	const int32 PreviousCounter = ValidatingIsland->TickFunctionsThreadSafeCounter.Set(-1);
	if (PreviousCounter != 0)
	{
		UE_LOGF(LogPoseSearch, Error, "Non thread safe call! TickFunctions running concurrently? %d", PreviousCounter);
		ValidatingIsland->LogTickDependencies();
	}
}

FInteractionValidator::~FInteractionValidator()
{
	if (!ValidatingAnimContext)
	{
		check(ValidatingIsland);
		const int32 PreviousCounter = ValidatingIsland->TickFunctionsThreadSafeCounter.Set(0);
		if (PreviousCounter != -1)
		{
			UE_LOGF(LogPoseSearch, Error, "Non thread safe call! TickFunctions running concurrently? %d", PreviousCounter);
		}
		return;
	}

	if (!ValidatingIsland)
	{
		return;
	}

	UPoseSearchInteractionSubsystem* InteractionSubsystem = UPoseSearchInteractionSubsystem::GetSubsystem_AnyThread(ValidatingAnimContext);
	if (!InteractionSubsystem)
	{
		return;
	}

	FInteractionIsland* Island = InteractionSubsystem->FindIsland(ValidatingAnimContext);
	if (Island != ValidatingIsland)
	{
		UE_LOGF(LogPoseSearch, Error, "FInteractionValidator why did the InteractionIsland changed?");
		return;
	}

	const AActor* Actor = GetContextOwningActor(ValidatingAnimContext, false);
	check(Actor);

	const AActor* MainActor = Island->GetMainActor();
	if (!MainActor)
	{
		UE_LOGF(LogPoseSearch, Error, "FInteractionValidator invalid MainActor! How did he died? Did you rebuild the animation blue print while PIE was running?");
		return;
	}
	
	if (Actor == MainActor)
	{
		const int32 PreviousCounter = Island->InteractionIslandThreadSafeCounter.Set(0);
		if (PreviousCounter >= 0)
		{
			UE_LOGF(LogPoseSearch, Error, "Non thread safe call! Why was there some other actor running while we ended the scheduling the MainActor (%ls)? %d", *MainActor->GetName(), PreviousCounter);
		}
	}
	else
	{
		// not the main actor
		const int32 PreviousCounter = Island->InteractionIslandThreadSafeCounter.Add(-1);
		if (PreviousCounter <= 0)
		{
			UE_LOGF(LogPoseSearch, Error, "Non thread safe call! The MainActor (%ls) is running! Nobody else (%ls) in the same island should be running! %d", *MainActor->GetName(), *Actor->GetName(), PreviousCounter);
		}
	}
}

#endif // ENABLE_ANIM_DEBUG

} // namespace UE::PoseSearch