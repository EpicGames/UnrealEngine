// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchLibrary.h"

#if UE_POSE_SEARCH_TRACE_ENABLED
#include "ObjectTrace.h"
#endif
#include "Animation/AnimationAsset.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimNode_SequencePlayer.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimSubsystem_Tag.h"
#include "Animation/AnimTrace.h"
#include "Animation/BlendSpace.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/Character.h"
#include "PoseSearch/AnimNode_MotionMatching.h"
#include "PoseSearch/AnimNode_PoseSearchHistoryCollector.h"
#include "PoseSearch/MultiAnimAsset.h"
#include "PoseSearch/PoseSearchAnimNotifies.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchDerivedData.h"
#include "PoseSearch/PoseSearchFeatureChannel_PermutationTime.h"
#include "PoseSearch/PoseSearchFeatureChannel_Trajectory.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearch/Trace/PoseSearchTraceLogger.h"
#include "UObject/FastReferenceCollector.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchLibrary)

#define LOCTEXT_NAMESPACE "PoseSearchLibrary"

#if ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
static bool GVarAnimMotionMatchDrawQueryEnable = false;
static FAutoConsoleVariableRef CVarAnimMotionMatchDrawQueryEnable(TEXT("a.MotionMatch.DrawQuery.Enable"), GVarAnimMotionMatchDrawQueryEnable, TEXT("Enable / Disable MotionMatch Draw Query"));

static bool GVarAnimMotionMatchDrawMatchEnable = false;
static FAutoConsoleVariableRef CVarAnimMotionMatchDrawMatchEnable(TEXT("a.MotionMatch.DrawMatch.Enable"), GVarAnimMotionMatchDrawMatchEnable, TEXT("Enable / Disable MotionMatch Draw Match"));
#endif

namespace UE::PoseSearch
{
	// an empty FStackAssetSet in any of the FAssetsToSearchPerDatabasePair entries means we need to search ALL the assets for the associated Database
	typedef TPair<const UPoseSearchDatabase*, FStackAssetSet> FAssetsToSearchPerDatabasePair;
	struct FAssetsToSearchPerDatabase : public TArray<FAssetsToSearchPerDatabasePair, TInlineAllocator<8, TMemStackAllocator<>>>
	{
		FAssetsToSearchPerDatabasePair* Find(const UPoseSearchDatabase* Database)
		{
			return FindByPredicate([Database](const FAssetsToSearchPerDatabasePair& Pair) { return Pair.Key == Database; });
		}

		bool Contains(const UPoseSearchDatabase* Database) const
		{
			return nullptr != FindByPredicate([Database](const FAssetsToSearchPerDatabasePair& Pair) { return Pair.Key == Database; });
		}
	};
	
	// used to cache the continuing pose search results
	struct FCachedContinuingPoseSearchResults : public TMap<const UObject*, FSearchResult, TInlineSetAllocator<16, TMemStackSetAllocator<>>>
	{
		void CheckedAdd(const UObject* Object, const FSearchResult& SearchResult)
		{
			check(Object);
			check(!Find(Object));
			check(SearchResult.IsValid());

			FSearchResult& NewSearchResult = Add(Object);
			NewSearchResult = SearchResult;
		}

		const FSearchResult& FindOrAdd(const UObject* Object, const FSearchResult& SearchResult)
		{
			check(Object);
			check(SearchResult.IsValid());

			if (const FSearchResult* FoundSearchResult = Find(Object))
			{
				return *FoundSearchResult;
			}

			FSearchResult& NewSearchResult = Add(Object);
			NewSearchResult = SearchResult;
			return NewSearchResult;
		}

		const FSearchResult& FindOrDefault(const UObject* Object) const
		{
			check(Object);
			if (const FSearchResult* FoundSearchResult = Find(Object))
			{
				return *FoundSearchResult;
			}

			static FSearchResult DefaultSearchResult;
			return DefaultSearchResult;
		}
	};

	static void FilterByAssetsToConsider(FSearchContext& SearchContext, FStackDatabaseToAssetIndexes& AssetIndexesToSearchPerDatabase)
	{
		// intersecting AssetsToSearchPerDatabase with asset to consider
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!SearchContext.GetInternalDeprecatedAssetsToConsider().IsEmpty())
		{
			// backward compatible path to support deprecated API FSearchContext::SetAssetsToConsider
			for (TPair<const UPoseSearchDatabase*, FStackAssetIndexes>& AssetIndexesToSearchPerDatabasePair : AssetIndexesToSearchPerDatabase)
			{
				const UPoseSearchDatabase* Database = AssetIndexesToSearchPerDatabasePair.Key;
				check(Database);

				FStackAssetIndexes& AssetIndexesToSearchForDatabase = AssetIndexesToSearchPerDatabasePair.Value;
				FStackAssetIndexes IntersectedAssetIndexesToSearchForDatabase;

				const FSearchIndex& SearchIndex = Database->GetSearchIndex();
				for (const UObject* AssetToConsider : SearchContext.GetInternalDeprecatedAssetsToConsider())
				{
					for (int32 DatabaseAnimationAssetIndex : Database->GetAssetIndexesForSourceAsset(AssetToConsider))
					{
						const FSearchIndexAsset& SearchIndexAsset = SearchIndex.Assets[DatabaseAnimationAssetIndex];
						const int32 SourceAssetIdx = SearchIndexAsset.GetSourceAssetIdx();
						if (AssetIndexesToSearchForDatabase.Find(SourceAssetIdx))
						{
							IntersectedAssetIndexesToSearchForDatabase.Add(SourceAssetIdx);
						}
					}
				}

				AssetIndexesToSearchForDatabase = IntersectedAssetIndexesToSearchForDatabase;
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		else if (const FStackAssetSet* AssetsToConsider = SearchContext.GetAssetsToConsiderSet())
		{
			if (!AssetsToConsider->IsEmpty())
			{
				for (TPair<const UPoseSearchDatabase*, FStackAssetIndexes>& AssetIndexesToSearchPerDatabasePair : AssetIndexesToSearchPerDatabase)
				{
					const UPoseSearchDatabase* Database = AssetIndexesToSearchPerDatabasePair.Key;
					check(Database);

					FStackAssetIndexes& AssetIndexesToSearchForDatabase = AssetIndexesToSearchPerDatabasePair.Value;
					FStackAssetIndexes IntersectedAssetIndexesToSearchForDatabase;

					const FSearchIndex& SearchIndex = Database->GetSearchIndex();
					for (const UObject* AssetToConsider : *AssetsToConsider)
					{
						for (int32 DatabaseAnimationAssetIndex : Database->GetAssetIndexesForSourceAsset(AssetToConsider))
						{
							const FSearchIndexAsset& SearchIndexAsset = SearchIndex.Assets[DatabaseAnimationAssetIndex];
							const int32 SourceAssetIdx = SearchIndexAsset.GetSourceAssetIdx();
							if (AssetIndexesToSearchForDatabase.Find(SourceAssetIdx))
							{
								IntersectedAssetIndexesToSearchForDatabase.Add(SourceAssetIdx);
							}
						}
					}

					AssetIndexesToSearchForDatabase = IntersectedAssetIndexesToSearchForDatabase;
				}
			}
		}
		else if (const FStackDatabaseToAssetIndexes* AssetIndexesToConsiderPerDatabase = SearchContext.GetAssetIndexesToConsiderPerDatabase())
		{
			for (TPair<const UPoseSearchDatabase*, FStackAssetIndexes>& AssetIndexesToSearchPerDatabasePair : AssetIndexesToSearchPerDatabase)
			{
				const UPoseSearchDatabase* Database = AssetIndexesToSearchPerDatabasePair.Key;
				check(Database);

				FStackAssetIndexes& AssetIndexesToSearchForDatabase = AssetIndexesToSearchPerDatabasePair.Value;
				FStackAssetIndexes IntersectedAssetIndexesToSearchForDatabase;

				if (const FStackAssetIndexes* AssetIndexesToConsider = AssetIndexesToConsiderPerDatabase->Find(Database))
				{
					for (int32 DatabaseAnimationAssetIndex : *AssetIndexesToConsider)
					{
						if (AssetIndexesToSearchForDatabase.Find(DatabaseAnimationAssetIndex))
						{
							check(DatabaseAnimationAssetIndex < Database->GetNumAnimationAssets());
							IntersectedAssetIndexesToSearchForDatabase.Add(DatabaseAnimationAssetIndex);
						}
					}
				}

				AssetIndexesToSearchForDatabase = IntersectedAssetIndexesToSearchForDatabase;
			}
		}
	}

	// this function adds AssetToSearch to the search of Database
	static void AddToSearchForDatabase(FStackDatabaseToAssetIndexes& AssetIndexesToSearchPerDatabase, const UObject* AssetToSearch, const UPoseSearchDatabase* Database, bool bContainsIsMandatory, FSearchContext& SearchContext)
	{
		// making sure AssetToSearch is not a databases! later on we could add support for nested databases, but currently we don't support that
		check(Cast<const UPoseSearchDatabase>(AssetToSearch) == nullptr);

#if WITH_EDITOR
		// no need to check if Database is indexing if found into AssetsToSearchPerDatabase, since it already passed RequestAsyncBuildIndex successfully in a previous AddToSearchForDatabase call
		if (EAsyncBuildIndexResult::Success != FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(Database, ERequestAsyncBuildFlag::ContinueRequest))
		{
			// database is still indexing... moving on
			SearchContext.SetAsyncBuildIndexInProgress();
		}
		else
#endif // WITH_EDITOR
		{
			const TConstArrayView<int32> AssetIndexesForSourceAssetToSearch = Database->GetAssetIndexesForSourceAsset(AssetToSearch);
			if (AssetIndexesForSourceAssetToSearch.IsEmpty())
			{
				if (bContainsIsMandatory)
				{
					UE_LOGF(LogPoseSearch, Error, "improperly setup UAnimSequenceBase. Database %ls doesn't contain UAnimSequenceBase %ls", *Database->GetName(), *AssetToSearch->GetName());
				}
			}
			else
			{
				const FSearchIndex& SearchIndex = Database->GetSearchIndex();
				FStackAssetIndexes& AssetIndexesToSearch = AssetIndexesToSearchPerDatabase.FindOrAdd(Database);
				AssetIndexesToSearch.Reserve(AssetIndexesToSearch.Num() + AssetIndexesForSourceAssetToSearch.Num());
				for (int32 AssetIndexToSearch : AssetIndexesForSourceAssetToSearch)
				{
					const FSearchIndexAsset& SearchIndexAsset = SearchIndex.Assets[AssetIndexToSearch];
					AssetIndexesToSearch.Add(SearchIndexAsset.GetSourceAssetIdx());
				}
			}
		}
	}

	// this function is looking for UPoseSearchDatabase(s) to search for the input AssetToSearch:
	// if AssetToSearch is a database search it ALL,
	// if it's a sequence containing UAnimNotifyState_PoseSearchBranchIn, we add to the search of the database UAnimNotifyState_PoseSearchBranchIn::Database the asset AssetToSearch
	static void AddToSearch(FStackDatabaseToAssetIndexes& AssetIndexesToSearchPerDatabase, const UObject* AssetToSearch, bool bUsePoseSearchBranchIn, FSearchContext& SearchContext)
	{
		if (bUsePoseSearchBranchIn)
		{
			if (const UAnimSequenceBase* SequenceBase = Cast<const UAnimSequenceBase>(AssetToSearch))
			{
				for (const FAnimNotifyEvent& NotifyEvent : SequenceBase->Notifies)
				{
					if (const UAnimNotifyState_PoseSearchBranchIn* PoseSearchBranchIn = Cast<UAnimNotifyState_PoseSearchBranchIn>(NotifyEvent.NotifyStateClass))
					{
						if (!PoseSearchBranchIn->Database)
						{
							UE_LOGF(LogPoseSearch, Error, "improperly setup UAnimNotifyState_PoseSearchBranchIn with null Database in %ls", *SequenceBase->GetName());
							continue;
						}

						// we just skip indexing databases to keep the experience as smooth as possible
						AddToSearchForDatabase(AssetIndexesToSearchPerDatabase, SequenceBase, PoseSearchBranchIn->Database, true, SearchContext);
					}
				}
			}
		}

		if (const UPoseSearchDatabase* Database = Cast<UPoseSearchDatabase>(AssetToSearch))
		{
#if WITH_EDITOR
			if (EAsyncBuildIndexResult::Success != FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(Database, ERequestAsyncBuildFlag::ContinueRequest))
			{
				SearchContext.SetAsyncBuildIndexInProgress();
			}
			else
#endif // WITH_EDITOR
			{
				FStackAssetIndexes& AssetIndexesToSearch = AssetIndexesToSearchPerDatabase.FindOrAdd(Database);
				const int32 NumAnimationAssets = Database->GetNumAnimationAssets();
				AssetIndexesToSearch.Reserve(NumAnimationAssets);
				for (int32 AnimationAssetIndex = 0; AnimationAssetIndex < NumAnimationAssets; ++AnimationAssetIndex)
				{
					AssetIndexesToSearch.Add(AnimationAssetIndex);
				}
			}
		}
	}

	static void PopulateContinuingPoseSearches(const FPoseSearchContinuingProperties& ContinuingProperties, const TArrayView<const UObject*> AssetsToSearch, FSearchContext& SearchContext, FStackDatabaseToAssetIndexes& ContinuingPoseAssetIndexesToSearchPerDatabase, bool bUsePoseSearchBranchIn)
	{
		if (const UObject* PlayingAnimationAsset = ContinuingProperties.PlayingAsset.Get())
		{
			// checking if any of the AssetsToSearch (databases) or ContinuingProperties.PlayingAssetDatabase contain PlayingAnimationAsset
			if (ContinuingProperties.PlayingAssetDatabase)
			{
				AddToSearchForDatabase(ContinuingPoseAssetIndexesToSearchPerDatabase, PlayingAnimationAsset, ContinuingProperties.PlayingAssetDatabase, false, SearchContext);
			}

			for (const UObject* AssetToSearch : AssetsToSearch)
			{
				if (const UPoseSearchDatabase* Database = Cast<UPoseSearchDatabase>(AssetToSearch))
				{
					// since it cannot be a database we can directly add it to ContinuingPoseAssetsToSearchPerDatabase
					AddToSearchForDatabase(ContinuingPoseAssetIndexesToSearchPerDatabase, PlayingAnimationAsset, Database, false, SearchContext);
				}
			}

			// checking if PlayingAnimationAsset has an associated database
			AddToSearch(ContinuingPoseAssetIndexesToSearchPerDatabase, PlayingAnimationAsset, bUsePoseSearchBranchIn, SearchContext);

			FilterByAssetsToConsider(SearchContext, ContinuingPoseAssetIndexesToSearchPerDatabase);
		}
	}
	
	static void PopulateSearches(const TArrayView<const UObject*> AssetsToSearch, FSearchContext& SearchContext, FStackDatabaseToAssetIndexes& AssetIndexesToSearchPerDatabase, bool bUsePoseSearchBranchIn)
	{
		for (const UObject* AssetToSearch : AssetsToSearch)
		{
			AddToSearch(AssetIndexesToSearchPerDatabase, AssetToSearch, bUsePoseSearchBranchIn, SearchContext);
		}

		FilterByAssetsToConsider(SearchContext, AssetIndexesToSearchPerDatabase);
	}

	// @todo: refine this logic. Currently if AssetsToSearch contains ONLY UPoseSearchDatabase we don't have to look for other databases referenced by other assets UAnimNotifyState_PoseSearchBranchIn(s)
	bool ShouldUsePoseSearchBranchIn(const TArrayView<const UObject*> AssetsToSearch)
	{
		for (const UObject* AssetToSearch : AssetsToSearch)
		{
			if (!Cast<UPoseSearchDatabase>(AssetToSearch))
			{
				return true;
			}
		}
		return false;
	}

	template <typename DatabasesContainer>
	static bool IsForceInterrupt(EPoseSearchInterruptMode InterruptMode, const UPoseSearchDatabase* CurrentResultDatabase, const DatabasesContainer& Databases)
	{
		switch (InterruptMode)
		{
		case EPoseSearchInterruptMode::DoNotInterrupt:
			return false;

		case EPoseSearchInterruptMode::InterruptOnDatabaseChange:	// Fall through
		case EPoseSearchInterruptMode::InterruptOnDatabaseChangeAndInvalidateContinuingPose:
			return !Databases.Contains(CurrentResultDatabase);

		case EPoseSearchInterruptMode::ForceInterrupt:				// Fall through
		case EPoseSearchInterruptMode::ForceInterruptAndInvalidateContinuingPose:
			return true;

		default:
			checkNoEntry();
			return false;
		}
	}

	template <typename DatabasesContainer>
	static bool IsInvalidatingContinuingPose(EPoseSearchInterruptMode InterruptMode, const UPoseSearchDatabase* CurrentResultDatabase, const DatabasesContainer& Databases)
	{
		switch (InterruptMode)
		{
		case EPoseSearchInterruptMode::DoNotInterrupt:				// Fall through
		case EPoseSearchInterruptMode::InterruptOnDatabaseChange:	// Fall through
		case EPoseSearchInterruptMode::ForceInterrupt:	
			return false;

		case EPoseSearchInterruptMode::InterruptOnDatabaseChangeAndInvalidateContinuingPose:
			return !Databases.Contains(CurrentResultDatabase);

		case EPoseSearchInterruptMode::ForceInterruptAndInvalidateContinuingPose:
			return true;

		default:
			checkNoEntry();
			return false;
		}
	}

	static bool ShouldUseCachedChannelData(const UPoseSearchDatabase* CurrentResultDatabase, const TConstArrayView<TObjectPtr<const UPoseSearchDatabase>> Databases)
	{
		const UPoseSearchSchema* OneOfTheSchemas = nullptr;
		if (CurrentResultDatabase)
		{
			OneOfTheSchemas = CurrentResultDatabase->Schema;
		}

		for (const TObjectPtr<const UPoseSearchDatabase>& Database : Databases)
		{
			if (Database)
			{
				if (OneOfTheSchemas != Database->Schema)
				{
					if (OneOfTheSchemas == nullptr)
					{
						OneOfTheSchemas = Database->Schema;
					}
					else
					{
						// we found we need to search multiple schemas
						return true;
					}
				}
			}
		}

		return false;
	}

	FRole GetCommonDefaultRole(const TConstArrayView<TObjectPtr<const UPoseSearchDatabase>> Databases)
	{
		FRole Role = DefaultRole;

		if (!Databases.IsEmpty())
		{
			if (const UPoseSearchDatabase* Database = Databases[0].Get())
			{
				if (const UPoseSearchSchema* Schema = Database->Schema)
				{
					Role = Schema->GetDefaultRole();
				}
			}

#if WITH_EDITOR && ENABLE_ANIM_DEBUG 
			for (int32 DatabaseIndex = 1; DatabaseIndex < Databases.Num(); ++DatabaseIndex)
			{
				if (const UPoseSearchDatabase* Database = Databases[DatabaseIndex].Get())
				{
					if (const UPoseSearchSchema* Schema = Database->Schema)
					{
						if (Role != Schema->GetDefaultRole())
						{
							UE_LOGF(LogPoseSearch, Error, "GetCommonDefaultRole - inconsistent Role between provided Databases!");
							break;
						}
					}
				}
			}
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG
		}

		return Role;
	}

	float CalculateWantedPlayRate(const FSearchResult& SearchResult, const FSearchContext& SearchContext, const FFloatInterval& PlayRate, float TrajectorySpeedMultiplier, const FPoseSearchEvent& EventToSearch)
	{
		float WantedPlayRate = 1.f;

		if (SearchResult.IsValid())
		{
			if (SearchResult.IsEventSearchResult())
			{
				// checking if SearchResult.EventPoseIdx is part of the EventToSearch.EventTag.
				// If not, it's an event from a continuing pose search that hasn't been interrupted,
				// so we keep the previously calculated WantedPlayRate
				const bool bIsEventSearchFromTag = SearchResult.IsEventSearchFromTag(EventToSearch.EventTag);
				if (bIsEventSearchFromTag)
				{
					const float TimeToEvent = SearchResult.CalculateTimeToEvent();
					if (TimeToEvent > UE_KINDA_SMALL_NUMBER && EventToSearch.TimeToEvent > UE_KINDA_SMALL_NUMBER)
					{
						// EventToSearch.TimeToEvent is the desired time to event, and TimeToEvent is the actually current time to event. we calculate WantedPlayRate as ratio between the two
						WantedPlayRate = TimeToEvent / EventToSearch.TimeToEvent;
					}
					// if we passed the event (TimeToEvent <= 0) we leave the WantedPlayRate as previously calculated
				}
			}
			else if (!ensure(PlayRate.Min <= PlayRate.Max && PlayRate.Min > UE_KINDA_SMALL_NUMBER))
			{
				UE_LOGF(LogPoseSearch, Error, "Couldn't update the WantedPlayRate in CalculateWantedPlayRate, because of invalid PlayRate interval (%f, %f)", PlayRate.Min, PlayRate.Max);
				WantedPlayRate = 1.f;
			}
			else if (!FMath::IsNearlyEqual(PlayRate.Min, PlayRate.Max, UE_KINDA_SMALL_NUMBER))
			{
				TConstArrayView<float> QueryData = SearchContext.GetCachedQuery(SearchResult.Database->Schema);
				if (!QueryData.IsEmpty())
				{
					if (const UPoseSearchFeatureChannel_Trajectory* TrajectoryChannel = SearchResult.Database->Schema->FindFirstChannelOfType<UPoseSearchFeatureChannel_Trajectory>())
					{
						const FSearchIndex& SearchIndex = SearchResult.Database->GetSearchIndex();
						const bool bReconstructPoseValues = SearchIndex.IsValuesEmpty();

						if (bReconstructPoseValues)
						{
							const int32 NumDimensions = SearchIndex.GetNumDimensions();

							// FMemory_Alloca is forced 16 bytes aligned and its allocated memory is in scope until the end of the function scope,
							// not the statement scope, so it's safe to use it in GetEstimatedSpeedRatio
							TArrayView<float> ReconstructedPoseValuesBuffer((float*)FMemory_Alloca(NumDimensions * sizeof(float)), NumDimensions);
							check(IsAligned(ReconstructedPoseValuesBuffer.GetData(), alignof(VectorRegister4Float)));

							const TConstArrayView<float> ResultData = SearchIndex.GetReconstructedPoseValues(SearchResult.PoseIdx, ReconstructedPoseValuesBuffer);
							if (ResultData.IsEmpty())
							{
								UE_LOGF(LogPoseSearch, Warning,
									"Couldn't update the WantedPlayRate in CalculateWantedPlayRate, because couldn't reconstruct the pose value in GetReconstructedPoseValues for pose %d",
									SearchResult.PoseIdx);
								WantedPlayRate = FMath::Clamp(1.f, PlayRate.Min, PlayRate.Max);
							}
							else
							{
								const float EstimatedSpeedRatio = TrajectoryChannel->GetEstimatedSpeedRatio(QueryData, ResultData);
								WantedPlayRate = FMath::Clamp(EstimatedSpeedRatio, PlayRate.Min, PlayRate.Max);
							}
						}
						else
						{
							const TConstArrayView<float> ResultData = SearchIndex.GetPoseValues(SearchResult.PoseIdx);
							const float EstimatedSpeedRatio = TrajectoryChannel->GetEstimatedSpeedRatio(QueryData, ResultData);
							WantedPlayRate = FMath::Clamp(EstimatedSpeedRatio, PlayRate.Min, PlayRate.Max);
						}
					}
					else
					{
						UE_LOGF(LogPoseSearch, Warning,
							"Couldn't update the WantedPlayRate in CalculateWantedPlayRate, because Schema '%ls' couldn't find a UPoseSearchFeatureChannel_Trajectory channel",
							*GetNameSafe(SearchResult.Database->Schema));
					}
				}
			}
			else if (!FMath::IsNearlyZero(TrajectorySpeedMultiplier))
			{
				WantedPlayRate = PlayRate.Min / TrajectorySpeedMultiplier;
			}
			else
			{
				WantedPlayRate = PlayRate.Min;
			}
		}

		return WantedPlayRate;
	}
}

//////////////////////////////////////////////////////////////////////////
// FMotionMatchingState

void FMotionMatchingState::Reset()
{
	SearchResult = FPoseSearchBlueprintResult();
	// Set the elapsed time to INFINITY to trigger a search right away
	ElapsedPoseSearchTime = std::numeric_limits<float>::infinity();
}

FVector FMotionMatchingState::GetEstimatedFutureRootMotionVelocity() const
{
	using namespace UE::PoseSearch;
	if (const UPoseSearchDatabase* Database = SearchResult.SelectedDatabase.Get())
	{
		if (const UPoseSearchFeatureChannel_Trajectory* TrajectoryChannel = Database->Schema->FindFirstChannelOfType<UPoseSearchFeatureChannel_Trajectory>())
		{
			const int32 PoseIndex = Database->GetPoseIndex(SearchResult.SelectedAnim.Get(), SearchResult.SelectedTime, SearchResult.bIsMirrored, SearchResult.BlendParameters);
			const FSearchIndex& SearchIndex = Database->GetSearchIndex();
			if (PoseIndex != INDEX_NONE && !SearchIndex.IsValuesEmpty())
			{
				TConstArrayView<float> ResultData = SearchIndex.GetPoseValues(PoseIndex);
				return TrajectoryChannel->GetEstimatedFutureRootMotionVelocity(ResultData);
			}
		}
	}

	return FVector::ZeroVector;
}

//////////////////////////////////////////////////////////////////////////
// FPoseSearchContinuingProperties

void FPoseSearchContinuingProperties::InitFrom(const FPoseSearchBlueprintResult& SearchResult, EPoseSearchInterruptMode InInterruptMode)
{
	PlayingAsset = SearchResult.SelectedAnim;
	PlayingAssetAccumulatedTime = SearchResult.SelectedTime;
	bIsPlayingAssetMirrored = SearchResult.bIsMirrored;
	PlayingAssetBlendParameters = SearchResult.BlendParameters;
	InterruptMode = InInterruptMode;
	PlayingAssetDatabase = SearchResult.SelectedDatabase;
}

//////////////////////////////////////////////////////////////////////////
// UPoseSearchLibrary

#if UE_POSE_SEARCH_TRACE_ENABLED

void UPoseSearchLibrary::TraceMotionMatching(
	UE::PoseSearch::FSearchContext& SearchContext,
	const UE::PoseSearch::FSearchResult& SearchResult,
	float ElapsedPoseSearchTime,
	float DeltaTime,
	bool bSearch,
	float WantedPlayRate,
	EPoseSearchInterruptMode InterruptMode)
{
	using namespace UE::PoseSearch;
	FSearchResults_Single SearchResults;
	if (bSearch && SearchResult.IsValid())
	{
		SearchResults.UpdateWith(SearchResult);
	}
	TraceMotionMatching(SearchContext, SearchResults, ElapsedPoseSearchTime, WantedPlayRate, InterruptMode);
}

void UPoseSearchLibrary::TraceMotionMatching(
	UE::PoseSearch::FSearchContext& SearchContext,
	const UE::PoseSearch::FSearchResults& SearchResults,
	float ElapsedPoseSearchTime,
	float WantedPlayRate,
	EPoseSearchInterruptMode InterruptMode)
{
	using namespace UE::PoseSearch;

	const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(PoseSearchChannel);
	if (!bChannelEnabled)
	{
		return;
	}

	float RecordingTime = 0.f;
	if (!SearchContext.GetContexts().IsEmpty())
	{
		if (const UObject* AnimContext = GetAnimContext(SearchContext.GetContexts()[0]))
		{
			RecordingTime = FObjectTrace::GetWorldElapsedTime(AnimContext->GetWorld());
		}
	}

	uint32 SearchId = 787;

	FTraceMotionMatchingStateMessage TraceState;

	TraceState.InterruptMode = InterruptMode;

	const int32 AnimContextsNum = SearchContext.GetContexts().Num();
	TraceState.SkeletalMeshComponentIds.SetNum(AnimContextsNum);

	for (int32 AnimInstanceIndex = 0; AnimInstanceIndex < AnimContextsNum; ++AnimInstanceIndex)
	{
		const UObject* AnimContext = GetAnimContext(SearchContext.GetContexts()[AnimInstanceIndex]);
		const UObject* SkeletalMeshComponent = nullptr;
		if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(AnimContext))
		{
			SkeletalMeshComponent = AnimInstance->GetOuter();
		}
		else if (const UActorComponent* ActorComponent = Cast<UActorComponent>(AnimContext))
		{
			const AActor* Actor = ActorComponent->GetOwner();
			check(Actor);
				
			SkeletalMeshComponent = Actor->GetComponentByClass<USkeletalMeshComponent>();
		}

		if (!SkeletalMeshComponent || CANNOT_TRACE_OBJECT(SkeletalMeshComponent))
		{
			return;
		}
			
		TRACE_OBJECT(SkeletalMeshComponent);
		TraceState.SkeletalMeshComponentIds[AnimInstanceIndex] = FObjectTrace::GetObjectId(SkeletalMeshComponent);
	}

	for (int32 AnimInstanceIndex = 0; AnimInstanceIndex < AnimContextsNum; ++AnimInstanceIndex)
	{
		if (const UObject* AnimContext = GetAnimContext(SearchContext.GetContexts()[AnimInstanceIndex]))
		{
			TRACE_OBJECT(AnimContext);
			SearchId = HashCombineFast(SearchId, GetTypeHash(FObjectTrace::GetObjectId(AnimContext)));
		}
	}

	TraceState.Roles.SetNum(AnimContextsNum);
	for (const FRoleToIndexPair& RoleToIndexPair : SearchContext.GetRoleToIndex())
	{
		TraceState.Roles[RoleToIndexPair.Value] = RoleToIndexPair.Key;
	}

	SearchId = HashCombineFast(SearchId, GetTypeHash(TraceState.Roles));

	// @todo: do we need to hash pose history names in SearchId as well?
	TraceState.PoseHistories.SetNum(AnimContextsNum);
	for (int32 AnimInstanceIndex = 0; AnimInstanceIndex < AnimContextsNum; ++AnimInstanceIndex)
	{
		TraceState.PoseHistories[AnimInstanceIndex].InitFrom(SearchContext.GetPoseHistories()[AnimInstanceIndex]);
	}

	// finalizing SearchContext.GetBestPoseCandidatesMap() by calling SearchContext::Track for all the SearchResults flagging them as EPoseCandidateFlags::Valid_CurrentPose
	// because, those candidate could have been discarded because of their cost was too high and didn't fit into SearchContext.GetBestPoseCandidatesMap() limits
	SearchResults.IterateOverSearchResults([&SearchContext](FSearchResult& SearchResult)
		{
			if (SearchResult.IsValid())
			{
				SearchContext.Track(SearchResult.Database.Get(), SearchResult.PoseIdx, EPoseCandidateFlags::Valid_CurrentPose, SearchResult.PoseCost);
			}
			return false;
		});

	TArray<uint64, TInlineAllocator<64>> DatabaseIds;
	int32 DbEntryIdx = 0;

	TraceState.DatabaseEntries.SetNum(SearchContext.GetBestPoseCandidatesMap().Num());
	for (TPair<const UPoseSearchDatabase*, FSearchContext::FBestPoseCandidates> DatabaseBestPoseCandidates : SearchContext.GetBestPoseCandidatesMap())
	{
		const UPoseSearchDatabase* Database = DatabaseBestPoseCandidates.Key;
		check(Database);

		FTraceMotionMatchingStateDatabaseEntry& DbEntry = TraceState.DatabaseEntries[DbEntryIdx];

		// if throttling is on, the continuing pose can be valid, but no actual search occurred, so the query will not be cached, and we need to build it
		DbEntry.QueryVector = SearchContext.GetOrBuildQuery(Database->Schema);
		TRACE_OBJECT(Database);
		DbEntry.DatabaseId = FObjectTrace::GetObjectId(Database);
		DatabaseIds.Add(DbEntry.DatabaseId);

		DatabaseBestPoseCandidates.Value.IterateOverBestPoseCandidates([&DbEntry](const FSearchContext::FPoseCandidate& PoseCandidate)
			{
				// @todo replace FTraceMotionMatchingStatePoseEntry with FSearchContext::FPoseCandidate
				FTraceMotionMatchingStatePoseEntry PoseEntry;
				PoseEntry.DbPoseIdx = PoseCandidate.PoseIdx;
				PoseEntry.Cost = PoseCandidate.Cost;
				PoseEntry.PoseCandidateFlags = PoseCandidate.PoseCandidateFlags;
				DbEntry.PoseEntries.Add(PoseEntry);
				return false;
			});

		++DbEntryIdx;
	}

	// @todo: reenable this code if needed
	//PRAGMA_DISABLE_DEPRECATION_WARNINGS
	//if (SearchResult.IsValid())
	//{
	//	TraceState.CurrentDbEntryIdx = DbEntryIdx;
	//	TraceState.CurrentPoseEntryIdx = DbEntry.PoseEntries.Add(PoseEntry);
	//}
	//PRAGMA_ENABLE_DEPRECATION_WARNINGS

	DatabaseIds.Sort();
	SearchId = HashCombineFast(SearchId, GetTypeHash(DatabaseIds));

	// @todo: instead of using the SearchResult (the best search result with the lowest cost) to calculate velocities etc, 
	//        shouldn't we aggregate the values from all the SearchResults (using SearchResults.IterateOverSearchResults)?
	const FSearchResult SearchResult = SearchResults.GetBestResult();
	const float AssetTime = SearchResult.IsValid() ? SearchResult.GetAssetTime() : 0.f;

	// @todo: integrate DeltaTime into SearchContext, and implement it for UAF as well
	float DeltaTime = FiniteDelta;
	if (!SearchContext.GetContexts().IsEmpty())
	{
		if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(GetAnimContext(SearchContext.GetContexts()[0])))
		{
			DeltaTime = AnimInstance->GetDeltaSeconds();
		}
	}

	if (DeltaTime > SMALL_NUMBER)
	{
		// simulation
		if (SearchContext.AnyCachedQuery())
		{
			TraceState.SimLinearVelocity = 0.f;
			TraceState.SimAngularVelocity = 0.f;

			const int32 NumRoles = SearchContext.GetRoleToIndex().Num();
			for (const FRoleToIndexPair& RoleToIndexPair : SearchContext.GetRoleToIndex())
			{
				const FRole& Role = RoleToIndexPair.Key;

				const FTransform PrevRoot = SearchContext.GetWorldBoneTransformAtTime(-DeltaTime, Role, RootSchemaBoneIdx);
				const FTransform CurrRoot = SearchContext.GetWorldBoneTransformAtTime(0.f, Role, RootSchemaBoneIdx);
				
				const FTransform SimDelta = CurrRoot.GetRelativeTransform(PrevRoot);
				TraceState.SimLinearVelocity += SimDelta.GetTranslation().Size() / (DeltaTime * NumRoles);
				TraceState.SimAngularVelocity += FMath::RadiansToDegrees(SimDelta.GetRotation().GetAngle()) / (DeltaTime * NumRoles);
			}
		}
		
		const FSearchIndexAsset* SearchIndexAsset = SearchResult.GetSearchIndexAsset();
		const UPoseSearchDatabase* CurrentResultDatabase = SearchResult.Database.Get();
		if (SearchIndexAsset && CurrentResultDatabase)
		{
			const FPoseSearchDatabaseAnimationAssetBase* DatabaseAsset = CurrentResultDatabase->GetDatabaseAnimationAsset(SearchIndexAsset->GetSourceAssetIdx());
			check(DatabaseAsset);
			if (UAnimationAsset* AnimationAsset = Cast<UAnimationAsset>(DatabaseAsset->GetAnimationAsset()))
			{
				// Simulate the time step to get accurate root motion prediction for this frame.
				FAnimationAssetSampler Sampler(AnimationAsset, FTransform::Identity,FVector::ZeroVector, FAnimationAssetSampler::DefaultRootTransformSamplingRate, true, false);

				const float TimeStep = DeltaTime * WantedPlayRate;
				const FTransform PrevRoot = Sampler.ExtractRootTransform(AssetTime);
				const FTransform CurrRoot = Sampler.ExtractRootTransform(AssetTime + TimeStep);
				const FTransform RootMotionTransformDelta = PrevRoot.GetRelativeTransform(CurrRoot);
				TraceState.AnimLinearVelocity = RootMotionTransformDelta.GetTranslation().Size() / DeltaTime;
				TraceState.AnimAngularVelocity = FMath::RadiansToDegrees(RootMotionTransformDelta.GetRotation().GetAngle()) / DeltaTime;

				// Need another root motion extraction for non-playrate version in case acceleration isn't the same.
				const FTransform CurrRootNoTimescale = Sampler.ExtractRootTransform(AssetTime + DeltaTime);
				const FTransform RootMotionTransformDeltaNoTimescale = PrevRoot.GetRelativeTransform(CurrRootNoTimescale);
				TraceState.AnimLinearVelocityNoTimescale = RootMotionTransformDeltaNoTimescale.GetTranslation().Size() / DeltaTime;
				TraceState.AnimAngularVelocityNoTimescale = FMath::RadiansToDegrees(RootMotionTransformDeltaNoTimescale.GetRotation().GetAngle()) / DeltaTime;
			}
		}
		TraceState.Playrate = WantedPlayRate;
	}

	TraceState.ElapsedPoseSearchTime = ElapsedPoseSearchTime;
	TraceState.AssetPlayerTime = AssetTime;;
	TraceState.DeltaTime = DeltaTime;

	TraceState.RecordingTime = RecordingTime;
	TraceState.SearchBestCost = SearchResult.PoseCost;
#if WITH_EDITOR && ENABLE_ANIM_DEBUG
	TraceState.SearchBruteForceCost = SearchResult.BruteForcePoseCost;
	TraceState.SearchBestPosePos = SearchResult.BestPosePos;
#else // WITH_EDITOR && ENABLE_ANIM_DEBUG
	TraceState.SearchBruteForceCost = 0.f;
	TraceState.SearchBestPosePos = 0;
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG

	TraceState.Cycle = FPlatformTime::Cycles64();

	// @todo: avoid publishing duplicated TraceState in ALL the AnimContexts! -currently necessary for multi character-
	for (const FChooserEvaluationContext* Context : SearchContext.GetContexts())
	{
		const UObject* AnimContextObject = GetAnimContext(Context);
		TRACE_OBJECT(AnimContextObject);
		TraceState.AnimInstanceId = FObjectTrace::GetObjectId(AnimContextObject);
		TraceState.NodeId = SearchId;
		TraceState.Output();
	}
}
#endif // UE_POSE_SEARCH_TRACE_ENABLED

void UPoseSearchLibrary::UpdateMotionMatchingState(
	FChooserEvaluationContext* AnimContext,
	const UE::PoseSearch::IPoseHistory* PoseHistory,
	const TConstArrayView<TObjectPtr<const UPoseSearchDatabase>> Databases,
	float DeltaTime,
	const FFloatInterval& PoseJumpThresholdTime,
	float PoseReselectHistory,
	float SearchThrottleTime,
	const FFloatInterval& PlayRate,
	FMotionMatchingState& InOutMotionMatchingState,
	EPoseSearchInterruptMode InterruptMode,
	bool bShouldUseCachedChannelData,
	bool bDebugDrawQuery,
	bool bDebugDrawCurResult,
	const FPoseSearchEvent& EventToSearch)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PoseSearch_Update);

	using namespace UE::PoseSearch;

	FMemMark Mark(FMemStack::Get());

	const FPoseSearchEvent PlayRateOverriddenEvent = EventToSearch.GetPlayRateOverriddenEvent(PlayRate);
	FSearchContext SearchContext(0.f, PoseJumpThresholdTime, PlayRateOverriddenEvent);
	SearchContext.AddRole(GetCommonDefaultRole(Databases), AnimContext, PoseHistory);

	const UPoseSearchDatabase* CurrentResultDatabase = InOutMotionMatchingState.SearchResult.SelectedDatabase.Get();
	if (IsInvalidatingContinuingPose(InterruptMode, CurrentResultDatabase, Databases))
	{
		InOutMotionMatchingState.SearchResult = FPoseSearchBlueprintResult();
	}
	else
	{
		FSearchResult SearchResult;
		SearchResult.InitFrom(InOutMotionMatchingState.SearchResult);
		SearchContext.UpdateContinuingPoseSearchResult(SearchResult, SearchResult);
	}

	FSearchResult SearchResult;

	const bool bCanAdvance = SearchContext.GetContinuingPoseSearchResult().PoseIdx != INDEX_NONE;

	// If we can't advance or enough time has elapsed since the last pose jump then search
	const bool bSearch = !bCanAdvance || (InOutMotionMatchingState.ElapsedPoseSearchTime >= SearchThrottleTime);
	if (bSearch)
	{
		InOutMotionMatchingState.ElapsedPoseSearchTime = 0.f;
		const bool bForceInterrupt = IsForceInterrupt(InterruptMode, CurrentResultDatabase, Databases);
		const bool bSearchContinuingPose = !bForceInterrupt && bCanAdvance;

		// calculating if it's worth bUseCachedChannelData (if we potentially have to build query with multiple schemas)
		SearchContext.SetUseCachedChannelData(bShouldUseCachedChannelData && ShouldUseCachedChannelData(bSearchContinuingPose ? CurrentResultDatabase : nullptr, Databases));

		FSearchResults_Single SearchResults;

		// Evaluate continuing pose
		if (bSearchContinuingPose)
		{
			CurrentResultDatabase->SearchContinuingPose(SearchContext, SearchResults);
		}

		for (const TObjectPtr<const UPoseSearchDatabase>& Database : Databases)
		{
			if (Database)
			{
				Database->Search(SearchContext, SearchResults);
			}
		}

		SearchResult = SearchResults.GetBestResult();

#if !NO_LOGGING
		if (!SearchResult.IsValid())
		{
			TStringBuilder<1024> StringBuilder;
			StringBuilder << "UPoseSearchLibrary::UpdateMotionMatchingState invalid search result : ForceInterrupt [";
			StringBuilder << bForceInterrupt;
			StringBuilder << "], CanAdvance [";
			StringBuilder << bCanAdvance;
			StringBuilder << "], Indexing [";

			bool bIsIndexing = false;
#if WITH_EDITOR
			bIsIndexing = SearchContext.IsAsyncBuildIndexInProgress();
#endif // WITH_EDITOR
			StringBuilder << bIsIndexing;

			StringBuilder << "], Databases [";

			for (int32 DatabaseIndex = 0; DatabaseIndex < Databases.Num(); ++DatabaseIndex)
			{
				StringBuilder << GetNameSafe(Databases[DatabaseIndex]);
				if (DatabaseIndex != Databases.Num() - 1)
				{
					StringBuilder << ", ";
				}
			}

			StringBuilder << "] ";

			FString String = StringBuilder.ToString();

			if (bIsIndexing)
			{
				UE_LOGF(LogPoseSearch, Log, "%ls", *String);
			}
			else
			{
				UE_LOGF(LogPoseSearch, Warning, "%ls", *String);
			}
		}
#endif // !NO_LOGGING
	}
	else
	{
		InOutMotionMatchingState.ElapsedPoseSearchTime += DeltaTime;
		
		SearchResult = SearchContext.GetContinuingPoseSearchResult();
		SearchResult.bIsContinuingPoseSearch = true;

#if UE_POSE_SEARCH_TRACE_ENABLED
		// in case we skipped the search, we still have to track we would have requested to evaluate Databases and CurrentResultDatabase
		for (const TObjectPtr<const UPoseSearchDatabase>& Database : Databases)
		{
			SearchContext.Track(Database);
		}
		SearchContext.Track(CurrentResultDatabase);
#endif // UE_POSE_SEARCH_TRACE_ENABLED
	}

	const float WantedPlayRate = CalculateWantedPlayRate(SearchResult, SearchContext, PlayRate, PoseHistory ? PoseHistory->GetTrajectorySpeedMultiplier() : 1.f, EventToSearch);
	if (PoseHistory)
	{
		if (const FPoseIndicesHistory* PoseIndicesHistory = PoseHistory->GetPoseIndicesHistory())
		{
			// const casting here is safe since we're in the thread owning the pose history, and it's the correct place to update the previously selected poses
			const_cast<FPoseIndicesHistory*>(PoseIndicesHistory)->Update(SearchResult, DeltaTime, PoseReselectHistory);
		}
	}

	InOutMotionMatchingState.SearchResult.InitFrom(SearchResult, WantedPlayRate);

#if UE_POSE_SEARCH_TRACE_ENABLED
	TraceMotionMatching(SearchContext, SearchResult, InOutMotionMatchingState.ElapsedPoseSearchTime, DeltaTime, bSearch, InOutMotionMatchingState.SearchResult.WantedPlayRate, InterruptMode);
#endif // UE_POSE_SEARCH_TRACE_ENABLED

#if WITH_EDITORONLY_DATA && ENABLE_ANIM_DEBUG
	if (bDebugDrawQuery || bDebugDrawCurResult)
	{
		const UPoseSearchDatabase* CurResultDatabase = SearchResult.Database.Get();

#if WITH_EDITOR
		if (EAsyncBuildIndexResult::Success == FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(CurResultDatabase, ERequestAsyncBuildFlag::ContinueRequest))
#endif // WITH_EDITOR
		{
			FDebugDrawParams DrawParams;
			if (bDebugDrawCurResult)
			{
				DrawParams.Init(SearchContext.GetContexts(), SearchContext.GetPoseHistories(), SearchContext.GetRoleToIndex(), CurResultDatabase, FDebugDrawParams::EDrawContext::DrawCandidate);
				DrawParams.DrawFeatureVector(SearchResult.PoseIdx);
			}

			if (bDebugDrawQuery)
			{
				DrawParams.Init(SearchContext.GetContexts(), SearchContext.GetPoseHistories(), SearchContext.GetRoleToIndex(), CurResultDatabase, FDebugDrawParams::EDrawContext::DrawQuery);
				DrawParams.DrawFeatureVector(SearchContext.GetOrBuildQuery(CurResultDatabase->Schema));
			}
		}
	}
#endif
}

void UPoseSearchLibrary::IsAnimationAssetLooping(const UObject* Asset, bool& bIsAssetLooping)
{
	if (const UAnimSequenceBase* SequenceBase = Cast<const UAnimSequenceBase>(Asset))
	{
		bIsAssetLooping = SequenceBase->bLoop;
	}
	else if (const UBlendSpace* BlendSpace = Cast<const UBlendSpace>(Asset))
	{
		bIsAssetLooping = BlendSpace->bLoop;
	}
	else if (const UMultiAnimAsset* MultiAnimAsset = Cast<const UMultiAnimAsset>(Asset))
	{
		bIsAssetLooping = MultiAnimAsset->IsLooping();
	}
	else
	{
		bIsAssetLooping = false;
	}
}

void UPoseSearchLibrary::GetDatabaseTags(const UPoseSearchDatabase* Database, TArray<FName>& Tags)
{
	if (Database)
	{
		Tags = Database->Tags;
	}
	else
	{
		Tags.Reset();
	}
}
	
void UPoseSearchLibrary::OverridePoseHistoryFromOwningMesh(UAnimInstance* AnimInstance, const FName PoseHistoryName)
{
	using namespace UE::PoseSearch;

	const FAnimNode_PoseSearchHistoryCollector_Base* PoseHistoryNode = FindPoseHistoryNode(PoseHistoryName, AnimInstance);
	if (!PoseHistoryNode)
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchLibrary::OverridePoseHistoryFromOwningMesh - Couldn't find pose history node with name '%ls'", *PoseHistoryName.ToString());
		return;
	}

	FPoseHistory* PoseHistory = const_cast<FPoseHistory*>(PoseHistoryNode->GetPoseHistoryPtr());
	if (!PoseHistory)
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchLibrary::OverridePoseHistoryFromOwningMesh - Couldn't find a valid pose history in the pose history node '%ls'", *PoseHistoryName.ToString());
		return;
	}

	TArray<FName> CollectedCurves;
	
	const USkeletalMeshComponent* SkeletalMeshComponent = AnimInstance->GetSkelMeshComponent();
	FSKMCComponentSpacePoseProvider ComponentSpacePoseProvider(SkeletalMeshComponent);
	if (ComponentSpacePoseProvider.GetSkeletonAsset())
	{
		TArray<FBoneIndexType> RequiredBones = PoseHistoryNode->GetRequiredBones(SkeletalMeshComponent);

		static bool bStoreScales = false;
		static bool bNeedsReset  = false;
		static bool bCacheBones = false;
		static float RootBoneRecoveryTime = 0.f;
		static float RootBoneTranslationRecoveryRatio = 1.f;
		static float RootBoneRotationRecoveryRatio = 1.f;

		const float SamplingInterval = PoseHistory->GetSamplingInterval();
		// setting SamplingInterval to infinity to always override the latest pose in the pose history
		PoseHistory->SetSamplingInterval(UE_MAX_FLT);
		PoseHistory->EvaluateComponentSpace_AnyThread(0.f, ComponentSpacePoseProvider, bStoreScales,
			RootBoneRecoveryTime, RootBoneTranslationRecoveryRatio, RootBoneRotationRecoveryRatio, bNeedsReset, bCacheBones,
			RequiredBones, FBlendedCurve(), CollectedCurves
#if WITH_EDITORONLY_DATA
			, FTransform::Identity // GetContextTransform(nullptr, false)
#endif //WITH_EDITORONLY_DATA
		);

		// restoring previous SamplingInterval
		PoseHistory->SetSamplingInterval(SamplingInterval);
	}
}

void UPoseSearchLibrary::MotionMatch(
	UAnimInstance* AnimInstance,
	TArray<UObject*> AssetsToSearch,
	const FName PoseHistoryName,
	const FPoseSearchContinuingProperties ContinuingProperties,
	const FPoseSearchFutureProperties Future,
	FPoseSearchBlueprintResult& Result)
{
	using namespace UE::PoseSearch;

	Result = FPoseSearchBlueprintResult();

	if (!AnimInstance)
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchLibrary::MotionMatch - null AnimInstances");
		return;
	}

	if (!AnimInstance->CurrentSkeleton)
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchLibrary::MotionMatch - null AnimInstances->CurrentSkeleton");
		return;
	}

	// @todo: improve this approach, perhaps promoting UAnimInstance::GetProxyOnAnyThread from protected to public
	// making sure there're no flying animation tasks when on game thread by calling GetProxyOnAnyThread so it's safe to access animinstance variables etc
	class UFinishFlyingAnimInstanceTasks : public UAnimInstance
	{
	public:
		static void Execute(UAnimInstance* AnimInstance)
		{
			static_cast<UFinishFlyingAnimInstanceTasks*>(AnimInstance)->GetProxyOnAnyThread<FAnimInstanceProxy>();
		}
	};
	UFinishFlyingAnimInstanceTasks::Execute(AnimInstance);


	FMemMark Mark(FMemStack::Get());

	TArray<const IPoseHistory*, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> PoseHistories;
	if (const FAnimNode_PoseSearchHistoryCollector_Base* PoseHistoryNode = FindPoseHistoryNode(PoseHistoryName, AnimInstance))
	{
		if (const FPoseHistory* PoseHistory = PoseHistoryNode->GetPoseHistoryPtr())
		{
			PoseHistories.Add(PoseHistory);
		}
		else
		{
			UE_LOGF(LogPoseSearch, Error, "UPoseSearchLibrary::MotionMatch - Couldn't find a valid pose history in the pose history node '%ls'", *PoseHistoryName.ToString());
			return;
		}
	}
	else
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchLibrary::MotionMatch - Couldn't find pose history node with name '%ls'", *PoseHistoryName.ToString());
		return;
	}

	TArray<const UObject*, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> AnimContexts;
	AnimContexts.Add(AnimInstance);

	TArray<FName, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> Roles;
	Roles.Add(DefaultRole);

	TArray<const UObject*>& AssetsToSearchConst = reinterpret_cast<TArray<const UObject*>&>(AssetsToSearch);
	const FSearchResult SearchResult = MotionMatch(AnimContexts, Roles, PoseHistories, AssetsToSearchConst, ContinuingProperties, Future, FPoseSearchEvent());
	if (SearchResult.IsValid())
	{
		const UPoseSearchDatabase* Database = SearchResult.Database.Get();
		check(Database);
		
		// figuring out the WantedPlayRate
		float WantedPlayRate = 1.f;
		if (Future.Animation && Future.IntervalTime > 0.f)
		{
			if (const UPoseSearchFeatureChannel_PermutationTime* PermutationTimeChannel = Database->Schema->FindFirstChannelOfType<UPoseSearchFeatureChannel_PermutationTime>())
			{
				const FSearchIndex& SearchIndex = Database->GetSearchIndex();
				if (!SearchIndex.IsValuesEmpty())
				{
					TConstArrayView<float> ResultData = Database->GetSearchIndex().GetPoseValues(SearchResult.PoseIdx);
					const float ActualIntervalTime = PermutationTimeChannel->GetPermutationTime(ResultData);
					WantedPlayRate = ActualIntervalTime / Future.IntervalTime;
				}
			}
		}

		Result.InitFrom(SearchResult, WantedPlayRate);
	}
}

UE::PoseSearch::FSearchResult UPoseSearchLibrary::MotionMatch(
	const TArrayView<const UObject*> AnimContexts,
	const TArrayView<const UE::PoseSearch::FRole> Roles,
	const TArrayView<const UE::PoseSearch::IPoseHistory*> PoseHistories, 
	const TArrayView<const UObject*> AssetsToSearch,
	const FPoseSearchContinuingProperties& ContinuingProperties,
	const FPoseSearchFutureProperties& Future,
	const FPoseSearchEvent& EventToSearch)
{
	using namespace UE::PoseSearch;
	FSearchResults_Single SearchResults;
	MotionMatch(AnimContexts, Roles, PoseHistories, AssetsToSearch, ContinuingProperties, Future, EventToSearch, SearchResults);
	return SearchResults.GetBestResult();
}

void UPoseSearchLibrary::MotionMatch(
	const TArrayView<const UObject*> AnimContexts,
	const TArrayView<const UE::PoseSearch::FRole> Roles,
	const TArrayView<const UE::PoseSearch::IPoseHistory*> PoseHistories, 
	const TArrayView<const UObject*> AssetsToSearch,
	const FPoseSearchContinuingProperties& ContinuingProperties,
	const FPoseSearchFutureProperties& Future,
	const FPoseSearchEvent& EventToSearch,
	UE::PoseSearch::FSearchResults& SearchResults)
{
	TArray<FChooserEvaluationContext, TInlineAllocator<UE::PoseSearch::PreallocatedRolesNum>> Contexts;
	const int NumContexts = AnimContexts.Num(); 
	Contexts.SetNum(NumContexts);
	for(int i = 0; i < NumContexts; i++)
	{
		Contexts[i].AddObjectParam(const_cast<UObject*>(AnimContexts[i]));
	}

	MotionMatch(Contexts, Roles, PoseHistories, AssetsToSearch, ContinuingProperties, Future, EventToSearch, SearchResults);
}

void UPoseSearchLibrary::MotionMatch(
	const TArrayView<const UObject*> AnimContexts,
	const TArrayView<const UE::PoseSearch::FRole> Roles,
	const TArrayView<const UE::PoseSearch::IPoseHistory*> PoseHistories, 
	const TArrayView<const UObject*> AssetsToSearch,
	const FPoseSearchContinuingProperties& ContinuingProperties,
	float DesiredPermutationTimeOffset,
	const FPoseSearchEvent& EventToSearch,
	UE::PoseSearch::FSearchResults& SearchResults)
{
	TArray<FChooserEvaluationContext, TInlineAllocator<UE::PoseSearch::PreallocatedRolesNum>> Contexts;
	const int NumContexts = AnimContexts.Num(); 
	Contexts.SetNum(NumContexts);
	for(int i = 0; i < NumContexts; i++)
	{
		Contexts[i].AddObjectParam(const_cast<UObject*>(AnimContexts[i]));
	}

	MotionMatch(Contexts, Roles, PoseHistories, AssetsToSearch, ContinuingProperties, DesiredPermutationTimeOffset, EventToSearch, SearchResults);
}

void UPoseSearchLibrary::MotionMatch(
	const TArrayView<FChooserEvaluationContext> Contexts,
	const TArrayView<const UE::PoseSearch::FRole> Roles,
	const TArrayView<const UE::PoseSearch::IPoseHistory*> PoseHistories, 
	const TArrayView<const UObject*> AssetsToSearch,
	const FPoseSearchContinuingProperties& ContinuingProperties,
	const FPoseSearchFutureProperties& Future,
	const FPoseSearchEvent& EventToSearch,
	UE::PoseSearch::FSearchResults& SearchResults)
{
	check(!Contexts.IsEmpty() && Contexts.Num() == Roles.Num() && Contexts.Num() == PoseHistories.Num());

	using namespace UE::PoseSearch;

	TArray<const IPoseHistory*, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> InternalPoseHistories;
	InternalPoseHistories = PoseHistories;

	// MemStackPoseHistories will hold future poses to match AssetSamplerBase (at FutureAnimationStartTime) TimeToFutureAnimationStart seconds in the future
	TArray<FMemStackPoseHistory, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> MemStackPoseHistories;
	float FutureIntervalTime = Future.IntervalTime;
	if (Future.Animation)
	{
		MemStackPoseHistories.SetNum(InternalPoseHistories.Num());

		float FutureAnimationTime = Future.AnimationTime;
		if (FutureAnimationTime < FiniteDelta)
		{
			UE_LOGF(LogPoseSearch, Warning, "UPoseSearchLibrary::MotionMatch - provided Future.AnimationTime (%f) is too small to be able to calculate velocities. Clamping it to minimum value of %f", FutureAnimationTime, FiniteDelta);
			FutureAnimationTime = FiniteDelta;
		}

		const float MinFutureIntervalTime = FiniteDelta + UE_KINDA_SMALL_NUMBER;
		if (FutureIntervalTime < MinFutureIntervalTime)
		{
			UE_LOGF(LogPoseSearch, Warning, "UPoseSearchLibrary::MotionMatch - provided TimeToFutureAnimationStart (%f) is too small. Clamping it to minimum value of %f", FutureIntervalTime, MinFutureIntervalTime);
			FutureIntervalTime = MinFutureIntervalTime;
		}

		for (int32 RoleIndex = 0; RoleIndex < Roles.Num(); ++RoleIndex)
		{
			if (const IPoseHistory* PoseHistory = InternalPoseHistories[RoleIndex])
			{
				const USkeleton* Skeleton = GetContextSkeleton(GetAnimContext(&Contexts[RoleIndex]), false);
				if (ensure(Skeleton))
				{
					// @todo: add input BlendParameters to support sampling FutureAnimation blendspaces and support for multi character
					const UAnimationAsset* AnimationAsset = Cast<UAnimationAsset>(Future.Animation);
					if (!AnimationAsset)
					{
						if (const UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(Future.Animation))
						{
							AnimationAsset = MultiAnimAsset->GetAnimationAsset(Roles[RoleIndex]);
						}
					}

					MemStackPoseHistories[RoleIndex].Init(InternalPoseHistories[RoleIndex]);
					MemStackPoseHistories[RoleIndex].ExtractAndAddFuturePoses(AnimationAsset, FutureAnimationTime, FiniteDelta, FVector::ZeroVector, FutureIntervalTime, Skeleton
#if WITH_EDITORONLY_DATA
						, false
						, GetContextTransform(GetAnimContext(&Contexts[RoleIndex]), false)
#endif //WITH_EDITORONLY_DATA
					);
					InternalPoseHistories[RoleIndex] = MemStackPoseHistories[RoleIndex].GetThisOrPoseHistory();
				}
			}
		}
	}		

	MotionMatch(Contexts, Roles, InternalPoseHistories, AssetsToSearch, ContinuingProperties, FutureIntervalTime, EventToSearch, SearchResults);
}

void UPoseSearchLibrary::MotionMatch(
	const TArrayView<FChooserEvaluationContext> Contexts,
	const TArrayView<const UE::PoseSearch::FRole> Roles,
	const TArrayView<const UE::PoseSearch::IPoseHistory*> PoseHistories,
	const TArrayView<const UObject*> AssetsToSearch,
	const FPoseSearchContinuingProperties& ContinuingProperties,
	const float DesiredPermutationTimeOffset,
	const FPoseSearchEvent& EventToSearch,
	UE::PoseSearch::FSearchResults& SearchResults)
{
	using namespace UE::PoseSearch;
	
	FSearchContext SearchContext(DesiredPermutationTimeOffset, FFloatInterval(0.f, 0.f), EventToSearch);
	for (int32 RoleIndex = 0; RoleIndex < Roles.Num(); ++RoleIndex)
	{
		SearchContext.AddRole(Roles[RoleIndex], &Contexts[RoleIndex], PoseHistories[RoleIndex]);
	}

	MotionMatch(SearchContext, AssetsToSearch, ContinuingProperties, SearchResults);
}

void UPoseSearchLibrary::MotionMatch(
	UE::PoseSearch::FSearchContext& SearchContext,
	const TArrayView<const UObject*> AssetsToSearch,
	const FPoseSearchContinuingProperties& ContinuingProperties,
	UE::PoseSearch::FSearchResults& SearchResults)
{
	using namespace UE::PoseSearch;

	const FStackAssetSet* CurrentAssetsToConsider = nullptr;
	const FStackDatabaseToAssetIndexes* CurrentAssetIndexesToConsiderPerDatabase = nullptr;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TArray<const UObject*> InternalDeprecatedAssetsToConsider = SearchContext.GetInternalDeprecatedAssetsToConsider();
	if (InternalDeprecatedAssetsToConsider.IsEmpty())
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		CurrentAssetsToConsider = SearchContext.GetAssetsToConsiderSet();
	}
	CurrentAssetIndexesToConsiderPerDatabase = SearchContext.GetAssetIndexesToConsiderPerDatabase();

	SearchContext.SetContinuingInteraction(ContinuingProperties.bIsContinuingInteraction);
	SearchContext.SetContinuingContextInteraction(ContinuingProperties.bIsContinuingContextInteraction);

	// collecting all the databases searches in AssetsToSearchPerDatabase
	// and all the continuing pose searches in ContinuingPoseAssetsToSearchPerDatabase
	const bool bUsePoseSearchBranchIn = ShouldUsePoseSearchBranchIn(AssetsToSearch);
	FStackDatabaseToAssetIndexes AssetIndexesToSearchPerDatabase;
	FStackDatabaseToAssetIndexes ContinuingPoseAssetIndexesToSearchPerDatabase;

	PopulateSearches(AssetsToSearch, SearchContext, AssetIndexesToSearchPerDatabase, bUsePoseSearchBranchIn);
	PopulateContinuingPoseSearches(ContinuingProperties, AssetsToSearch, SearchContext, ContinuingPoseAssetIndexesToSearchPerDatabase, bUsePoseSearchBranchIn);

	FCachedContinuingPoseSearchResults CachedContinuingPoseSearchResults;
	for (const TPair<const UPoseSearchDatabase*, FStackAssetIndexes>& AssetIndexesToSearchPerDatabasePair : ContinuingPoseAssetIndexesToSearchPerDatabase)
	{
		const UPoseSearchDatabase* Database = AssetIndexesToSearchPerDatabasePair.Key;
		check(Database);

		const bool bInvalidatingContinuingPose = IsInvalidatingContinuingPose(ContinuingProperties.InterruptMode, Database, AssetIndexesToSearchPerDatabase);
		if (!bInvalidatingContinuingPose)
		{
			const int32 ContinuingPoseIdx = Database->GetPoseIndex(ContinuingProperties.PlayingAsset.Get(), ContinuingProperties.PlayingAssetAccumulatedTime, ContinuingProperties.bIsPlayingAssetMirrored, ContinuingProperties.PlayingAssetBlendParameters, &AssetIndexesToSearchPerDatabasePair.Value);

			if (ContinuingPoseIdx != INDEX_NONE)
			{
				// reconstructing and caching all the required continuing pose search results
				FSearchResult DatabaseContinuingPoseSearchResult;
				DatabaseContinuingPoseSearchResult.SetAssetTime(ContinuingProperties.PlayingAssetAccumulatedTime);
				DatabaseContinuingPoseSearchResult.PoseIdx = ContinuingPoseIdx;
				DatabaseContinuingPoseSearchResult.Database = Database;

				CachedContinuingPoseSearchResults.CheckedAdd(Database, DatabaseContinuingPoseSearchResult);

				// adding the continuing pose search result relative to the schema - first instance of the DatabaseContinuingPoseSearchResult 
				// (used to gather the continuing pose search values adopted to create the MM query - relative to the schema, NOT the database)
				const FSearchResult& SchemaContinuingPoseSearchResult = CachedContinuingPoseSearchResults.FindOrAdd(Database->Schema, DatabaseContinuingPoseSearchResult);

				const bool bForceInterrupt = IsForceInterrupt(ContinuingProperties.InterruptMode, Database, AssetIndexesToSearchPerDatabase);
				if (!bForceInterrupt)
				{
					SearchContext.UpdateContinuingPoseSearchResult(DatabaseContinuingPoseSearchResult, SchemaContinuingPoseSearchResult);
					Database->SearchContinuingPose(SearchContext, SearchResults);
				}
			}
		}
	}

	// performing all the other databases searches
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TArray<const UObject*> EmptyAssetsToConsider;
	SearchContext.SetInternalDeprecatedAssetsToConsider(EmptyAssetsToConsider);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	SearchContext.SetAssetsToConsiderSet(nullptr);
	SearchContext.SetAssetIndexesToConsiderPerDatabase(&AssetIndexesToSearchPerDatabase);

	for (const TPair<const UPoseSearchDatabase*, FStackAssetIndexes>& AssetIndexesToSearchPerDatabasePair : AssetIndexesToSearchPerDatabase)
	{
		const UPoseSearchDatabase* Database = AssetIndexesToSearchPerDatabasePair.Key;
		check(Database);

		// in case we haven't searched the continuing pose for this Database, we haven't created and cached the query yet,
		// but if we didn't invalidated the continuing pose (when IsInvalidatingContinuingPose is true), we still can reuse
		// the updated FirstInstanceOfReconstructedContinuingPoseSearchResult data, and by calling UpdateContinuingPoseSearchResult we set the 
		// SearchContext to be able to create a query for Database using the continuing pose data.
		SearchContext.UpdateContinuingPoseSearchResult(CachedContinuingPoseSearchResults.FindOrDefault(Database), CachedContinuingPoseSearchResults.FindOrDefault(Database->Schema));
		
		Database->Search(SearchContext, SearchResults);
	}

#if ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
	const FSearchResult SearchResult = SearchResults.GetBestResult();
	if (SearchResult.IsValid())
	{
		FDebugDrawParams DrawParams;
		if (GVarAnimMotionMatchDrawMatchEnable)
		{
			DrawParams.Init(SearchContext.GetContexts(), SearchContext.GetPoseHistories(), SearchContext.GetRoleToIndex(), SearchResult.Database.Get(), FDebugDrawParams::EDrawContext::DrawCandidate);
			DrawParams.DrawFeatureVector(SearchResult.PoseIdx);
		}

		if (GVarAnimMotionMatchDrawQueryEnable)
		{
			DrawParams.Init(SearchContext.GetContexts(), SearchContext.GetPoseHistories(), SearchContext.GetRoleToIndex(), SearchResult.Database.Get(), FDebugDrawParams::EDrawContext::DrawQuery);
			DrawParams.DrawFeatureVector(SearchContext.GetOrBuildQuery(SearchResult.Database->Schema));
		}
	}
#endif // ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG

#if UE_POSE_SEARCH_TRACE_ENABLED
	TraceMotionMatching(SearchContext, SearchResults, 0.f, 1.f, ContinuingProperties.InterruptMode);
#endif // UE_POSE_SEARCH_TRACE_ENABLED

	// restoring the assets to consider
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SearchContext.SetInternalDeprecatedAssetsToConsider(InternalDeprecatedAssetsToConsider);
	if (InternalDeprecatedAssetsToConsider.IsEmpty())
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		SearchContext.SetAssetsToConsiderSet(CurrentAssetsToConsider);
	}
	SearchContext.SetAssetIndexesToConsiderPerDatabase(CurrentAssetIndexesToConsiderPerDatabase);

#if WITH_EDITOR && !NO_LOGGING
	if (SearchContext.IsAsyncBuildIndexInProgress())
	{
		UE_LOGF(LogPoseSearch, Warning, "UPoseSearchLibrary::MotionMatch - some searches have been skipped, since databases AsyncBuildIndex are in still in progress or failed");
	}
#endif // WITH_EDITOR && !NO_LOGGING
}

const FAnimNode_PoseSearchHistoryCollector_Base* UPoseSearchLibrary::FindPoseHistoryNode(
	const FName PoseHistoryName,
	const UAnimInstance* AnimInstance)
{
	if (AnimInstance)
	{
		TSet<const UAnimInstance*, DefaultKeyFuncs<const UAnimInstance*>, TInlineSetAllocator<128>> AlreadyVisited;
		TArray<const UAnimInstance*, TInlineAllocator<128>> ToVisit;

		ToVisit.Add(AnimInstance);
		AlreadyVisited.Add(AnimInstance);

		while (!ToVisit.IsEmpty())
		{
			const UAnimInstance* Visiting = ToVisit.Pop();

			if (IAnimClassInterface* AnimBlueprintClass = IAnimClassInterface::GetFromClass(Visiting->GetClass()))
			{
				if (const FAnimSubsystem_Tag* TagSubsystem = AnimBlueprintClass->FindSubsystem<FAnimSubsystem_Tag>())
				{
					if (const FAnimNode_PoseSearchHistoryCollector_Base* HistoryCollector = TagSubsystem->FindNodeByTag<FAnimNode_PoseSearchHistoryCollector_Base>(PoseHistoryName, Visiting))
					{
						return HistoryCollector;
					}
				}
			}

			const USkeletalMeshComponent* SkeletalMeshComponent = Visiting->GetSkelMeshComponent();
			const TArray<UAnimInstance*>& LinkedAnimInstances = SkeletalMeshComponent->GetLinkedAnimInstances();
			for (const UAnimInstance* LinkedAnimInstance : LinkedAnimInstances)
			{
				bool bIsAlreadyInSet = false;
				AlreadyVisited.Add(LinkedAnimInstance, &bIsAlreadyInSet);
				if (!bIsAlreadyInSet)
				{
					ToVisit.Add(LinkedAnimInstance);
				}
			}
		}
	}
	return nullptr;
}

void UPoseSearchLibrary::ClearPoseSearchDatabasesManagement()
{
#if WITH_EDITOR
	using namespace UE::PoseSearch;

	FAsyncPoseSearchDatabasesManagement::Clear();
#endif // WITH_EDITOR
}

const AActor* UPoseSearchLibrary::GetActor(const FPoseSearchBlueprintResult& Result)
{
	using namespace UE::PoseSearch;

	if (ensure(Result.AnimContexts.IsValidIndex(Result.RoleIndex)))
	{
		return GetContextOwningActor(Result.AnimContexts[Result.RoleIndex]);
	}
	return nullptr;
}

const AActor* UPoseSearchLibrary::GetActorForRole(const FPoseSearchBlueprintResult& Result, const FName& Role)
{
	using namespace UE::PoseSearch;

	if (const UMultiAnimAsset* MultiAnimAsset = Cast<const UMultiAnimAsset>(Result.SelectedAnim))
	{
		const int32 NumRoles = MultiAnimAsset->GetNumRoles();
		for (int32 RoleIndex = 0; RoleIndex < NumRoles; ++RoleIndex)
		{
			if (MultiAnimAsset->GetRole(RoleIndex) == Role)
			{
				if (ensure(Result.AnimContexts.IsValidIndex(RoleIndex)))
				{
					return GetContextOwningActor(Result.AnimContexts[RoleIndex]);
				}
			}
		}
	}
	else if (Result.Role == Role)
	{
		return GetActor(Result);
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
