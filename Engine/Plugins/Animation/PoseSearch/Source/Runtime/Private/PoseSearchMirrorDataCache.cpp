// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchMirrorDataCache.h"
#include "AnimationRuntime.h"
#include "PoseSearch/PoseSearchDefines.h"

namespace UE::PoseSearch
{

FMirrorDataCache::FMirrorDataCache()
{
}

FMirrorDataCache::FMirrorDataCache(const UMirrorDataTable* InMirrorDataTable)
{
	Init(InMirrorDataTable);
}

FMirrorDataCache::FMirrorDataCache(const UMirrorDataTable* InMirrorDataTable, const FBoneContainer& BoneContainer)
{
	Init(InMirrorDataTable, BoneContainer);
}

void FMirrorDataCache::Init(const UMirrorDataTable* InMirrorDataTable)
{
	if (InMirrorDataTable != nullptr)
	{
		if (InMirrorDataTable->Skeleton)
		{
			MirrorDataTable = InMirrorDataTable;

			// array containing the bone index of the root bone (0)
			TArray<uint16, TInlineAllocator<1>> BoneIndices;
			BoneIndices.SetNumZeroed(1);

			// extracting the pose, containing only the root bone from the Sampler 
			FBoneContainer BoneContainer;
			BoneContainer.InitializeTo(BoneIndices, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *InMirrorDataTable->Skeleton);

			MirrorDataTable->FillCompactPoseAndComponentRefRotations(BoneContainer, CompactPoseMirrorBones, ComponentSpaceRefRotations);
		}
		else
		{
			UE_LOGF(LogPoseSearch, Error, "FMirrorDataCache::Init - UMirrorDataTable '%ls' Skeleton is not set!", *InMirrorDataTable->GetName());
			Reset();
		}
	}
	else
	{
		Reset();
	}
}

void FMirrorDataCache::Init(const UMirrorDataTable* InMirrorDataTable, const FBoneContainer& BoneContainer)
{
	if (InMirrorDataTable != nullptr)
	{
		if (InMirrorDataTable->Skeleton)
		{
			check(BoneContainer.IsValid());
			MirrorDataTable = InMirrorDataTable;
			MirrorDataTable->FillCompactPoseAndComponentRefRotations(BoneContainer, CompactPoseMirrorBones, ComponentSpaceRefRotations);
		}
		else
		{
			UE_LOGF(LogPoseSearch, Error, "FMirrorDataCache::Init - UMirrorDataTable '%ls' Skeleton is not set!", *InMirrorDataTable->GetName());
			Reset();
		}
	}
	else
	{
		Reset();
	}
}

void FMirrorDataCache::Reset()
{
	MirrorDataTable = nullptr;
	CompactPoseMirrorBones.Reset();
	ComponentSpaceRefRotations.Reset();
}

FTransform FMirrorDataCache::MirrorTransform(const FTransform& InTransform) const
{
	if (MirrorDataTable != nullptr)
	{
		const EAxis::Type MirrorAxis = MirrorDataTable->MirrorAxis;
		const FQuat& ReferenceRotation = ComponentSpaceRefRotations[FCompactPoseBoneIndex(RootBoneIndexType)];

		const FVector T = FAnimationRuntime::MirrorVector(InTransform.GetTranslation(), MirrorAxis);
		const FQuat Q = FAnimationRuntime::MirrorQuat(InTransform.GetRotation(), MirrorAxis);
		const FQuat QR = Q * FAnimationRuntime::MirrorQuat(ReferenceRotation, MirrorAxis).Inverse() * ReferenceRotation;
		const FTransform Result = FTransform(QR, T, InTransform.GetScale3D());
		return Result;
	}
	return InTransform;
}

void FMirrorDataCache::MirrorPose(FCompactPose& Pose) const
{
	if (MirrorDataTable != nullptr)
	{
		FAnimationRuntime::MirrorPose(
			Pose,
			MirrorDataTable->MirrorAxis,
			CompactPoseMirrorBones,
			ComponentSpaceRefRotations
		);
		// Note curves and attributes are not used during the indexing process and therefore don't need to be mirrored
	}
}

} // namespace UE::PoseSearch
