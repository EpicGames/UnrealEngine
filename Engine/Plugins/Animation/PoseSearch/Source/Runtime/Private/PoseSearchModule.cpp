// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimPoseSearchProvider.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNodeBase.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchLibrary.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearch/PoseSearchFeatureChannel_PermutationTime.h"
#include "RewindDebuggerRuntimeInterface/IRewindDebuggerRuntimeExtension.h"
#include "Animation/AttributeTypes.h"
#include "PoseSearch/PoseSearchHistoryAttribute.h"

class FRewindDebuggerPoseSearchRuntime : public IRewindDebuggerRuntimeExtension 
{
public:
	virtual void RecordingStarted() override
	{
		UE::Trace::ToggleChannel(TEXT("PoseSearch"), true);
	}
	
	virtual void RecordingStopped() override
	{
		UE::Trace::ToggleChannel(TEXT("PoseSearch"), false);
	}
};

class FPoseSearchModule final : public IModuleInterface, public UE::Anim::IPoseSearchProvider
{
public:
	
	// IModuleInterface
	virtual void StartupModule() override
	{
		IModularFeatures::Get().RegisterModularFeature(UE::Anim::IPoseSearchProvider::GetModularFeatureName(), this);
		IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, &RewindDebuggerPoseSearchRuntime);

		UE::Anim::AttributeTypes::RegisterType<FPoseHistoryAnimationAttribute>();
	}

	virtual void ShutdownModule() override
	{
		IModularFeatures::Get().UnregisterModularFeature(UE::Anim::IPoseSearchProvider::GetModularFeatureName(), this);
		IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, &RewindDebuggerPoseSearchRuntime);

		UE::Anim::AttributeTypes::UnregisterType<FPoseHistoryAnimationAttribute>();
	}

	// IPoseSearchProvider
	virtual UE::Anim::IPoseSearchProvider::FSearchResult Search(const FAnimationBaseContext& GraphContext, TArrayView<const UObject*> AssetsToSearch,
		const FSearchPlayingAsset& PlayingAsset, const FSearchFutureAsset& FutureAsset) const override
	{
		using namespace UE::PoseSearch;

		FPoseSearchContinuingProperties ContinuingProperties;
		ContinuingProperties.PlayingAsset = PlayingAsset.Asset;
		ContinuingProperties.PlayingAssetAccumulatedTime = PlayingAsset.AccumulatedTime;
		ContinuingProperties.bIsPlayingAssetMirrored = PlayingAsset.bMirrored;
		ContinuingProperties.PlayingAssetBlendParameters = PlayingAsset.BlendParameters;
		// @todo: ContinuingProperties.PlayingAssetDatabase = ???;

		FPoseSearchFutureProperties Future;
		Future.Animation = FutureAsset.Asset;
		Future.AnimationTime = FutureAsset.AccumulatedTime;
		Future.IntervalTime = FutureAsset.IntervalTime;

		const IPoseHistory* PoseHistory = nullptr;
		if (FPoseHistoryProvider* PoseHistoryProvider = GraphContext.GetMessage<FPoseHistoryProvider>())
		{
			PoseHistory = PoseHistoryProvider->GetPoseHistoryPtr();
		}

		UE::Anim::IPoseSearchProvider::FSearchResult ProviderResult;
		if (!PoseHistory)
		{
			UE_LOGF(LogPoseSearch, Error, "FPoseSearchModule::Search, missing IPoseHistory");
		}
		else
		{
			// @todo: Add event support
			const UObject* AnimContext = GraphContext.AnimInstanceProxy->GetAnimInstanceObject();
			const UE::PoseSearch::FSearchResult SearchResult = UPoseSearchLibrary::MotionMatch(MakeArrayView(&AnimContext, 1),
				MakeArrayView(&DefaultRole, 1), MakeArrayView(&PoseHistory, 1), AssetsToSearch, ContinuingProperties, Future, FPoseSearchEvent());

			if (const FSearchIndexAsset* SearchIndexAsset = SearchResult.GetSearchIndexAsset())
			{
				const UPoseSearchDatabase* Database = SearchResult.Database.Get();
				if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAssetBase = Database->GetDatabaseAnimationAsset(SearchIndexAsset->GetSourceAssetIdx()))
				{
					ProviderResult.SelectedAsset = DatabaseAnimationAssetBase->GetAnimationAsset();
					ProviderResult.Dissimilarity = SearchResult.PoseCost;
					ProviderResult.TimeOffsetSeconds = SearchResult.GetAssetTime();
					ProviderResult.bIsFromContinuingPlaying = SearchResult.bIsContinuingPoseSearch;
					ProviderResult.bMirrored = SearchIndexAsset->IsMirrored();
					ProviderResult.BlendParameters = SearchIndexAsset->GetBlendParameters();

					// figuring out the WantedPlayRate
					ProviderResult.WantedPlayRate = 1.f;
					if (Future.Animation && Future.IntervalTime > 0.f)
					{
						if (const UPoseSearchFeatureChannel_PermutationTime* PermutationTimeChannel = Database->Schema->FindFirstChannelOfType<UPoseSearchFeatureChannel_PermutationTime>())
						{
							const FSearchIndex& SearchIndex = Database->GetSearchIndex();
							if (!SearchIndex.IsValuesEmpty())
							{
								TConstArrayView<float> ResultData = Database->GetSearchIndex().GetPoseValues(SearchResult.PoseIdx);
								const float ActualIntervalTime = PermutationTimeChannel->GetPermutationTime(ResultData);
								ProviderResult.WantedPlayRate = ActualIntervalTime / Future.IntervalTime;
							}
						}
					}
				}
			}
		}
		return ProviderResult;
	}

private:
	FRewindDebuggerPoseSearchRuntime RewindDebuggerPoseSearchRuntime;
};

IMPLEMENT_MODULE(FPoseSearchModule, PoseSearch);