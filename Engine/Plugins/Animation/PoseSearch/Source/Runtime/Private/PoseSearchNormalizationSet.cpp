// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchNormalizationSet.h"
#include "PoseSearch/PoseSearchDatabase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchNormalizationSet)

#if WITH_EDITORONLY_DATA

void UPoseSearchNormalizationSet::AddUniqueDatabases(TArray<const UPoseSearchDatabase*>& UniqueDatabases) const
{
	for (const TObjectPtr<const UPoseSearchDatabase>& Database : Databases)
	{
		if (Database)
		{
			UniqueDatabases.AddUnique(Database.Get());
		}
	}
}

#endif // WITH_EDITORONLY_DATA
