// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchResult.h"
#include "Animation/BlendSpace.h"
#include "Animation/AnimationAsset.h"
#include "PoseSearch/MultiAnimAsset.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchSchema.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchResult)

namespace UE::PoseSearch
{

/////////////////////////////////////////////////////
// FDatabasePoseIdx

const FSearchIndexAsset* FDatabasePoseIdx::GetSearchIndexAsset(bool bMandatory) const
{
	if (bMandatory)
	{
		if (!ensure(IsValid()))
		{
			return nullptr;
		}
	}
	else if (!IsValid())
	{
		return nullptr;
	}

	return &Database->GetSearchIndex().GetAssetForPose(PoseIdx);
}

const UAnimationAsset* FDatabasePoseIdx::GetCurrentResultAnimationAsset() const
{
	if (const FSearchIndexAsset* SearchIndexAsset = GetSearchIndexAsset())
	{
		return Database->GetDatabaseAnimationAsset(SearchIndexAsset->GetSourceAssetIdx())->GetAnimationAssetForRole(Database->Schema->GetDefaultRole());
	}
	return nullptr;
}

const UAnimationAsset* FDatabasePoseIdx::GetCurrentResultAnimationAsset(const FRole& Role) const
{
	if (const FSearchIndexAsset* SearchIndexAsset = GetSearchIndexAsset())
	{
		return Database->GetDatabaseAnimationAsset(SearchIndexAsset->GetSourceAssetIdx())->GetAnimationAssetForRole(Role);
	}
	return nullptr;
}

/////////////////////////////////////////////////////
// FSearchResult

bool FSearchResult::DebugValidate() const
{
	bool bIsValidated = true;

#if WITH_EDITOR && ENABLE_ANIM_DEBUG

	if (IsValid())
	{
		const FSearchIndex& SearchIndex = Database->GetSearchIndex();
		const FSearchIndexAsset& SearchIndexAsset = SearchIndex.GetAssetForPose(PoseIdx);
		if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAssetBase = Database->GetDatabaseAnimationAsset(SearchIndexAsset.GetSourceAssetIdx()))
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			const float RealAssetTime = AssetTime * SearchIndexAsset.GetToRealTimeFactor();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			const int32 RecalculatedPoseIdx = SearchIndexAsset.GetPoseIndexFromTime(RealAssetTime, Database->Schema->SampleRate);
			if (RecalculatedPoseIdx != PoseIdx)
			{
				bIsValidated = false;
			}
		}
		else
		{
			bIsValidated = false;
		}
	}

#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG

	return bIsValidated;
}

void FSearchResult::UpdateWithNormalizedTime(float NormalizedTime)
{
	ensure(DebugValidate());

	if (IsValid())
	{
		// for non blend spaces the real time corrisponds to the normalized time!
		const FSearchIndexAsset& SearchIndexAsset = Database->GetSearchIndex().GetAssetForPose(PoseIdx);
		const float RealTime = NormalizedTime * SearchIndexAsset.GetToRealTimeFactor();
		
#if WITH_EDITOR && DO_CHECK
		const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAssetBase = Database->GetDatabaseAnimationAsset(SearchIndexAsset.GetSourceAssetIdx());
		if (ensure(DatabaseAnimationAssetBase))
		{
			if (Cast<UBlendSpace>(DatabaseAnimationAssetBase->GetAnimationAssetForRole(Database->Schema->GetDefaultRole())))
			{
				const float PlayLength = DatabaseAnimationAssetBase->GetPlayLength(SearchIndexAsset.GetBlendParameters());
				if (PlayLength > UE_KINDA_SMALL_NUMBER)
				{
					// Asset player time for blendspaces is normalized [0, 1] so we need to convert 
					// to a real time before we advance it
					ensure(NormalizedTime >= 0.f && NormalizedTime <= 1.f);
					ensure(FMath::IsNearlyEqual(RealTime, NormalizedTime * PlayLength));
				}
				else
				{
					ensure(FMath::IsNearlyEqual(SearchIndexAsset.GetToRealTimeFactor(), 1.f));
				}
			}
			else
			{
				ensure(FMath::IsNearlyEqual(SearchIndexAsset.GetToRealTimeFactor(), 1.f));
			}
		}
#endif // WITH_EDITOR && DO_CHECK
		
		PoseIdx = SearchIndexAsset.GetPoseIndexFromTime(RealTime, Database->Schema->SampleRate);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		AssetTime = NormalizedTime;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		Reset();
	}
	
	ensure(DebugValidate());
}

void FSearchResult::UpdateWithRealTime(float RealTime)
{
	ensure(DebugValidate());

	if (IsValid())
	{
		const FSearchIndexAsset& SearchIndexAsset = Database->GetSearchIndex().GetAssetForPose(PoseIdx);
		ensure(SearchIndexAsset.GetToRealTimeFactor() > UE_KINDA_SMALL_NUMBER);
		const float NormalizedTime = RealTime / SearchIndexAsset.GetToRealTimeFactor();

#if WITH_EDITOR && DO_CHECK
		const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAssetBase = Database->GetDatabaseAnimationAsset(SearchIndexAsset.GetSourceAssetIdx());
		if (ensure(DatabaseAnimationAssetBase))
		{
			if (Cast<UBlendSpace>(DatabaseAnimationAssetBase->GetAnimationAssetForRole(Database->Schema->GetDefaultRole())))
			{
				const float PlayLength = DatabaseAnimationAssetBase->GetPlayLength(SearchIndexAsset.GetBlendParameters());
				if (PlayLength > UE_KINDA_SMALL_NUMBER)
				{
					// Asset player time for blendspaces is normalized [0, 1] so we need to convert 
					// to a real time before we advance it
					ensure(RealTime >= 0.f && RealTime <= PlayLength);
					ensure(FMath::IsNearlyEqual(NormalizedTime, PlayLength > UE_KINDA_SMALL_NUMBER ? RealTime / PlayLength : 0.f));
				}
				else
				{
					ensure(FMath::IsNearlyEqual(SearchIndexAsset.GetToRealTimeFactor(), 1.f));
				}
			}
			else
			{
				ensure(FMath::IsNearlyEqual(SearchIndexAsset.GetToRealTimeFactor(), 1.f));
			}
		}
#endif // WITH_EDITOR && DO_CHECK

		PoseIdx = SearchIndexAsset.GetPoseIndexFromTime(RealTime, Database->Schema->SampleRate);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		AssetTime = NormalizedTime;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		Reset();
	}
	
	ensure(DebugValidate());
}

bool FSearchResult::IsEventSearchFromTag(const FGameplayTag& EventTag) const
{
	if (ensure(IsValid()))
	{
		return Database->GetSearchIndex().EventData.IsPoseFromEventTag(EventPoseIdx, EventTag);
	}
	return false;
}

float FSearchResult::CalculateTimeToEvent() const
{
	if (!ensure(IsValid() && IsEventSearchResult()))
	{
		return 0.f;
	}
	
	const FSearchIndex& SearchIndex = Database->GetSearchIndex();

	ensure(SearchIndex.PoseMetadata[PoseIdx].GetAssetIndex() == SearchIndex.PoseMetadata[EventPoseIdx].GetAssetIndex());
	const FSearchIndexAsset& SearchIndexAsset = SearchIndex.GetAssetForPose(PoseIdx);

	// @todo: the mathc here can be simplified between GetDeltaTimeBetweenPoseIndexes and GetTimeFromPoseIndex methods
	// DeltaTimeBetweenPoseAndEvent is the time in seconds between the event pose (EventPoseIdx) and the current pose (PoseIdx) taking into consideration looping.
	const float DeltaTimeBetweenPoseAndEvent = SearchIndexAsset.GetDeltaTimeBetweenPoseIndexes(PoseIdx, EventPoseIdx, Database->Schema->SampleRate);
	// PoseQuantizedTime is the quantized time associated to PoseIdx
	const float PoseQuantizedTime = SearchIndexAsset.GetTimeFromPoseIndex(PoseIdx, Database->Schema->SampleRate);
	// AssetTime is the current search result time, that differs from PoseQuantizedTime in case this search result is from a continuing pose search
	// we calculate the QuantizationError as differnece between AssetTime and PoseQuantizedTime, time in seconds that this search result drifted away from the quantized time of the associated PoseIdx
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const float QuantizationError = AssetTime - PoseQuantizedTime;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// time to event can be negative if PoseIdx already passed EventPoseIdx and the asset is not looping
	const float TimeToEvent = DeltaTimeBetweenPoseAndEvent - QuantizationError;
	return TimeToEvent;
}

void FSearchResult::InitFrom(const FPoseSearchBlueprintResult& BlueprintResult)
{
	PoseCost = FPoseSearchCost(BlueprintResult.SearchCost);
	PoseIdx = BlueprintResult.SelectedDatabase ? BlueprintResult.SelectedDatabase->GetPoseIndex(BlueprintResult.SelectedAnim.Get(), BlueprintResult.SelectedTime, BlueprintResult.bIsMirrored, BlueprintResult.BlendParameters) : INDEX_NONE;
	EventPoseIdx = INDEX_NONE;
	Database = BlueprintResult.SelectedDatabase;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	AssetTime = BlueprintResult.SelectedTime;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	bIsContinuingPoseSearch = BlueprintResult.bIsContinuingPoseSearch;

#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
	BruteForcePoseCost = FPoseSearchCost();
	BestPosePos = 0;
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
}

/////////////////////////////////////////////////////
// FSearchResults_Single
FSearchResults_Single::FSearchResults_Single()
{
	UpdateWith = [this](const FSearchResult& SearchResult)
		{
			ensure(SearchResult.IsValid());
			if (SearchResult.PoseCost < SingleSearchResult.PoseCost)
			{
				SingleSearchResult = SearchResult;
			}
		};

	FinalizeResults = [this]()
		{
			if (SingleSearchResult.IsValid() && !SingleSearchResult.IsAssetTimeValid())
			{
				SingleSearchResult.SetAssetTime(SingleSearchResult.Database->GetNormalizedAssetTime(SingleSearchResult.PoseIdx));
				ensure(SingleSearchResult.DebugValidate());
			}
		};

	GetBestResult = [this]()
		{
			return SingleSearchResult;
		};
	
	ShouldPerformSearch = [this](float SearchMinimumCost)
		{
			return SingleSearchResult.PoseCost > SearchMinimumCost;
		};

	IterateOverSearchResults = [this](const TFunctionRef<bool(FSearchResult& SearchResult)> IterateOverSearchResultsFunction)
		{
			return IterateOverSearchResultsFunction(SingleSearchResult);
		};
}

/////////////////////////////////////////////////////
// FSearchResults_Multi
FSearchResults_Multi::FSearchResults_Multi(int32 InMaxNumberOfWantedResults)
	: MaxNumberOfWantedResults(InMaxNumberOfWantedResults)
{
	ensure(MaxNumberOfWantedResults > 0); // or else we need to refine the logic in UpdateWith
	SearchResults.Reserve(MaxNumberOfWantedResults);

	UpdateWith = [this](const FSearchResult& SearchResult)
		{
			ensure(SearchResult.IsValid());
			if (SearchResults.Num() < MaxNumberOfWantedResults)
			{
				SearchResults.Add(SearchResult);

				// @todo: speed up this sort, since SearchResults started alreay sorted!
				SearchResults.Sort([](const FSearchResult& A, const FSearchResult& B)
					{
						return A.PoseCost < B.PoseCost;
					});
			}
			else
			{
				FSearchResult& WorstResult = SearchResults.Last();
				if (SearchResult.PoseCost < WorstResult.PoseCost)
				{
					WorstResult = SearchResult;

					// @todo: speed up this sort, since SearchResults started alreay sorted!
					SearchResults.Sort([](const FSearchResult& A, const FSearchResult& B)
						{
							return A.PoseCost < B.PoseCost;
						});
				}
			}
		};

	FinalizeResults = [this]()
		{
			for (FSearchResult& SearchResult : SearchResults)
			{
				ensure(SearchResult.IsValid());
				if (!SearchResult.IsAssetTimeValid())
				{
					SearchResult.SetAssetTime(SearchResult.Database->GetNormalizedAssetTime(SearchResult.PoseIdx));
					ensure(SearchResult.DebugValidate());
				}
			}
		};

	GetBestResult = [this]()
		{
			if (!SearchResults.IsEmpty())
			{
				return SearchResults[0];
			}
			return FSearchResult();
		};
	
	ShouldPerformSearch = [this](float SearchMinimumCost)
		{
			if (SearchResults.IsEmpty())
			{
				return true;
			}

			return SearchResults.Last().PoseCost > SearchMinimumCost;
		};

	IterateOverSearchResults = [this](const TFunctionRef<bool(FSearchResult& SearchResult)> IterateOverSearchResultsFunction)
		{
			for (FSearchResult& SearchResult : SearchResults)
			{
				if (IterateOverSearchResultsFunction(SearchResult))
				{
					return true;
				}
			}
			return false;
		};
}

/////////////////////////////////////////////////////
// FSearchResults_AssetBests
FSearchResults_AssetBests::FSearchResults_AssetBests()
{
	UpdateWith = [this](const FSearchResult& SearchResult)
		{
			ensure(SearchResult.IsValid());
			if (const FSearchIndexAsset* SearchIndexAsset = SearchResult.GetSearchIndexAsset())
			{
				FSourceAssetIdxToBestSearchResult& SourceAssetIdxToBestSearchResult = PerDatabaseSourceAssetIdxToBestSearchResult.FindOrAdd(SearchResult.Database.Get());
				FSearchResult& BestSearchResultForSourceAssetIdx = SourceAssetIdxToBestSearchResult.FindOrAdd(SearchIndexAsset->GetSourceAssetIdx());
				if (SearchResult.PoseCost < BestSearchResultForSourceAssetIdx.PoseCost)
				{
					BestSearchResultForSourceAssetIdx = SearchResult;
				}
			}
		};

	FinalizeResults = [this]()
		{
			for (TPair<const UPoseSearchDatabase*, FSourceAssetIdxToBestSearchResult>& PerDatabaseSourceAssetIdxToBestSearchResultPair : PerDatabaseSourceAssetIdxToBestSearchResult)
			{
				for (TPair<int32, FSearchResult>& SourceAssetIdxToBestSearchResultPair : PerDatabaseSourceAssetIdxToBestSearchResultPair.Value)
				{
					FSearchResult& BestSearchResultForSourceAssetIdx = SourceAssetIdxToBestSearchResultPair.Value;
					ensure(BestSearchResultForSourceAssetIdx.IsValid());
					if (!BestSearchResultForSourceAssetIdx.IsAssetTimeValid())
					{
						BestSearchResultForSourceAssetIdx.SetAssetTime(BestSearchResultForSourceAssetIdx.Database->GetNormalizedAssetTime(BestSearchResultForSourceAssetIdx.PoseIdx));
						ensure(BestSearchResultForSourceAssetIdx.DebugValidate());
					}
				}
			}
		};

	GetBestResult = [this]()
		{
			FSearchResult BestSearchResult;
			for (const TPair<const UPoseSearchDatabase*, FSourceAssetIdxToBestSearchResult>& PerDatabaseSourceAssetIdxToBestSearchResultPair : PerDatabaseSourceAssetIdxToBestSearchResult)
			{
				for (const TPair<int32, FSearchResult>& SourceAssetIdxToBestSearchResultPair : PerDatabaseSourceAssetIdxToBestSearchResultPair.Value)
				{
					const FSearchResult& BestSearchResultForSourceAssetIdx = SourceAssetIdxToBestSearchResultPair.Value;
					if (BestSearchResultForSourceAssetIdx.PoseCost < BestSearchResult.PoseCost)
					{
						BestSearchResult = BestSearchResultForSourceAssetIdx;
					}
				}
			}
			return BestSearchResult;
		};
	
	ShouldPerformSearch = [this](float SearchMinimumCost)
		{
			return true;
		};

	IterateOverSearchResults = [this](const TFunctionRef<bool(FSearchResult& SearchResult)> IterateOverSearchResultsFunction)
		{
			for (TPair<const UPoseSearchDatabase*, FSourceAssetIdxToBestSearchResult>& PerDatabaseSourceAssetIdxToBestSearchResultPair : PerDatabaseSourceAssetIdxToBestSearchResult)
			{
				for (TPair<int32, FSearchResult>& SourceAssetIdxToBestSearchResultPair : PerDatabaseSourceAssetIdxToBestSearchResultPair.Value)
				{
					if (IterateOverSearchResultsFunction(SourceAssetIdxToBestSearchResultPair.Value))
					{
						return true;
					}
				}
			}
			return false;
		};
}

const FSearchResult* FSearchResults_AssetBests::FindSearchResultFor(const UPoseSearchDatabase* Database, int32 SourceAssetIdx) const
{
	if (const FSourceAssetIdxToBestSearchResult* SourceAssetIdxToBestSearchResult = PerDatabaseSourceAssetIdxToBestSearchResult.Find(Database))
	{
		return SourceAssetIdxToBestSearchResult->Find(SourceAssetIdx);
	}
	return nullptr;
}

void FSearchResults_AssetBests::Shrink(int32 MaxNumResults)
{
	int32 NumResults = 0;
	for (const TPair<const UPoseSearchDatabase*, FSourceAssetIdxToBestSearchResult>& PerDatabaseSourceAssetIdxToBestSearchResultPair : PerDatabaseSourceAssetIdxToBestSearchResult)
	{
		NumResults += PerDatabaseSourceAssetIdxToBestSearchResultPair.Value.Num();
	}

	if (NumResults > MaxNumResults)
	{
		FMemMark Mark(FMemStack::Get());

		struct FSearchResultHeapCompare
		{
			bool operator()(const FSearchResult& A, const FSearchResult& B) const
			{
				// using > to create a max heap where BestSearchResults.HeapTop().Cost is the greatest Cost (to behave like the std::priority_queue)
				return A.PoseCost > B.PoseCost;
			}
		};
		
		// max heap of FSearchResult(s).
		TArray<FSearchResult, TInlineAllocator<8, TMemStackAllocator<>>> BestSearchResults;
		BestSearchResults.Reserve(MaxNumResults);

		for (const TPair<const UPoseSearchDatabase*, FSourceAssetIdxToBestSearchResult>& PerDatabaseSourceAssetIdxToBestSearchResultPair : PerDatabaseSourceAssetIdxToBestSearchResult)
		{
			for (const TPair<int32, FSearchResult>& SourceAssetIdxToBestSearchResultPair : PerDatabaseSourceAssetIdxToBestSearchResultPair.Value)
			{
				const FSearchResult& SearchResult = SourceAssetIdxToBestSearchResultPair.Value;
				if (BestSearchResults.Num() < MaxNumResults)
				{
					BestSearchResults.HeapPush(SearchResult, FSearchResultHeapCompare());
				}
				else if (SearchResult.PoseCost < BestSearchResults.HeapTop().PoseCost)
				{
					// popping the max heap (BestSearchResults) head since it costs more than the FSearchResult we're about to insert
					FSearchResult PoppedSearchResult;
					BestSearchResults.HeapPop(PoppedSearchResult, FSearchResultHeapCompare(), EAllowShrinking::No);
					BestSearchResults.HeapPush(SearchResult, FSearchResultHeapCompare());
				}
			}
		}

		PerDatabaseSourceAssetIdxToBestSearchResult.Reset();
		for (const FSearchResult& BestSearchResult : BestSearchResults)
		{
			const FSearchIndexAsset* SearchIndexAsset = BestSearchResult.GetSearchIndexAsset();
			if (ensure(SearchIndexAsset))
			{
				FSourceAssetIdxToBestSearchResult& SourceAssetIdxToBestSearchResult = PerDatabaseSourceAssetIdxToBestSearchResult.FindOrAdd(BestSearchResult.Database.Get());
				FSearchResult& BestSearchResultForSourceAssetIdx = SourceAssetIdxToBestSearchResult.FindOrAdd(SearchIndexAsset->GetSourceAssetIdx());
				BestSearchResultForSourceAssetIdx = BestSearchResult;
			}
		}
	}
}

} // namespace UE::PoseSearch

bool FPoseSearchBlueprintResult::InitFrom(const UE::PoseSearch::FSearchResult& SearchResult, float InWantedPlayRate)
{
	using namespace UE::PoseSearch;

	if (const FSearchIndexAsset* SearchIndexAsset = SearchResult.GetSearchIndexAsset())
	{
		const UPoseSearchDatabase* Database = SearchResult.Database.Get();

		const FPoseSearchDatabaseAnimationAssetBase* DatabaseAsset = Database->GetDatabaseAnimationAsset(SearchIndexAsset->GetSourceAssetIdx());
		if (ensure(DatabaseAsset))
		{
			SelectedAnim = DatabaseAsset->GetAnimationAsset();
			SelectedTime = SearchResult.GetAssetTime();
			bIsContinuingPoseSearch = SearchResult.bIsContinuingPoseSearch;
			WantedPlayRate = InWantedPlayRate;
			bLoop = SearchIndexAsset->IsLooping();
			bIsMirrored = SearchIndexAsset->IsMirrored();
			BlendParameters = SearchIndexAsset->GetBlendParameters();
			SelectedDatabase = Database;
			SearchCost = SearchResult.PoseCost;
			Role = Database->Schema->GetDefaultRole();

			bIsInteraction = false;
			ActorRootTransforms.Reset();
			ActorRootBoneTransforms.Reset();
			AnimContexts.Reset();
			return true;
		}
	}
	
	*this = FPoseSearchBlueprintResult();
	return false;
}

UAnimationAsset* FPoseSearchBlueprintResult::GetAnimationAssetForRole()
{
	if (UAnimationAsset* AnimationAsset = Cast<UAnimationAsset>(SelectedAnim))
	{
		return AnimationAsset;
	}
	
	if (UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(SelectedAnim))
	{
		return MultiAnimAsset->GetAnimationAsset(Role);
	}

	return nullptr;
}

bool FPoseSearchBlueprintResult::operator==(const FPoseSearchBlueprintResult& Other) const
{
	return	
		SelectedAnim == Other.SelectedAnim &&
		SelectedTime == Other.SelectedTime &&
		bIsContinuingPoseSearch == Other.bIsContinuingPoseSearch &&
		WantedPlayRate == Other.WantedPlayRate &&
		bLoop == Other.bLoop &&
		bIsMirrored == Other.bIsMirrored &&
		BlendParameters == Other.BlendParameters &&
		SelectedDatabase == Other.SelectedDatabase &&
		SearchCost == Other.SearchCost &&
		bIsInteraction == Other.bIsInteraction &&
		Role == Other.Role &&
		AnimContexts == Other.AnimContexts &&

		// FTransform(s) don't have operator == so we need to do do some custom work here :/
		ActorRootTransforms.Num() == Other.ActorRootTransforms.Num() &&
		ActorRootBoneTransforms.Num() == Other.ActorRootBoneTransforms.Num() &&
		FMemory::Memcmp(ActorRootTransforms.GetData(), Other.ActorRootTransforms.GetData(), sizeof(FTransform) * ActorRootTransforms.Num()) == 0 &&
		FMemory::Memcmp(ActorRootBoneTransforms.GetData(), Other.ActorRootBoneTransforms.GetData(), sizeof(FTransform) * ActorRootBoneTransforms.Num()) == 0;
}