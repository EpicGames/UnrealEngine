// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchSchema.h"
#include "Animation/MirrorDataTable.h"
#include "AnimationRuntime.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "PoseSearch/PoseSearchResult.h"
#include "PoseSearch/PoseSearchFeatureChannel_Padding.h"
#include "PoseSearch/PoseSearchFeatureChannel_PermutationTime.h"
#include "PoseSearch/PoseSearchFeatureChannel_Pose.h"
#include "PoseSearch/PoseSearchFeatureChannel_Trajectory.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchSchema)

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FPoseSearchRoledSkeleton::FPoseSearchRoledSkeleton(const FPoseSearchRoledSkeleton&) = default;
FPoseSearchRoledSkeleton& FPoseSearchRoledSkeleton::operator=(const FPoseSearchRoledSkeleton&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UPoseSearchSchema::AddChannel(UPoseSearchFeatureChannel* Channel)
{
	Channels.Add(Channel);
}

void UPoseSearchSchema::AddTemporaryChannel(UPoseSearchFeatureChannel* TemporaryChannel)
{
	TemporaryChannel->Finalize(this);
	FinalizedChannels.Add(TemporaryChannel);
}

TConstArrayView<float> UPoseSearchSchema::BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PoseSearch_BuildQuery);

	SearchContext.AddNewFeatureVectorBuilder(this);

	for (const TObjectPtr<UPoseSearchFeatureChannel>& ChannelPtr : GetChannels())
	{
		ChannelPtr->BuildQuery(SearchContext);
	}

	return SearchContext.EditFeatureVector();
}

void UPoseSearchSchema::AddSkeleton(USkeleton* Skeleton, UMirrorDataTable* MirrorDataTable, const UE::PoseSearch::FRole& Role)
{
	FPoseSearchRoledSkeleton& RoledSkeleton = Skeletons.AddDefaulted_GetRef();
	RoledSkeleton.Skeleton = Skeleton;
	RoledSkeleton.MirrorDataTable = MirrorDataTable;
	RoledSkeleton.Role = Role;
}

bool UPoseSearchSchema::AreSkeletonsCompatible(const UPoseSearchSchema* Other) const
{
	if (Skeletons.Num() != Other->Skeletons.Num())
	{
		return false;
	}

	for (int32 SkeletonIndex = 0; SkeletonIndex < Skeletons.Num(); ++SkeletonIndex)
	{
		if (Skeletons[SkeletonIndex].Skeleton != Other->Skeletons[SkeletonIndex].Skeleton)
		{
			return false;
		}

		if (Skeletons[SkeletonIndex].Role != Other->Skeletons[SkeletonIndex].Role)
		{
			return false;
		}
	}

	return true;
}

void UPoseSearchSchema::AddDefaultChannels()
{
	// defaulting UPoseSearchSchema for a meaningful locomotion setup
	AddChannel(NewObject<UPoseSearchFeatureChannel_Trajectory>(this, NAME_None, RF_Transactional));
	AddChannel(NewObject<UPoseSearchFeatureChannel_Pose>(this, NAME_None, RF_Transactional));
	Finalize();
}

const FPoseSearchRoledSkeleton* UPoseSearchSchema::GetRoledSkeleton(const UE::PoseSearch::FRole& Role) const
{
	for (const FPoseSearchRoledSkeleton& RoledSkeleton : Skeletons)
	{
		if (RoledSkeleton.Role == Role)
		{
			return &RoledSkeleton;
		}
	}
	return nullptr;
}

FPoseSearchRoledSkeleton* UPoseSearchSchema::GetRoledSkeleton(const UE::PoseSearch::FRole& Role)
{
	for (FPoseSearchRoledSkeleton& RoledSkeleton : Skeletons)
	{
		if (RoledSkeleton.Role == Role)
		{
			return &RoledSkeleton;
		}
	}
	return nullptr;
}

const UE::PoseSearch::FRole UPoseSearchSchema::GetDefaultRole() const
{
	if (!Skeletons.IsEmpty())
	{
		return Skeletons[0].Role;
	}
	return UE::PoseSearch::DefaultRole;
}

USkeleton* UPoseSearchSchema::GetSkeleton(const UE::PoseSearch::FRole& Role) const
{
	if (const FPoseSearchRoledSkeleton* RoledSkeleton = GetRoledSkeleton(Role))
	{
		return RoledSkeleton->Skeleton.Get();
	}
	return nullptr;
}

UMirrorDataTable* UPoseSearchSchema::GetMirrorDataTable(const UE::PoseSearch::FRole& Role) const
{
	if (const FPoseSearchRoledSkeleton* RoledSkeleton = GetRoledSkeleton(Role))
	{
		return RoledSkeleton->MirrorDataTable.Get();
	}
	return nullptr;
}

TConstArrayView<FBoneReference> UPoseSearchSchema::GetBoneReferences(const UE::PoseSearch::FRole& Role) const
{
	const FPoseSearchRoledSkeleton* RoledSkeleton = GetRoledSkeleton(Role);
	check(RoledSkeleton);
	return RoledSkeleton->BoneReferences;
}

int8 UPoseSearchSchema::AddBoneReference(const FBoneReference& BoneReference, const UE::PoseSearch::FRole& Role)
{
	return AddBoneReference(BoneReference, Role, true);
}


int8 UPoseSearchSchema::FindSchemaBoneIdx(const FName& BoneName, const UE::PoseSearch::FRole& Role, bool bDefaultWithRootBone) const
{
	using namespace UE::PoseSearch;

	const FPoseSearchRoledSkeleton* RoledSkeleton = GetRoledSkeleton(Role);
	if (!RoledSkeleton)
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchSchema::FindSchemaBoneIdx: couldn't find data for the requested Role '%ls' in UPoseSearchSchema '%ls'", *Role.ToString(), *GetNameSafe(this));
		return InvalidSchemaBoneIdx;
	}

	if (BoneName == NAME_None)
	{
		if (bDefaultWithRootBone)
		{
			if (RoledSkeleton->BoneReferences.IsEmpty())
			{
				return InvalidSchemaBoneIdx;
			}

			return RootSchemaBoneIdx;
		}
		
		return TrajectorySchemaBoneIdx;
	}

	for (int32 SchemaBoneIdx = 0; SchemaBoneIdx < RoledSkeleton->BoneReferences.Num(); ++SchemaBoneIdx)
	{
		const FBoneReference& BoneReference = RoledSkeleton->BoneReferences[SchemaBoneIdx];
		if (BoneReference.BoneName == BoneName)
		{
			check(SchemaBoneIdx < 128);
			return int8(SchemaBoneIdx);
		}
	}

	return InvalidSchemaBoneIdx;
}

int8 UPoseSearchSchema::AddBoneReference(const FBoneReference& BoneReference, const UE::PoseSearch::FRole& Role, bool bDefaultWithRootBone)
{
	using namespace UE::PoseSearch;

	FPoseSearchRoledSkeleton* RoledSkeleton = GetRoledSkeleton(Role);
	if (!RoledSkeleton)
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchSchema::AddBoneReference: couldn't find data for the requested Role '%ls' in UPoseSearchSchema '%ls'", *Role.ToString(), *GetNameSafe(this));
		return InvalidSchemaBoneIdx;
	}

	int32 SchemaBoneIdx = 0;
	const USkeleton* Skeleton = RoledSkeleton->Skeleton;
	if (!Skeleton)
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchSchema::AddBoneReference: couldn't find Skeleton with Role '%ls' in UPoseSearchSchema '%ls'", *Role.ToString(), *GetNameSafe(this));
		return InvalidSchemaBoneIdx;
	}

	bool bDefaultToRootBone = true;
	FBoneReference TempBoneReference = BoneReference;
	if (TempBoneReference.BoneName != NAME_None)
	{
		TempBoneReference.Initialize(Skeleton);
		if (!TempBoneReference.HasValidSetup())
		{
			UE_LOGF(LogPoseSearch, Error, "UPoseSearchSchema::AddBoneReference: couldn't initialize FBoneReference '%ls' with Skeleton '%ls' with Role '%ls' in UPoseSearchSchema '%ls'",
				*TempBoneReference.BoneName.ToString(), *GetNameSafe(Skeleton), *Role.ToString(), *GetNameSafe(this));
			return InvalidSchemaBoneIdx;
		}
	}
	else if (bDefaultWithRootBone)
	{
		TempBoneReference.BoneName = Skeleton->GetReferenceSkeleton().GetBoneName(int32(RootBoneIndexType));
		TempBoneReference.Initialize(Skeleton);
		check(TempBoneReference.HasValidSetup());
	}
	else
	{
		return TrajectorySchemaBoneIdx;
	}

	SchemaBoneIdx = RoledSkeleton->BoneReferences.AddUnique(TempBoneReference);
	check(SchemaBoneIdx >= 0 && SchemaBoneIdx < 128);
	return int8(SchemaBoneIdx);
}

int8 UPoseSearchSchema::AddCurveReference(const FName& CurveReference, const UE::PoseSearch::FRole& Role)
{
	using namespace UE::PoseSearch;

	FPoseSearchRoledSkeleton* RoledSkeleton = GetRoledSkeleton(Role);
	if (!RoledSkeleton)
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchSchema::AddCurveReference: couldn't find data for the requested Role '%ls' in UPoseSearchSchema '%ls'", *Role.ToString(), *GetNameSafe(this));
		return InvalidSchemaCurveIdx;
	}

	const USkeleton* Skeleton = RoledSkeleton->Skeleton;
	if (!Skeleton)
	{
		UE_LOGF(LogPoseSearch, Error, "UPoseSearchSchema::AddCurveReference: couldn't find Skeleton with Role '%ls' in UPoseSearchSchema '%ls'", *Role.ToString(), *GetNameSafe(this));
		return InvalidSchemaCurveIdx;
	}

	// Curves are loosely bound, so there's no guarantee this curve will ever exist in any of the assets indexed by the database.
	const int32 CurveIdx = RoledSkeleton->RequiredCurves.AddUnique(CurveReference);
	check(CurveIdx >= 0 && CurveIdx < 128);
	return int8(CurveIdx);
}

void UPoseSearchSchema::ResetFinalize()
{
	for (FPoseSearchRoledSkeleton& RoledSkeleton : Skeletons)
	{
		RoledSkeleton.BoneReferences.Reset();
	}

	FinalizedChannels.Reset();
	SchemaCardinality = 0;
}

void UPoseSearchSchema::Finalize()
{
	using namespace UE::PoseSearch;

	ResetFinalize();

	// adding as first bone reference the root bone
	for (int32 RoledSkeletonIndex = 0; RoledSkeletonIndex < Skeletons.Num(); ++RoledSkeletonIndex)
	{
		const FPoseSearchRoledSkeleton& RoledSkeleton = Skeletons[RoledSkeletonIndex];
		for (int32 ComparisonIndex = RoledSkeletonIndex + 1; ComparisonIndex < Skeletons.Num(); ++ComparisonIndex)
		{
			if (Skeletons[ComparisonIndex].Role == RoledSkeleton.Role)
			{
				UE_LOGF(LogPoseSearch, Error, "UPoseSearchSchema::Finalize: couldn't Finalize '%ls' because of duplicate Role '%ls' in Skeletons", *GetNameSafe(this), *RoledSkeleton.Role.ToString());

				ResetFinalize();
				return;
			}
		}

		const int8 SchemaBoneIdx = AddBoneReference(FBoneReference(), RoledSkeleton.Role);
		if (SchemaBoneIdx != RootSchemaBoneIdx)
		{
			UE_LOGF(LogPoseSearch, Error, "UPoseSearchSchema::Finalize: couldn't Finalize '%ls' because couldn't initialize root bone properly", *GetNameSafe(this));

			ResetFinalize();
			return;
		}
	}

	for (const TObjectPtr<UPoseSearchFeatureChannel>& ChannelPtr : Channels)
	{
		if (ChannelPtr)
		{
			FinalizedChannels.Add(ChannelPtr);
			if (!ChannelPtr->Finalize(this))
			{
				#if WITH_EDITOR
				TLabelBuilder LabelBuilder;
				FString Label = ChannelPtr->GetLabel(LabelBuilder).ToString();
				UE_LOGF(LogPoseSearch, Error, "UPoseSearchSchema::Finalize: couldn't Finalize '%ls' because of Channel '%ls'", *GetNameSafe(this), *Label);
				#else // WITH_EDITOR
				UE_LOGF(LogPoseSearch, Error, "UPoseSearchSchema::Finalize: couldn't Finalize '%ls' because of Channel '%ls'", *GetNameSafe(this), *GetNameSafe(ChannelPtr));
				#endif // WITH_EDITOR

				ResetFinalize();
				return;
			}
		}
	}

	// AddDependentChannels can add channels to FinalizedChannels, so we need a while loop
	int32 ChannelIndex = 0;
	while (ChannelIndex < FinalizedChannels.Num())
	{
		FinalizedChannels[ChannelIndex]->AddDependentChannels(this);
		++ChannelIndex;
	}

	for (const TObjectPtr<UPoseSearchFeatureChannel>& ChannelPtr : FinalizedChannels)
	{
		check(ChannelPtr);
		if (ChannelPtr->GetPermutationTimeType() != EPermutationTimeType::UseSampleTime)
		{
			// there's at least one channel that uses UsePermutationTime or UseSampleToPermutationTime: we automatically add a UPoseSearchFeatureChannel_PermutationTime if not already in the schema
			UPoseSearchFeatureChannel_PermutationTime::FindOrAddToSchema(this);
			break;
		}
	}
	
	// adding padding if required
	if (bAddDataPadding)
	{
		// calculating how many floats of padding are required to make the data 16 bytes padded
		const int32 PaddingSize = SchemaCardinality % (16 / sizeof(float));
		if (PaddingSize > 0)
		{
			UPoseSearchFeatureChannel_Padding::AddToSchema(this, PaddingSize);
		}
	}
}

void UPoseSearchSchema::PostLoad()
{
	Super::PostLoad();

	for (FPoseSearchRoledSkeleton& Skeleton : Skeletons)
	{
		if (Skeleton.MirrorDataTable)
		{
			// adding a ConditionalPostLoad dependency to UMirrorDataTable, that via UMirrorDataTable::FillMirrorArrays
			// populates UMirrorDataTable::BoneToMirrorBoneIndex used in UPoseSearchSchema::Finalize 
			Skeleton.MirrorDataTable->ConditionalPostLoad();
		}
	}

	Finalize();
}

#if WITH_EDITOR
void UPoseSearchSchema::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Finalize();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif
