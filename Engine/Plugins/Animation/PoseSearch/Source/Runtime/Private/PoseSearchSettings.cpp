// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchSettings.h"

UPoseSearchSettings::UPoseSearchSettings()
{
	CategoryName = TEXT("Plugins");
}

const UPoseSearchSettings& UPoseSearchSettings::Get()
{
	UPoseSearchSettings* MutableCDO = GetMutableDefault<UPoseSearchSettings>();
	check(MutableCDO != nullptr)

	return *MutableCDO;
}
