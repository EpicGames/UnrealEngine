// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchTrajectoryLibrary.h"
#include "PoseSearch/PoseSearchTrajectoryPredictor.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimMontage.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/Engine.h"
#include "GameFramework/Character.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "Kismet/KismetMathLibrary.h"
#include "Animation/TrajectoryTypes.h"
#include "MotionWarpingComponent.h"
#include "MotionWarpingMontageTrajectoryAdapter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchTrajectoryLibrary)

// @todo: remove GVarPoseSearchTrajectoryFacingFromMesh once the behavior is enabled by defaulted
static bool GVarPoseSearchTrajectoryFacingFromMesh = true;
#if ENABLE_ANIM_DEBUG
static FAutoConsoleVariableRef CVarPoseSearchTrajectoryFacingFromMesh(TEXT("a.PoseSearchTrajectory.FacingFromMesh")
	, GVarPoseSearchTrajectoryFacingFromMesh
	, TEXT("Always generating the trajectory in mesh space rather than using the Character.View.Rotation.Yaw when the character is set to orienting the rotation to movement"));
#endif // ENABLE_ANIM_DEBUG

static bool GVarPoseSearchTrajectoryUseWarpedMontage = false;
static FAutoConsoleVariableRef CVarPoseSearchTrajectoryUseWarpedMontage(TEXT("a.PoseSearchTrajectory.UseWarpedMontage")
	, GVarPoseSearchTrajectoryUseWarpedMontage
	, TEXT("Experimental. If a montage is playing, use that montage's warped root motion for generated trajectories."));

bool FPoseSearchTrajectoryData::UpdateData(
	float DeltaTime,
	const FAnimInstanceProxy& AnimInstanceProxy,
	FDerived& TrajectoryDataDerived,
	FState& TrajectoryDataState) const
{
	return UpdateData(DeltaTime, AnimInstanceProxy.GetAnimInstanceObject(), TrajectoryDataDerived, TrajectoryDataState);
}

bool FPoseSearchTrajectoryData::UpdateData(
	float DeltaTime,
	const UObject* Context,
	FDerived& TrajectoryDataDerived,
	FState& TrajectoryDataState) const
{
	const ACharacter* Character = Cast<ACharacter>(Context);
	if (!Character)
	{
		if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(Context))
		{
			Character = Cast<ACharacter>(AnimInstance->GetOwningActor());
		}
		else if (const UActorComponent* AnimNextComponent = Cast<UActorComponent>(Context))
		{
			Character = Cast<ACharacter>(AnimNextComponent->GetOwner());
		}
		
		if (!Character)
		{
			return false;
		}
	}

	const UCharacterMovementComponent* MoveComp = Character->GetCharacterMovement();
	const USkeletalMeshComponent* MeshComp = Character->GetMesh();
	if (!MoveComp || !MeshComp)
	{
		return false;
	}

	TrajectoryDataDerived.MaxSpeed = FMath::Max(MoveComp->GetMaxSpeed() * MoveComp->GetAnalogInputModifier(), MoveComp->GetMinAnalogSpeed());
	TrajectoryDataDerived.BrakingDeceleration = FMath::Max(0.f, MoveComp->GetMaxBrakingDeceleration());
	TrajectoryDataDerived.BrakingSubStepTime = MoveComp->BrakingSubStepTime;
	TrajectoryDataDerived.bOrientRotationToMovement = MoveComp->bOrientRotationToMovement;

	TrajectoryDataDerived.Velocity = MoveComp->Velocity;
	TrajectoryDataDerived.Acceleration = MoveComp->GetCurrentAcceleration();
	
	// @TODO: Consider if our trajectory implies we will become falling mid future trajectory, rather than rely only on initial state
	TrajectoryDataDerived.bStepGroundPrediction = !MoveComp->IsFalling() && !MoveComp->IsFlying();

	if (TrajectoryDataDerived.Acceleration.IsZero())
	{
		TrajectoryDataDerived.Friction = MoveComp->bUseSeparateBrakingFriction ? MoveComp->BrakingFriction : MoveComp->GroundFriction;
		const float FrictionFactor = FMath::Max(0.f, MoveComp->BrakingFrictionFactor);
		TrajectoryDataDerived.Friction = FMath::Max(0.f, TrajectoryDataDerived.Friction * FrictionFactor);
	}
	else
	{
		TrajectoryDataDerived.Friction = MoveComp->GroundFriction;
	}

	const float DesiredControllerYaw = Character->GetViewRotation().Yaw;
		
	const float DesiredYawDelta = DesiredControllerYaw - TrajectoryDataState.DesiredControllerYawLastUpdate;
	TrajectoryDataState.DesiredControllerYawLastUpdate = DesiredControllerYaw;

	if (DeltaTime > UE_SMALL_NUMBER)
	{
		// An AnimInstance might call this during an AnimBP recompile with 0 delta time, so we don't update ControllerYawRate
		TrajectoryDataDerived.ControllerYawRate = FRotator::NormalizeAxis(DesiredYawDelta) / DeltaTime;
		if (MaxControllerYawRate >= 0.f)
		{
			TrajectoryDataDerived.ControllerYawRate = FMath::Sign(TrajectoryDataDerived.ControllerYawRate) * FMath::Min(FMath::Abs(TrajectoryDataDerived.ControllerYawRate), MaxControllerYawRate);
		}
	}

	TrajectoryDataDerived.Position = MeshComp->GetComponentLocation();
	TrajectoryDataDerived.MeshCompRelativeRotation = MeshComp->GetRelativeRotation().Quaternion();
	
	if (GVarPoseSearchTrajectoryFacingFromMesh || TrajectoryDataDerived.bOrientRotationToMovement)
	{
		TrajectoryDataDerived.Facing = MeshComp->GetComponentTransform().GetRotation();
	}
	else
	{
		TrajectoryDataDerived.Facing = FQuat::MakeFromRotator(FRotator(0,TrajectoryDataState.DesiredControllerYawLastUpdate,0)) * TrajectoryDataDerived.MeshCompRelativeRotation;
	}

	return true;
}

FVector FPoseSearchTrajectoryData::StepCharacterMovementGroundPrediction(
	float DeltaTime,
	const FVector& InVelocity,
	const FVector& InAcceleration,
	const FDerived& TrajectoryDataDerived) const
{
	FVector OutVelocity = InVelocity;

	// Braking logic is copied from UCharacterMovementComponent::ApplyVelocityBraking()
	if (InAcceleration.IsZero())
	{
		if (InVelocity.IsZero())
		{
			return FVector::ZeroVector;
		}

		const bool bZeroFriction = (TrajectoryDataDerived.Friction == 0.f);
		const bool bZeroBraking = (TrajectoryDataDerived.BrakingDeceleration == 0.f);

		if (bZeroFriction && bZeroBraking)
		{
			return InVelocity;
		}

		float RemainingTime = DeltaTime;
		const float MaxTimeStep = FMath::Clamp(TrajectoryDataDerived.BrakingSubStepTime, 1.0f / 75.0f, 1.0f / 20.0f);

		const FVector PrevLinearVelocity = OutVelocity;
		const FVector RevAccel = (bZeroBraking ? FVector::ZeroVector : (-TrajectoryDataDerived.BrakingDeceleration * OutVelocity.GetSafeNormal()));

		// Decelerate to brake to a stop
		while (RemainingTime >= UCharacterMovementComponent::MIN_TICK_TIME)
		{
			// Zero friction uses constant deceleration, so no need for iteration.
			const float dt = ((RemainingTime > MaxTimeStep && !bZeroFriction) ? FMath::Min(MaxTimeStep, RemainingTime * 0.5f) : RemainingTime);
			RemainingTime -= dt;

			// apply friction and braking
			OutVelocity = OutVelocity + ((-TrajectoryDataDerived.Friction) * OutVelocity + RevAccel) * dt;

			// Don't reverse direction
			if ((OutVelocity | PrevLinearVelocity) <= 0.f)
			{
				OutVelocity = FVector::ZeroVector;
				return OutVelocity;
			}
		}

		// Clamp to zero if nearly zero, or if below min threshold and braking
		const float VSizeSq = OutVelocity.SizeSquared();
		if (VSizeSq <= KINDA_SMALL_NUMBER || (!bZeroBraking && VSizeSq <= FMath::Square(UCharacterMovementComponent::BRAKE_TO_STOP_VELOCITY)))
		{
			OutVelocity = FVector::ZeroVector;
		}
	}
	// Acceleration logic is copied from  UCharacterMovementComponent::CalcVelocity
	else
	{
		const FVector AccelDir = InAcceleration.GetSafeNormal();
		const float VelSize = OutVelocity.Size();

		OutVelocity = OutVelocity - (OutVelocity - AccelDir * VelSize) * FMath::Min(DeltaTime * TrajectoryDataDerived.Friction, 1.f);

		OutVelocity += InAcceleration * DeltaTime;
		OutVelocity = OutVelocity.GetClampedToMaxSize(TrajectoryDataDerived.MaxSpeed);
	}

	return OutVelocity;
}

void UPoseSearchTrajectoryLibrary::InitTrajectorySamples(FTransformTrajectory& Trajectory, FVector DefaultPosition, FQuat DefaultFacing, const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling, float DeltaTime)
{
	const int32 NumHistorySamples = TrajectoryDataSampling.NumHistorySamples;
	const int32 NumPredictionSamples = TrajectoryDataSampling.NumPredictionSamples;

	// History + current sample + prediction
	const int32 TotalNumSamples = NumHistorySamples + 1 + NumPredictionSamples;

	if (Trajectory.Samples.Num() != TotalNumSamples)
	{
		Trajectory.Samples.SetNumUninitialized(TotalNumSamples);

		// Initialize history samples
		const float SecondsPerHistorySample = FMath::Max(TrajectoryDataSampling.SecondsPerHistorySample, 0.f);
		for (int32 i = 0; i < NumHistorySamples; ++i)
		{
			Trajectory.Samples[i].Position = DefaultPosition;
			Trajectory.Samples[i].Facing = DefaultFacing;
			Trajectory.Samples[i].TimeInSeconds = SecondsPerHistorySample * (i - NumHistorySamples - 1);
		}

		// Initialize current sample and prediction
		const float SecondsPerPredictionSample = FMath::Max(TrajectoryDataSampling.SecondsPerPredictionSample, 0.f);
		for (int32 i = NumHistorySamples; i < Trajectory.Samples.Num(); ++i)
		{
			Trajectory.Samples[i].Position = DefaultPosition;
			Trajectory.Samples[i].Facing = DefaultFacing;
			Trajectory.Samples[i].TimeInSeconds = SecondsPerPredictionSample * (i - NumHistorySamples) + DeltaTime;
		}
	}
}

// Deprecated
void UPoseSearchTrajectoryLibrary::UpdateHistory_TransformHistory(
	FTransformTrajectory& Trajectory,
	FVector CurrentPosition,
	FVector CurrentVelocity,
	const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling,
	float DeltaTime,
	float CurrentTime)
{
	UpdateHistory_TransformHistory(Trajectory, CurrentPosition, CurrentVelocity, TrajectoryDataSampling, DeltaTime);
}

void UPoseSearchTrajectoryLibrary::UpdateHistory_TransformHistory(FTransformTrajectory& Trajectory,
                                                                  FVector CurrentPosition,
                                                                  FVector CurrentVelocity,
                                                                  const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling,
                                                                  float DeltaTime)
{
	UpdateHistory_TransformHistoryInternal(Trajectory, CurrentPosition, CurrentVelocity,
		FQuat::Identity, FVector::ZeroVector, /*bApplyAngularCorrection*/ false,
		TrajectoryDataSampling, DeltaTime);
}

void UPoseSearchTrajectoryLibrary::UpdateHistory_TransformHistoryWithRotation(FTransformTrajectory& Trajectory,
                                                                  FVector CurrentPosition,
                                                                  FVector CurrentVelocity,
                                                                  FQuat CurrentFacing,
                                                                  FVector CurrentAngularVelocityDegrees,
                                                                  const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling,
                                                                  float DeltaTime)
{
	UpdateHistory_TransformHistoryInternal(Trajectory, CurrentPosition, CurrentVelocity,
		CurrentFacing, CurrentAngularVelocityDegrees, /*bApplyAngularCorrection*/ true,
		TrajectoryDataSampling, DeltaTime);
}

void UPoseSearchTrajectoryLibrary::UpdateHistory_TransformHistoryInternal(FTransformTrajectory& Trajectory,
                                                                  FVector CurrentPosition,
                                                                  FVector CurrentVelocity,
                                                                  FQuat CurrentFacing,
                                                                  FVector CurrentAngularVelocityDegrees,
                                                                  bool bApplyAngularCorrection,
                                                                  const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling,
                                                                  float DeltaTime)
{
	const int32 NumHistorySamples = TrajectoryDataSampling.NumHistorySamples;
	if (NumHistorySamples > 0)
	{
		const float SecondsPerHistorySample = TrajectoryDataSampling.SecondsPerHistorySample;

		// Trajectory should include room for history + current + future
		// So num history samples needs to be less than the total number
		check(NumHistorySamples < Trajectory.Samples.Num());

		// Trajectory.Samples[NumHistorySamples] is last frame position! (assuming this is called every frame)
		const FVector CurrentTranslationFromMover = CurrentVelocity * DeltaTime;
		const FVector TranslationSinceLastFrame = CurrentPosition - Trajectory.Samples[NumHistorySamples].Position;
		const FVector InferredGroundTranslation = TranslationSinceLastFrame - CurrentTranslationFromMover;

		// Angular analogue of the translation ground-delta: subtract the character's rotational intent from the actual
		// frame-over-frame rotation to isolate the platform's angular contribution
		FQuat InferredGroundRotation = FQuat::Identity;
		if (bApplyAngularCorrection)
		{
			const FQuat LastFrameFacing = Trajectory.Samples[NumHistorySamples].Facing;
			const FQuat ActualRotationDelta = CurrentFacing * LastFrameFacing.Inverse();
			const FVector AngularIntentRadians = FMath::DegreesToRadians(CurrentAngularVelocityDegrees) * DeltaTime;
			const FQuat AngularIntent = FQuat::MakeFromRotationVector(AngularIntentRadians);
			InferredGroundRotation = (ActualRotationDelta * AngularIntent.Inverse()).GetNormalized();
		}

		auto ApplyGroundDelta = [&InferredGroundTranslation, &InferredGroundRotation, &CurrentPosition](FVector& InOutPosition, FQuat& InOutFacing)
		{
			const FVector Translated = InOutPosition + InferredGroundTranslation;
			InOutPosition = CurrentPosition + InferredGroundRotation.RotateVector(Translated - CurrentPosition);
			InOutFacing = (InferredGroundRotation * InOutFacing).GetNormalized();
		};

		const float TimeShift = Trajectory.Samples[NumHistorySamples].TimeInSeconds;	// This works out as the delta time of the last frame

		// Shift history Samples when it's time to record a new one.
		if (SecondsPerHistorySample <= 0.f || FMath::Abs(Trajectory.Samples[NumHistorySamples - 1].TimeInSeconds) >= SecondsPerHistorySample)
		{
			for (int32 Index = 0; Index < NumHistorySamples - 1; ++Index)
			{
				Trajectory.Samples[Index].TimeInSeconds = Trajectory.Samples[Index + 1].TimeInSeconds - TimeShift;
				Trajectory.Samples[Index].Position = Trajectory.Samples[Index + 1].Position;
				Trajectory.Samples[Index].Facing = Trajectory.Samples[Index + 1].Facing;
				ApplyGroundDelta(Trajectory.Samples[Index].Position, Trajectory.Samples[Index].Facing);
			}

			// Adding a new history record
			// Copy over the last frame's current transform (stored at i == NumHistorySamples) into a sample at t = 0
			Trajectory.Samples[NumHistorySamples - 1].TimeInSeconds = 0.0f;
			Trajectory.Samples[NumHistorySamples - 1].Position = Trajectory.Samples[NumHistorySamples].Position;
			Trajectory.Samples[NumHistorySamples - 1].Facing = Trajectory.Samples[NumHistorySamples].Facing;

			// And apply ground delta to this record
			ApplyGroundDelta(Trajectory.Samples[NumHistorySamples - 1].Position, Trajectory.Samples[NumHistorySamples - 1].Facing);
		}
		else
		{
			// Didn't record a new history position, update timers and shift by ground delta

			for (int32 Index = 0; Index < NumHistorySamples; ++Index)
			{
				Trajectory.Samples[Index].TimeInSeconds -= TimeShift;
				ApplyGroundDelta(Trajectory.Samples[Index].Position, Trajectory.Samples[Index].Facing);
			}
		}
	}
}

void UPoseSearchTrajectoryLibrary::UpdateHistory_WorldSpace(FTransformTrajectory& Trajectory,
                                                            const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling,
                                                            float DeltaTime)
{
	const int32 NumHistorySamples = TrajectoryDataSampling.NumHistorySamples;
	if (NumHistorySamples > 0)
	{
		const float SecondsPerHistorySample = TrajectoryDataSampling.SecondsPerHistorySample;

		// Trajectory should include room for history + current + future
		// So num history samples needs to be less than the total number
		check(NumHistorySamples < Trajectory.Samples.Num());

		float TimeShift = Trajectory.Samples[NumHistorySamples].TimeInSeconds;	// This works out as the delta time of the last frame  

		// Shift history Samples when it's time to record a new one.
		if (SecondsPerHistorySample <= 0.f || FMath::Abs(Trajectory.Samples[NumHistorySamples - 1].TimeInSeconds) >= SecondsPerHistorySample)
		{
			for (int32 Index = 0; Index < NumHistorySamples - 1; ++Index)
			{
				Trajectory.Samples[Index].TimeInSeconds = Trajectory.Samples[Index + 1].TimeInSeconds - TimeShift;
				Trajectory.Samples[Index].Position = Trajectory.Samples[Index + 1].Position;
				Trajectory.Samples[Index].Facing = Trajectory.Samples[Index + 1].Facing;
			}

			// Adding a new history record
			// Copy over the last frame's current transform (stored at i == NumHistorySamples) into a sample at t = 0
			Trajectory.Samples[NumHistorySamples - 1].TimeInSeconds = 0.0f;
			Trajectory.Samples[NumHistorySamples - 1].Position = Trajectory.Samples[NumHistorySamples].Position;
			Trajectory.Samples[NumHistorySamples - 1].Facing = Trajectory.Samples[NumHistorySamples].Facing;
		}
		else
		{
			// Didn't record a new history position, update timers and shift by ground translation
			for (int32 Index = 0; Index < NumHistorySamples; ++Index)
			{
				Trajectory.Samples[Index].TimeInSeconds -= TimeShift;
			}
		}
	}	
}

FVector UPoseSearchTrajectoryLibrary::RemapVectorMagnitudeWithCurve(
	const FVector& Vector,
	bool bUseCurve,
	const FRuntimeFloatCurve& Curve)
{
	if (bUseCurve)
	{
		const float Length = Vector.Length();
		if (Length > UE_KINDA_SMALL_NUMBER)
		{
			const float RemappedLength = Curve.GetRichCurveConst()->Eval(Length);
			return Vector * (RemappedLength / Length);
		}
	}

	return Vector;
}

void UPoseSearchTrajectoryLibrary::UpdatePrediction_SimulateCharacterMovementInit(FTransformTrajectory& Trajectory, 
	FVector& OutCurrentPositionWS, 
	FVector& OutCurrentVelocityWS, 
	FVector& OutCurrentAccelerationWS,
	FQuat& OutCurrentFacingWS,
	const FPoseSearchTrajectoryData& TrajectoryData, 
	const FPoseSearchTrajectoryData::FDerived& TrajectoryDataDerived)
{
	OutCurrentPositionWS = TrajectoryDataDerived.Position;
	OutCurrentVelocityWS = RemapVectorMagnitudeWithCurve(TrajectoryDataDerived.Velocity, TrajectoryData.bUseSpeedRemappingCurve, TrajectoryData.SpeedRemappingCurve);
	OutCurrentAccelerationWS = RemapVectorMagnitudeWithCurve(TrajectoryDataDerived.Acceleration, TrajectoryData.bUseAccelerationRemappingCurve, TrajectoryData.AccelerationRemappingCurve);

	// Bending CurrentVelocityWS towards CurrentAccelerationWS
	if (TrajectoryData.BendVelocityTowardsAcceleration > UE_KINDA_SMALL_NUMBER && !OutCurrentAccelerationWS.IsNearlyZero())
	{
		const float CurrentSpeed = OutCurrentVelocityWS.Length();
		const FVector VelocityWSAlongAcceleration = OutCurrentAccelerationWS.GetUnsafeNormal() * CurrentSpeed;
		if (TrajectoryData.BendVelocityTowardsAcceleration < 1.f - UE_KINDA_SMALL_NUMBER)
		{
			OutCurrentVelocityWS = FMath::Lerp(OutCurrentVelocityWS, VelocityWSAlongAcceleration, TrajectoryData.BendVelocityTowardsAcceleration);

			const float NewLength = OutCurrentVelocityWS.Length();
			if (NewLength > UE_KINDA_SMALL_NUMBER)
			{
				OutCurrentVelocityWS *= CurrentSpeed / NewLength;
			}
			else
			{
				// @todo: consider setting the CurrentVelocityWS = VelocityWSAlongAcceleration if vel and acc are in opposite directions
			}
		}
		else
		{
			OutCurrentVelocityWS = VelocityWSAlongAcceleration;
		}
	}

	OutCurrentFacingWS = TrajectoryDataDerived.Facing;
}

void UPoseSearchTrajectoryLibrary::UpdatePrediction_SimulateCharacterMovementStep(FTransformTrajectory& Trajectory,
	FVector& InOutCurrentPositionWS,
	FVector& InOutCurrentVelocityWS,
	FVector& InOutCurrentAccelerationWS,
	FQuat& InOutCurrentFacingWS,
	const FPoseSearchTrajectoryData& TrajectoryData,
	const FPoseSearchTrajectoryData::FDerived& TrajectoryDataDerived,
	const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling,
	float& InOutAccumulatedSeconds)
{
	const float SecondsPerPredictionSample = TrajectoryDataSampling.SecondsPerPredictionSample;
	const FQuat ControllerRotationPerStep = FQuat::MakeFromEuler(FVector(0.f, 0.f, TrajectoryDataDerived.ControllerYawRate * SecondsPerPredictionSample));

	InOutCurrentPositionWS += InOutCurrentVelocityWS * SecondsPerPredictionSample;
	InOutAccumulatedSeconds += SecondsPerPredictionSample;

	if (TrajectoryDataDerived.bStepGroundPrediction)
	{
		InOutCurrentAccelerationWS = RemapVectorMagnitudeWithCurve(ControllerRotationPerStep * InOutCurrentAccelerationWS,
			TrajectoryData.bUseAccelerationRemappingCurve, TrajectoryData.AccelerationRemappingCurve);
		const FVector NewVelocityWS = TrajectoryData.StepCharacterMovementGroundPrediction(SecondsPerPredictionSample, InOutCurrentVelocityWS, InOutCurrentAccelerationWS, TrajectoryDataDerived);
		InOutCurrentVelocityWS = RemapVectorMagnitudeWithCurve(NewVelocityWS, TrajectoryData.bUseSpeedRemappingCurve, TrajectoryData.SpeedRemappingCurve);

		// Account for the controller (e.g. the camera) rotating.
		InOutCurrentFacingWS = ControllerRotationPerStep * InOutCurrentFacingWS;
		if (TrajectoryDataDerived.bOrientRotationToMovement && !InOutCurrentAccelerationWS.IsNearlyZero())
		{
			// Rotate towards acceleration.
			const FVector CurrentAccelerationCS = TrajectoryDataDerived.MeshCompRelativeRotation.RotateVector(InOutCurrentAccelerationWS);
			InOutCurrentFacingWS = FMath::QInterpConstantTo(InOutCurrentFacingWS, CurrentAccelerationCS.ToOrientationQuat(), SecondsPerPredictionSample, TrajectoryData.RotateTowardsMovementSpeed);
		}
	}
}

void UPoseSearchTrajectoryLibrary::UpdatePrediction_SimulateCharacterMovement(FTransformTrajectory& Trajectory,
	const FPoseSearchTrajectoryData& TrajectoryData, const FPoseSearchTrajectoryData::FDerived& TrajectoryDataDerived,
	const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling, float DeltaTime)
{
	FVector CurrentPositionWS = FVector::ZeroVector;
	FVector CurrentVelocityWS = FVector::ZeroVector;
	FVector CurrentAccelerationWS = FVector::ZeroVector;
	FQuat CurrentFacingWS = FQuat::Identity;

	UPoseSearchTrajectoryLibrary::UpdatePrediction_SimulateCharacterMovementInit(Trajectory, 
		CurrentPositionWS, 
		CurrentVelocityWS, 
		CurrentAccelerationWS, 
		CurrentFacingWS,
		TrajectoryData, 
		TrajectoryDataDerived);
	
	const int32 NumHistorySamples = TrajectoryDataSampling.NumHistorySamples;

	float AccumulatedSeconds = DeltaTime;

	const int32 LastIndex = Trajectory.Samples.Num() - 1;
	if (NumHistorySamples <= LastIndex)
	{
		for (int32 Index = NumHistorySamples; ; ++Index)
		{
			Trajectory.Samples[Index].Position = CurrentPositionWS;
			Trajectory.Samples[Index].Facing = CurrentFacingWS;
			Trajectory.Samples[Index].TimeInSeconds = AccumulatedSeconds;

			if (Index == LastIndex)
			{
				break;
			}

			UPoseSearchTrajectoryLibrary::UpdatePrediction_SimulateCharacterMovementStep(Trajectory, 
				CurrentPositionWS, 
				CurrentVelocityWS, 
				CurrentAccelerationWS, 
				CurrentFacingWS, 
				TrajectoryData, 
				TrajectoryDataDerived, 
				TrajectoryDataSampling,
				AccumulatedSeconds);
		}
	}
}

void UPoseSearchTrajectoryLibrary::UpdatePrediction_SimulateMontage(const UObject* InContext,
	const FAnimMontageInstance* MontageInstance,
	FTransformTrajectory& Trajectory,
	const FPoseSearchTrajectoryData& TrajectoryData, const FPoseSearchTrajectoryData::FDerived& TrajectoryDataDerived,
	const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling, float DeltaTime, bool bSimulateWarping, bool bSimulateCharacterMovement, UMotionWarpingComponent* InWarpingComponent)
{
	FVector CurrentPositionWS = FVector::ZeroVector;
	FVector CurrentVelocityWS = FVector::ZeroVector;
	FVector CurrentAccelerationWS = FVector::ZeroVector;
	FQuat CurrentFacingWS = FQuat::Identity;

	if (bSimulateCharacterMovement)
	{
		UPoseSearchTrajectoryLibrary::UpdatePrediction_SimulateCharacterMovementInit(Trajectory,
			CurrentPositionWS,
			CurrentVelocityWS,
			CurrentAccelerationWS,
			CurrentFacingWS,
			TrajectoryData,
			TrajectoryDataDerived);
	}
	else
	{
		CurrentPositionWS = TrajectoryDataDerived.Position;
		CurrentFacingWS = TrajectoryDataDerived.Facing;
	}

	FTransform CurrentTransform = FTransform(TrajectoryDataDerived.Facing, TrajectoryDataDerived.Position);
	
	const int32 NumHistorySamples = TrajectoryDataSampling.NumHistorySamples;
	const float SecondsPerPredictionSample = TrajectoryDataSampling.SecondsPerPredictionSample;

	float AccumulatedSeconds = DeltaTime;

	const AActor* OwningActor = Cast<AActor>(InContext);
	if (!OwningActor)
	{
		if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(InContext))
		{
			OwningActor = Cast<AActor>(AnimInstance->GetOwningActor());
		}
		else if (const UActorComponent* AnimNextComponent = Cast<UActorComponent>(InContext))
		{
			OwningActor = Cast<AActor>(AnimNextComponent->GetOwner());
		}
	}

	UMotionWarpingComponent* WarpingComp = InWarpingComponent;
	if (!WarpingComp && bSimulateWarping)
	{
		WarpingComp = OwningActor ? OwningActor->FindComponentByClass<UMotionWarpingComponent>() : nullptr;
	}
	UMotionWarpingMontageTrajectoryAdapter* PredictionAdapter = nullptr;

	FTransform VisualRootTransform = CurrentTransform;
	if (bSimulateWarping && WarpingComp && !WarpingComp->GetModifiers().IsEmpty())
	{
		WarpingComp->BeginTrajectoryPredictions(MontageInstance);

		// Visual roots should be base visual * transform, but in practice they are adapter implementation specifc & deviate (Individually consider scaled capsule height / etc).
		// Preserve this deviation by transforming the visual root in addition to to current transform
		PredictionAdapter = CastChecked<UMotionWarpingMontageTrajectoryAdapter>(WarpingComp->GetOwnerAdapter());
		if (PredictionAdapter)
		{
			VisualRootTransform.SetLocation(PredictionAdapter->GetVisualRootLocation());
		}
	}

	const int32 LastIndex = Trajectory.Samples.Num() - 1;
	if (NumHistorySamples <= LastIndex)
	{
		for (int32 Index = NumHistorySamples; ; ++Index)
		{
			// Store Values
			Trajectory.Samples[Index].Position = CurrentPositionWS;
			Trajectory.Samples[Index].Facing = CurrentFacingWS;
			Trajectory.Samples[Index].TimeInSeconds = AccumulatedSeconds;

			if (Index == LastIndex)
			{
				break;
			}

			// Forward step
			float CurrentMontagePosition = MontageInstance->GetPosition() + AccumulatedSeconds;
			FAnimExtractContext ExtractionCtx(0.0, true, FDeltaTimeRecord(), false);
			FTransform RootMotion = MontageInstance->Montage->ExtractRootMotionFromTrackRange(CurrentMontagePosition, CurrentMontagePosition + SecondsPerPredictionSample, ExtractionCtx);
			
			if (bSimulateWarping && PredictionAdapter && WarpingComp && WarpingComp->IsPredictingTrajectory())
			{
				const FTransform WarpedRootMotion = PredictionAdapter->WarpLocalRootMotionOnMontageTrajectory(RootMotion, SecondsPerPredictionSample, CurrentMontagePosition, CurrentMontagePosition + SecondsPerPredictionSample);

				CurrentTransform = WarpedRootMotion * CurrentTransform;
				VisualRootTransform = WarpedRootMotion * VisualRootTransform;

				// Root modifiers expect actor component transforms, not root transforms
				PredictionAdapter->SetCurrentTransform(FTransform(PredictionAdapter->GetBaseVisualRotationOffset(), PredictionAdapter->GetBaseVisualTranslationOffset()).Inverse() * CurrentTransform);

				// Adapter separates base visual rotation / offset capsule concerns, opt to preserve via iterative changes
				PredictionAdapter->SetVisualRootLocation(VisualRootTransform.GetLocation());
			}
			else
			{
				CurrentTransform = RootMotion * CurrentTransform;
			}

			CurrentPositionWS = CurrentTransform.GetLocation();
			CurrentFacingWS = CurrentTransform.GetRotation();

			if (bSimulateCharacterMovement && PredictionAdapter)
			{
				// @TODO: Ideally we should feedback our prediction, but there are issues in trajectory simulation not considering accel that make this a bad idea
				// For now, don't feedback, which is equivalent to only using initial velocity as an offset. As that gives better results / closer mimics CMC.
				
				// Need to manually update visual root before velocity changes
				// PredictionAdapter->SetVisualRootLocation(VisualRootTransform.GetLocation() += CurrentVelocityWS * SecondsPerPredictionSample);

				UPoseSearchTrajectoryLibrary::UpdatePrediction_SimulateCharacterMovementStep(Trajectory,
					CurrentPositionWS,
					CurrentVelocityWS,
					CurrentAccelerationWS,
					CurrentFacingWS,
					TrajectoryData,
					TrajectoryDataDerived,
					TrajectoryDataSampling,
					AccumulatedSeconds);

				// CurrentTransform.SetLocation(CurrentPositionWS);
				// CurrentTransform.SetRotation(CurrentFacingWS);
				// PredictionAdapter->SetCurrentTransform(FTransform(PredictionAdapter->GetBaseVisualRotationOffset(), PredictionAdapter->GetBaseVisualTranslationOffset()).Inverse() * CurrentTransform);
			}
			else
			{
				AccumulatedSeconds += SecondsPerPredictionSample;
			}
		}
	}

	if (bSimulateWarping && WarpingComp && WarpingComp->IsPredictingTrajectory())
	{
		WarpingComp->EndTrajectoryPredictions();
	}
}

void UPoseSearchTrajectoryLibrary::PoseSearchGenerateTransformTrajectory(const UObject* InContext, const FPoseSearchTrajectoryData& InTrajectoryData,
	float InDeltaTime, FTransformTrajectory& InOutTrajectory, float& InOutDesiredControllerYawLastUpdate, FTransformTrajectory& OutTrajectory,
	float InHistorySamplingInterval, int32 InTrajectoryHistoryCount, float InPredictionSamplingInterval, int32 InTrajectoryPredictionCount)
{
	if (GVarPoseSearchTrajectoryUseWarpedMontage)
	{
		UPoseSearchTrajectoryLibrary::PoseSearchGenerateWarpedTransformTrajectory(
			InContext,
			InTrajectoryData,
			InDeltaTime,
			InOutTrajectory,
			InOutDesiredControllerYawLastUpdate,
			OutTrajectory,
			InHistorySamplingInterval,
			InTrajectoryHistoryCount,
			InPredictionSamplingInterval,
			InTrajectoryPredictionCount,
			true,
			true);

		return;
	}

	FPoseSearchTrajectoryData::FSampling TrajectoryDataSampling;
	TrajectoryDataSampling.NumHistorySamples = InTrajectoryHistoryCount;
	TrajectoryDataSampling.SecondsPerHistorySample = InHistorySamplingInterval;
	TrajectoryDataSampling.NumPredictionSamples = InTrajectoryPredictionCount;
	TrajectoryDataSampling.SecondsPerPredictionSample = InPredictionSamplingInterval;

	FPoseSearchTrajectoryData::FState TrajectoryDataState;
	TrajectoryDataState.DesiredControllerYawLastUpdate = InOutDesiredControllerYawLastUpdate;

	FPoseSearchTrajectoryData::FDerived TrajectoryDataDerived;
	InTrajectoryData.UpdateData(InDeltaTime, InContext, TrajectoryDataDerived, TrajectoryDataState);
	InitTrajectorySamples(InOutTrajectory, TrajectoryDataDerived.Position, TrajectoryDataDerived.Facing, TrajectoryDataSampling, InDeltaTime);
	UpdateHistory_TransformHistory(InOutTrajectory, TrajectoryDataDerived.Position, TrajectoryDataDerived.Velocity, TrajectoryDataSampling, InDeltaTime);
	UpdatePrediction_SimulateCharacterMovement(InOutTrajectory, InTrajectoryData, TrajectoryDataDerived, TrajectoryDataSampling, InDeltaTime);

	InOutDesiredControllerYawLastUpdate = TrajectoryDataState.DesiredControllerYawLastUpdate;

	OutTrajectory = InOutTrajectory;
}

void UPoseSearchTrajectoryLibrary::PoseSearchGenerateWarpedTransformTrajectory(const UObject* InContext, const FPoseSearchTrajectoryData& InTrajectoryData,
	float InDeltaTime, FTransformTrajectory& InOutTrajectory, float& InOutDesiredControllerYawLastUpdate, FTransformTrajectory& OutTrajectory,
	float InHistorySamplingInterval, int32 InTrajectoryHistoryCount, float InPredictionSamplingInterval, int32 InTrajectoryPredictionCount,
	bool bSimulateMontages, bool bSimulateWarping)
{
	FPoseSearchTrajectoryData::FSampling TrajectoryDataSampling;
	TrajectoryDataSampling.NumHistorySamples = InTrajectoryHistoryCount;
	TrajectoryDataSampling.SecondsPerHistorySample = InHistorySamplingInterval;
	TrajectoryDataSampling.NumPredictionSamples = InTrajectoryPredictionCount;
	TrajectoryDataSampling.SecondsPerPredictionSample = InPredictionSamplingInterval;

	FPoseSearchTrajectoryData::FState TrajectoryDataState;
	TrajectoryDataState.DesiredControllerYawLastUpdate = InOutDesiredControllerYawLastUpdate;

	FPoseSearchTrajectoryData::FDerived TrajectoryDataDerived;
	InTrajectoryData.UpdateData(InDeltaTime, InContext, TrajectoryDataDerived, TrajectoryDataState);
	InitTrajectorySamples(InOutTrajectory, TrajectoryDataDerived.Position, TrajectoryDataDerived.Facing, TrajectoryDataSampling, InDeltaTime);
	UpdateHistory_TransformHistory(InOutTrajectory, TrajectoryDataDerived.Position, TrajectoryDataDerived.Velocity, TrajectoryDataSampling, InDeltaTime);

	FAnimMontageInstance* AnimMontageInstance = nullptr;

	if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(InContext))
	{
		AnimMontageInstance = AnimInstance->GetActiveMontageInstance();
	}
	else if (const ACharacter* Character = Cast<ACharacter>(InContext))
	{
		AnimMontageInstance = Character->GetRootMotionAnimMontageInstance();
	}
	else if (const UActorComponent* AnimNextComponent = Cast<UActorComponent>(InContext))
	{
		ACharacter* OwningCharacter = Cast<ACharacter>(AnimNextComponent->GetOwner());
		AnimMontageInstance = OwningCharacter ? OwningCharacter->GetRootMotionAnimMontageInstance() : nullptr;
	}

	if (AnimMontageInstance && bSimulateMontages)
	{
		UpdatePrediction_SimulateMontage(InContext, AnimMontageInstance, InOutTrajectory, InTrajectoryData, TrajectoryDataDerived, TrajectoryDataSampling, InDeltaTime, bSimulateWarping);
	}
	else
	{
		UpdatePrediction_SimulateCharacterMovement(InOutTrajectory, InTrajectoryData, TrajectoryDataDerived, TrajectoryDataSampling, InDeltaTime);
	}

	InOutDesiredControllerYawLastUpdate = TrajectoryDataState.DesiredControllerYawLastUpdate;

	OutTrajectory = InOutTrajectory;
}

void UPoseSearchTrajectoryLibrary::PoseSearchGeneratePredictorTransformTrajectory(UObject* InPredictor,
	const FPoseSearchTrajectoryData& InTrajectoryData, float InDeltaTime, FTransformTrajectory& InOutTrajectory,
	float& InOutDesiredControllerYawLastUpdate, FTransformTrajectory& OutTrajectory, float InHistorySamplingInterval,
	int32 InTrajectoryHistoryCount, float InPredictionSamplingInterval, int32 InTrajectoryPredictionCount)
{
	IPoseSearchTrajectoryPredictorInterface* Predictor = Cast<IPoseSearchTrajectoryPredictorInterface>(InPredictor);
	if (Predictor != nullptr)
	{
		PoseSearchGenerateTransformTrajectoryWithPredictor(InPredictor,
			InDeltaTime,
			InOutTrajectory,
			InOutDesiredControllerYawLastUpdate,
			OutTrajectory,
			InHistorySamplingInterval,
			InTrajectoryHistoryCount,
			InPredictionSamplingInterval,
			InTrajectoryPredictionCount);
	}
}

void UPoseSearchTrajectoryLibrary::PoseSearchGenerateTransformTrajectoryWithPredictor(TScriptInterface<IPoseSearchTrajectoryPredictorInterface> InPredictor,
	float InDeltaTime, FTransformTrajectory& InOutTrajectory, float& InOutDesiredControllerYawLastUpdate, FTransformTrajectory& OutTrajectory,
	float InHistorySamplingInterval, int32 InTrajectoryHistoryCount, float InPredictionSamplingInterval, int32 InTrajectoryPredictionCount)
{
	FPoseSearchTrajectoryData::FSampling TrajectoryDataSampling;
	TrajectoryDataSampling.NumHistorySamples = InTrajectoryHistoryCount;
	TrajectoryDataSampling.SecondsPerHistorySample = InHistorySamplingInterval;
	TrajectoryDataSampling.NumPredictionSamples = InTrajectoryPredictionCount;
	TrajectoryDataSampling.SecondsPerPredictionSample = InPredictionSamplingInterval;

	// TODO: handle controller yaw
	//TrajectoryDataState.DesiredControllerYawLastUpdate = InOutDesiredControllerYawLastUpdate;

	FVector CurrentPosition = FVector::ZeroVector;
	FVector CurrentVelocity = FVector::ZeroVector;
	FVector CurrentAngularVelocityDegrees = FVector::ZeroVector;
	FQuat CurrentFacing = FQuat::Identity;

	if (InPredictor != nullptr)
	{
		InPredictor->GetCurrentState(CurrentPosition, CurrentFacing, CurrentVelocity);
		InPredictor->GetAngularVelocity(CurrentAngularVelocityDegrees);
	}

	InitTrajectorySamples(InOutTrajectory, CurrentPosition, CurrentFacing, TrajectoryDataSampling, InDeltaTime);
	UpdateHistory_TransformHistoryWithRotation(InOutTrajectory, CurrentPosition, CurrentVelocity,
		CurrentFacing, CurrentAngularVelocityDegrees,
		TrajectoryDataSampling, InDeltaTime);

	// Set the current position at i == NumHistoryCount at t == delta time
	// Remember: t == 0 is the previous position, t == delta time is the current frame's position
	// assuming we call this method after the movement component has updated to a new position
	InOutTrajectory.Samples[InTrajectoryHistoryCount].TimeInSeconds = InDeltaTime;
	InOutTrajectory.Samples[InTrajectoryHistoryCount].Position = CurrentPosition;
	InOutTrajectory.Samples[InTrajectoryHistoryCount].Facing = CurrentFacing;

	auto TryGenerateTrajectoryWithMontage = [&]()
	{
		if (GVarPoseSearchTrajectoryUseWarpedMontage && InPredictor.GetObject())
		{
			if (const UAnimInstance* AnimInstance = Cast<UAnimInstance>(InPredictor.GetObject()->GetOuter()))
			{
				if (FAnimMontageInstance* AnimMontageInstance = AnimInstance->GetActiveMontageInstance())
				{
					bool bSimulateWarping = true;
					bool bSimulateCharacterMovement = false;

					// When not simulating CMC, simulate montage only needs current position / facing
					FPoseSearchTrajectoryData TrajectoryData;
					FPoseSearchTrajectoryData::FDerived TrajectoryDataDerived;
					TrajectoryDataDerived.Position = CurrentPosition;
					TrajectoryDataDerived.Facing = CurrentFacing;

					UpdatePrediction_SimulateMontage(AnimInstance, AnimMontageInstance, InOutTrajectory, TrajectoryData, TrajectoryDataDerived, TrajectoryDataSampling, InDeltaTime, bSimulateWarping, bSimulateCharacterMovement);

					return true;
				}
			}
		}

		return false;
	};

	if (InPredictor && !TryGenerateTrajectoryWithMontage())
	{
		InPredictor->Predict(InOutTrajectory, InTrajectoryPredictionCount, InPredictionSamplingInterval, InTrajectoryHistoryCount);
	}

	//InOutDesiredControllerYawLastUpdate = TrajectoryDataState.DesiredControllerYawLastUpdate;

	OutTrajectory = InOutTrajectory;
}

void UPoseSearchTrajectoryLibrary::HandleTransformTrajectoryWorldCollisions(const UObject* WorldContextObject, const UAnimInstance* AnimInstance,
	const FTransformTrajectory& InTrajectory, bool bApplyGravity, float FloorCollisionsOffset, FTransformTrajectory& OutTrajectory,
	FPoseSearchTrajectory_WorldCollisionResults& CollisionResult, ETraceTypeQuery TraceChannel, bool bTraceComplex,
	const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, bool bIgnoreSelf, float MaxObstacleHeight, FLinearColor TraceColor,
	FLinearColor TraceHitColor, float DrawTime)
{
	FVector StartingVelocity = FVector::ZeroVector;
	FVector GravityAccel = FVector::ZeroVector;
	if (bApplyGravity && AnimInstance)
	{
		if (const ACharacter* Character = Cast<ACharacter>(AnimInstance->GetOwningActor()))
		{
			if (const UCharacterMovementComponent* MoveComp = Character->GetCharacterMovement())
			{
				GravityAccel = MoveComp->GetGravityDirection() * -MoveComp->GetGravityZ();
				StartingVelocity = Character->GetVelocity();
			}
		}
	}

	HandleTransformTrajectoryWorldCollisionsWithGravity(WorldContextObject, InTrajectory, StartingVelocity, bApplyGravity, GravityAccel, 
		FloorCollisionsOffset, OutTrajectory, CollisionResult, TraceChannel, bTraceComplex, ActorsToIgnore, 
		DrawDebugType, bIgnoreSelf, MaxObstacleHeight, TraceColor, TraceHitColor, DrawTime);
}

void UPoseSearchTrajectoryLibrary::HandleTransformTrajectoryWorldCollisionsWithGravity(const UObject* WorldContextObject,
	const FTransformTrajectory& InTrajectory, FVector StartingVelocity, bool bApplyGravity, FVector GravityAccel, float FloorCollisionsOffset,
	FTransformTrajectory& OutTrajectory, FPoseSearchTrajectory_WorldCollisionResults& CollisionResult, ETraceTypeQuery TraceChannel,
	bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, bool bIgnoreSelf, float MaxObstacleHeight,
	FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	OutTrajectory = InTrajectory;

	TArray<FTransformTrajectorySample>& Samples = OutTrajectory.Samples;
	const int32 NumSamples = Samples.Num();

	FVector GravityDirection = FVector::ZeroVector;
	float GravityZ = 0.f;
	float InitialVelocityZ = StartingVelocity.Z;

	if (bApplyGravity && !GravityAccel.IsNearlyZero())
	{
		GravityAccel.ToDirectionAndLength(GravityDirection, GravityZ);
		GravityZ = -GravityZ;
		const FVector VelocityOnGravityAxis = StartingVelocity.ProjectOnTo(GravityDirection);
		
		InitialVelocityZ = VelocityOnGravityAxis.Length() * -FMath::Sign(GravityDirection.Dot(VelocityOnGravityAxis));
	}

	CollisionResult.TimeToLand = OutTrajectory.Samples.Last().TimeInSeconds;

	if (!FMath::IsNearlyZero(GravityZ))
	{
		FVector LastImpactPoint;
		FVector LastImpactNormal;
		bool bIsLastImpactValid = false;
		bool bIsFirstFall = true;

		const FVector Gravity = GravityDirection * -GravityZ;
		float FreeFallAccumulatedSeconds = 0.f;
		for (int32 SampleIndex = 1; SampleIndex < NumSamples; ++SampleIndex)
		{
			FTransformTrajectorySample& Sample = Samples[SampleIndex];
			if (Sample.TimeInSeconds > 0.f)
			{
				const int32 PrevSampleIndex = SampleIndex - 1;
				const FTransformTrajectorySample& PrevSample = Samples[PrevSampleIndex];

				FreeFallAccumulatedSeconds += Sample.TimeInSeconds - PrevSample.TimeInSeconds;

				if (bIsLastImpactValid)
				{
					const FPlane GroundPlane = FPlane(PrevSample.Position, -GravityDirection);
					Sample.Position = FPlane::PointPlaneProject(Sample.Position, GroundPlane);
				}

				// applying gravity
				const FVector FreeFallOffset =  Gravity * (0.5f * FreeFallAccumulatedSeconds * FreeFallAccumulatedSeconds);
				Sample.Position += FreeFallOffset;

				FHitResult HitResult;
				if (FloorCollisionsOffset > 0.f && UKismetSystemLibrary::LineTraceSingle(WorldContextObject, Sample.Position + (GravityDirection * -MaxObstacleHeight), Sample.Position, TraceChannel, bTraceComplex, ActorsToIgnore, DrawDebugType, HitResult, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime))
				{
					// Only allow our trace to move trajectory along gravity direction.
					LastImpactPoint = UKismetMathLibrary::FindClosestPointOnLine(HitResult.ImpactPoint, Sample.Position, GravityDirection);
					LastImpactNormal = HitResult.Normal;
					bIsLastImpactValid = true;

					Sample.Position = LastImpactPoint - GravityDirection * FloorCollisionsOffset;

					if (bIsFirstFall)
					{
						const float InitialHeight = OutTrajectory.GetSampleAtTime(0.0f).Position.Z;
						const float FinalHeight = Sample.Position.Z;
						const float FallHeight = FMath::Abs(FinalHeight - InitialHeight);

						bIsFirstFall = false;
						CollisionResult.TimeToLand = (InitialVelocityZ / -GravityZ) + ((FMath::Sqrt(FMath::Square(InitialVelocityZ) + (2.f * -GravityZ * FallHeight))) / -GravityZ);
						CollisionResult.LandSpeed = InitialVelocityZ + GravityZ * CollisionResult.TimeToLand;
					}

					FreeFallAccumulatedSeconds = 0.f;
				}
			}
		}
	}
	else if (FloorCollisionsOffset > 0.f)
	{
		for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
		{
			FTransformTrajectorySample& Sample = OutTrajectory.Samples[SampleIndex];
			if (Sample.TimeInSeconds > 0.f)
			{
				FHitResult HitResult;
				if (UKismetSystemLibrary::LineTraceSingle(WorldContextObject, Sample.Position + FVector::UpVector * 3000.f, Sample.Position, TraceChannel, bTraceComplex, ActorsToIgnore, DrawDebugType, HitResult, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime))
				{
					Sample.Position.Z = HitResult.ImpactPoint.Z + FloorCollisionsOffset;
				}
			}
		}
	}

	CollisionResult.LandSpeed = InitialVelocityZ + GravityZ * CollisionResult.TimeToLand;
}

void UPoseSearchTrajectoryLibrary::GetTransformTrajectorySampleAtTime(const FTransformTrajectory& InTrajectory, float Time,
	FTransformTrajectorySample& OutTrajectorySample, bool bExtrapolate)
{
	OutTrajectorySample = InTrajectory.GetSampleAtTime(Time, bExtrapolate);
}

void UPoseSearchTrajectoryLibrary::GetTransformTrajectoryVelocity(const FTransformTrajectory& InTrajectory, float Time1, float Time2, FVector& OutVelocity,
	bool bExtrapolate)
{
	if (FMath::IsNearlyEqual(Time1, Time2))
	{
		UE_LOGF(LogPoseSearch, Warning, "UPoseSearchTrajectoryLibrary::GetTrajectoryVelocity - Time1 is same as Time2. Invalid time horizon.");
		OutVelocity = FVector::ZeroVector;
		return;
	}

	FTransformTrajectorySample Sample1 = InTrajectory.GetSampleAtTime(Time1, bExtrapolate);
	FTransformTrajectorySample Sample2 = InTrajectory.GetSampleAtTime(Time2, bExtrapolate);

	OutVelocity = (Sample2.Position - Sample1.Position) / (Time2 - Time1);
}

void UPoseSearchTrajectoryLibrary::GetTransformTrajectoryAngularVelocity(const FTransformTrajectory& InTrajectory, float Time1, float Time2,
	FVector& OutAngularVelocity, bool bExtrapolate)
{
	if (FMath::IsNearlyEqual(Time1, Time2))
	{
		UE_LOGF(LogPoseSearch, Warning, "UPoseSearchTrajectoryLibrary::GetTrajectoryAngularVelocity - Time1 is same as Time2. Invalid time horizon.");
		OutAngularVelocity = FVector::ZeroVector;
		return;
	}

	FTransformTrajectorySample Sample1 = InTrajectory.GetSampleAtTime(Time1, bExtrapolate);
	FTransformTrajectorySample Sample2 = InTrajectory.GetSampleAtTime(Time2, bExtrapolate);

	const FQuat DeltaRotation = (Sample2.Facing * Sample1.Facing.Inverse()).GetShortestArcWith(FQuat::Identity);
	const FVector AngularVelocityInRadians = DeltaRotation.ToRotationVector() / (Time2 - Time1);

	OutAngularVelocity = FVector(
		FMath::RadiansToDegrees(AngularVelocityInRadians.X),
		FMath::RadiansToDegrees(AngularVelocityInRadians.Y),
		FMath::RadiansToDegrees(AngularVelocityInRadians.Z));
}

FTransform UPoseSearchTrajectoryLibrary::GetTransformTrajectorySampleTransform(const FTransformTrajectorySample& InTrajectorySample)
{
	return InTrajectorySample.GetTransform();
}

void UPoseSearchTrajectoryLibrary::DrawTransformTrajectory(const UObject* WorldContextObject, const FTransformTrajectory& InTrajectory,
	const float DebugThickness, float HeightOffset)
{
#if ENABLE_ANIM_DEBUG
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		UTransformTrajectoryBlueprintLibrary::DebugDrawTrajectory(InTrajectory, World, DebugThickness, HeightOffset);
	}
#endif // ENABLE_ANIM_DEBUG
}

