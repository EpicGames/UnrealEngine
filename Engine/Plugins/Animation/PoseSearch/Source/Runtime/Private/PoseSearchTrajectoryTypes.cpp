// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/PoseSearchTrajectoryTypes.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimTypes.h"
#include "Components/SceneComponent.h"
#include "DrawDebugHelpers.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "Math/Axis.h"
#include "VisualLogger/VisualLogger.h"
#include "Animation/TrajectoryTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseSearchTrajectoryTypes)

PRAGMA_DISABLE_DEPRECATION_WARNINGS

FPoseSearchQueryTrajectorySample::FPoseSearchQueryTrajectorySample(const FQuat& InFacing, const FVector& InPosition, float InAccumulatedSeconds)
	: Facing(InFacing), Position(InPosition), AccumulatedSeconds(InAccumulatedSeconds)
{
}

FPoseSearchQueryTrajectorySample FPoseSearchQueryTrajectorySample::Lerp(const FPoseSearchQueryTrajectorySample& Other, float Alpha) const
{
	check(Facing.IsNormalized());
	check(Other.Facing.IsNormalized());

	FPoseSearchQueryTrajectorySample Result;
	
	Result.Facing = FQuat::FastLerp(Facing, Other.Facing, Alpha).GetNormalized();
	Result.Position = FMath::Lerp(Position, Other.Position, Alpha);
	Result.AccumulatedSeconds = FMath::Lerp(AccumulatedSeconds, Other.AccumulatedSeconds, Alpha);

	return Result;
}

void FPoseSearchQueryTrajectorySample::SetTransform(const FTransform& Transform)
{
	Position = Transform.GetTranslation();
	Facing = Transform.GetRotation();
}

FPoseSearchQueryTrajectorySample FPoseSearchQueryTrajectory::GetSampleAtTime(float Time, bool bExtrapolate) const
{
	const int32 Num = Samples.Num();
	if (Num > 1)
	{
		const int32 LowerBoundIdx = Algo::LowerBound(Samples, Time, [](const FPoseSearchQueryTrajectorySample& TrajectorySample, float Value)
			{
				return Value > TrajectorySample.AccumulatedSeconds;
			});

		const int32 NextIdx = FMath::Clamp(LowerBoundIdx, 1, Samples.Num() - 1);
		const int32 PrevIdx = NextIdx - 1;

		const float Denominator = Samples[NextIdx].AccumulatedSeconds - Samples[PrevIdx].AccumulatedSeconds;
		if (!FMath::IsNearlyZero(Denominator))
		{
			const float Numerator = Time - Samples[PrevIdx].AccumulatedSeconds;
			const float LerpValue = bExtrapolate ? Numerator / Denominator : FMath::Clamp(Numerator / Denominator, 0.f, 1.f);
			return Samples[PrevIdx].Lerp(Samples[NextIdx], LerpValue);
		}

		return Samples[PrevIdx];
	}

	if (Num > 0)
	{
		return Samples[0];
	}

	return FPoseSearchQueryTrajectorySample();
}

#if ENABLE_ANIM_DEBUG
void FPoseSearchQueryTrajectory::DebugDrawTrajectory(const UWorld* World, const float DebugThickness, float HeightOffset) const
{
	const FVector OffsetVector = FVector::UpVector * HeightOffset;

	const int32 LastIndex = Samples.Num() - 1;
	if (LastIndex >= 0)
	{
		for (int32 Index = 0; ; ++Index)
		{
			const FVector Pos = Samples[Index].Position + OffsetVector;

			DrawDebugSphere(World, Pos, 1.f, 4, FColor::Black, false, -1.f, SDPG_Foreground, DebugThickness);

			const FRotationMatrix R(FRotator(Samples[Index].Facing));
			const FVector X = R.GetScaledAxis( EAxis::X );
			const FVector Y = R.GetScaledAxis( EAxis::Y );

			const float Scale = 12.f;

			const bool IsPast = Samples[Index].AccumulatedSeconds <= 0.f;

			DrawDebugLine(World, Pos, Pos + X * Scale, IsPast ? FColor::Red : FColor::Blue, false, -1.f, SDPG_Foreground, DebugThickness);
			DrawDebugLine(World, Pos, Pos + Y * Scale, IsPast ? FColor::Orange : FColor::Turquoise, false, -1.f, SDPG_Foreground, DebugThickness);

			if (Index == LastIndex)
			{
				break;
			}
			
			const FVector NextPos = Samples[Index + 1].Position + OffsetVector;
			DrawDebugLine(World, Pos, NextPos, FColor::Black, false, -1.f, SDPG_Foreground, DebugThickness);
		}
	}
}

void FPoseSearchQueryTrajectory::DebugDrawTrajectory(FAnimInstanceProxy& AnimInstanceProxy, const float DebugThickness, float HeightOffset, int MaxHistorySamples, int MaxPredictionSamples) const
{
	const FVector OffsetVector = FVector::UpVector * HeightOffset;
	
	int AvailableHistorySamplesCount = 0;
	
	if (MaxHistorySamples != -1 || MaxPredictionSamples != -1)
	{
		for (int32 i = 0; i < Samples.Num(); ++i)
		{
			if (Samples[i].AccumulatedSeconds <= 0)
			{
				++AvailableHistorySamplesCount;
			}
			else
			{
				break;
			}
		}
	}
	
	const int32 LastIndex = Samples.Num() - 1;
	const int32 ZeroTimeIndex = AvailableHistorySamplesCount - 1;
	const int32 StartIndex = MaxHistorySamples < 0 ? 0 : FMath::Max(AvailableHistorySamplesCount - MaxHistorySamples, 0); 
	const int32 EndIndex = MaxPredictionSamples < 0 ? LastIndex : FMath::Min(ZeroTimeIndex + MaxPredictionSamples, LastIndex);
	
	if (LastIndex >= 0 && StartIndex <= EndIndex)
	{
		for (int32 Index = StartIndex; ; ++Index)
		{
			const FVector Pos = Samples[Index].Position + OffsetVector;

			AnimInstanceProxy.AnimDrawDebugSphere(Samples[Index].Position + OffsetVector, 1.f, 4, FColor::Black, false, -1.f, DebugThickness, SDPG_Foreground);

			const FRotationMatrix R(FRotator(Samples[Index].Facing));
			const FVector X = R.GetScaledAxis( EAxis::X );
			const FVector Y = R.GetScaledAxis( EAxis::Y );

			const float Scale = 12.f;

			const bool IsPast = Samples[Index].AccumulatedSeconds <= 0.f;

			AnimInstanceProxy.AnimDrawDebugLine(Pos, Pos + X * Scale, IsPast ? FColor::Red : FColor::Blue, false, -1.f, DebugThickness, SDPG_Foreground);
			AnimInstanceProxy.AnimDrawDebugLine(Pos, Pos + Y * Scale, IsPast ? FColor::Orange : FColor::Turquoise, false, -1.f, DebugThickness, SDPG_Foreground);

			if (Index == EndIndex)
			{
				break;
			}
			
			const FVector NextPos = Samples[Index + 1].Position + OffsetVector;
			AnimInstanceProxy.AnimDrawDebugLine(Pos, NextPos, FColor::Black, false, -1.f, DebugThickness, SDPG_Foreground);
		}
	}
}

void FPoseSearchQueryTrajectory::DebugDrawTrajectory(const UObject* Owner, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, const float DebugThickness, float HeightOffset) const
{
#if ENABLE_VISUAL_LOG

	UWorld* World = nullptr;
	FVisualLogEntry* CurrentEntry = nullptr;
	if (FVisualLogger::CheckVisualLogInputInternal(Owner, Category.GetCategoryName(), Verbosity, &World, &CurrentEntry) == false)
	{
		return;
	}

	const FVector OffsetVector = FVector::UpVector * HeightOffset;

	const int32 LastIndex = Samples.Num() - 1;
	if (LastIndex >= 0)
	{
		for (int32 Index = 0; ; ++Index)
		{
			const FVector Pos = Samples[Index].Position + OffsetVector;

			CurrentEntry->AddSphere(Pos, 1, Category.GetCategoryName(), Verbosity, FColor::Black, TEXT(""), true);

			const FRotationMatrix R(FRotator(Samples[Index].Facing));
			const FVector X = R.GetScaledAxis(EAxis::X);
			const FVector Y = R.GetScaledAxis(EAxis::Y);

			const float Scale = 12.f;

			const bool IsPast = Samples[Index].AccumulatedSeconds <= 0.f;

			CurrentEntry->AddSegment(Pos, Pos + X * Scale, Category.GetCategoryName(), Verbosity, IsPast ? FColor::Red : FColor::Blue, TEXT(""), DebugThickness);
			CurrentEntry->AddSegment(Pos, Pos + Y * Scale, Category.GetCategoryName(), Verbosity, IsPast ? FColor::Orange : FColor::Turquoise, TEXT(""), DebugThickness);

			if (Index == LastIndex)
			{
				break;
			}

			const FVector NextPos = Samples[Index + 1].Position + OffsetVector;
			CurrentEntry->AddSegment(Pos, NextPos, Category.GetCategoryName(), Verbosity, FColor::Black, TEXT(""), DebugThickness);
		}
	}
#endif // ENABLE_VISUAL_LOG
}

#endif // ENABLE_ANIM_DEBUG

FArchive& operator<<(FArchive& Ar, FPoseSearchQueryTrajectorySample& TrajectorySample)
{
	Ar << TrajectorySample.Facing;
	Ar << TrajectorySample.Position;
	Ar << TrajectorySample.AccumulatedSeconds;
	return Ar;
}

FPoseSearchQueryTrajectory::FPoseSearchQueryTrajectory(const FTransformTrajectory& InTrajectory)
{
#if WITH_EDITOR
	UE_LOGF(LogPoseSearch, Warning, "Converting FTransformTrajectory to FPoseSearchQueryTrajectory (deprecated type) implicitly. Consider updating code to avoid this conversion.")
#endif
	
	Samples.SetNum(InTrajectory.Samples.Num(), EAllowShrinking::No);
		
	for (int i = 0; i < InTrajectory.Samples.Num(); ++i)
	{
		Samples[i].Position = InTrajectory.Samples[i].Position;
		Samples[i].Facing = InTrajectory.Samples[i].Facing;
		Samples[i].AccumulatedSeconds = InTrajectory.Samples[i].TimeInSeconds;
	}
}

FPoseSearchQueryTrajectory::operator FTransformTrajectory() const
{
	FTransformTrajectory OutTrajectory;

	for (const FPoseSearchQueryTrajectorySample& Sample : Samples)
	{
		OutTrajectory.Samples.Add({ Sample.Facing, Sample.Position, Sample.AccumulatedSeconds });
	}

	return OutTrajectory;
}

FArchive& operator<<(FArchive& Ar, FPoseSearchQueryTrajectory& Trajectory)
{
	Ar << Trajectory.Samples;
	return Ar;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
