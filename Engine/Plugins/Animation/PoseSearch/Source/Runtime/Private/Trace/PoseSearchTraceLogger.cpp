// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/Trace/PoseSearchTraceLogger.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNodeBase.h"
#include "Components/SkeletalMeshComponent.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Trace/Trace.inl"
#include "TraceFilter.h"

UE_TRACE_CHANNEL_DEFINE(PoseSearchChannel, "Traces motion matching animation state data for the PoseSearch system. Captures serialized pose data, \
velocities, search costs, database information, and real-time animation search operations for debugging animation \
matching decisions.");

// No custom versions applied when loading.
UE_TRACE_EVENT_BEGIN(PoseSearch, MotionMatchingState)
	UE_TRACE_EVENT_FIELD(uint8[], Data)
UE_TRACE_EVENT_END()

// We apply the FPoseSearchCustomVersion::DeprecatedTrajectoryTypes version when loading.
UE_TRACE_EVENT_BEGIN(PoseSearch, MotionMatchingState2)
	UE_TRACE_EVENT_FIELD(uint8[], Data)
UE_TRACE_EVENT_END()

// We apply the FPoseSearchCustomVersion::AddedInterruptModeToDebugger version when loading.
UE_TRACE_EVENT_BEGIN(PoseSearch, MotionMatchingState3)
	UE_TRACE_EVENT_FIELD(uint8[], Data)
UE_TRACE_EVENT_END()

namespace UE::PoseSearch
{

const FName FTraceLogger::Name("PoseSearch");
const FName FTraceMotionMatchingStateMessage::Name("MotionMatchingState3");

FArchive& operator<<(FArchive& Ar, FTraceMessage& State)
{
	Ar << State.Cycle;
	Ar << State.AnimInstanceId;
	Ar << State.NodeId;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FTraceMotionMatchingStatePoseEntry& Entry)
{
	Ar << Entry.DbPoseIdx;
	Ar << Entry.Cost;
	Ar << Entry.PoseCandidateFlags;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FTraceMotionMatchingStateDatabaseEntry& Entry)
{
	Ar << Entry.DatabaseId;
	Ar << Entry.QueryVector;
	Ar << Entry.PoseEntries;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FTraceMotionMatchingStateMessage& State)
{
	Ar << static_cast<FTraceMessage&>(State);

	Ar << State.ElapsedPoseSearchTime;
	Ar << State.AssetPlayerTime;
	Ar << State.DeltaTime;
	Ar << State.SimLinearVelocity;
	Ar << State.SimAngularVelocity;
	Ar << State.AnimLinearVelocity;
	Ar << State.AnimAngularVelocity;
	Ar << State.Playrate;
	Ar << State.AnimLinearVelocityNoTimescale;
	Ar << State.AnimAngularVelocityNoTimescale;
	Ar << State.RecordingTime;
	Ar << State.SearchBestCost;
	Ar << State.SearchBruteForceCost;
	Ar << State.SearchBestPosePos;
	Ar << State.SkeletalMeshComponentIds;
	Ar << State.Roles;
	Ar << State.DatabaseEntries;
	Ar << State.PoseHistories;	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Ar << State.CurrentDbEntryIdx;
	Ar << State.CurrentPoseEntryIdx;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// InterruptMode has been introduced with AddedInterruptModeToDebugger version
	if (Ar.CustomVer(FPoseSearchCustomVersion::GUID) >= FPoseSearchCustomVersion::AddedInterruptModeToDebugger)
	{
		Ar << State.InterruptMode;
	}
	return Ar;
}

void FTraceMotionMatchingStateMessage::Input(FPoseSearchCustomVersion::Type CustomVersionType, TConstArrayView<uint8> Data)
{
	FCustomVersionContainer CustomVersionContainer;

	FMemoryReaderView Archive(Data);
	if (CustomVersionType != FPoseSearchCustomVersion::BeforeCustomVersionWasAdded)
	{
		// Ensure we have the version that matches this RouteId event.
		CustomVersionContainer.SetVersion(FPoseSearchCustomVersion::GUID, CustomVersionType, TEXT("Dev-PoseSearch-Version"));
	}

	Archive.SetCustomVersions(CustomVersionContainer);
	// setting SetFilterEditorOnly to have archived data sharable between game and editor
	Archive.SetFilterEditorOnly(true);
	Archive << *this;
}

void FTraceMotionMatchingStateMessage::Output()
{
#if OBJECT_TRACE_ENABLED
	TArray<uint8> ArchiveData;
	FMemoryWriter Archive(ArchiveData);

	// Ensure we have the latest version.
	FCustomVersionContainer CustomVersionContainer;
	CustomVersionContainer.SetVersion(FPoseSearchCustomVersion::GUID, FPoseSearchCustomVersion::LatestVersion, TEXT("Dev-PoseSearch-Version"));
	Archive.SetCustomVersions(CustomVersionContainer);
	// setting SetFilterEditorOnly to have archived data sharable between game and editor
	Archive.SetFilterEditorOnly(true);
	Archive << *this;
	UE_TRACE_LOG(PoseSearch, MotionMatchingState3, PoseSearchChannel) << MotionMatchingState3.Data(ArchiveData.GetData(), ArchiveData.Num());
#endif
}

const UPoseSearchDatabase* FTraceMotionMatchingStateMessage::GetCurrentDatabase() const
{
	unimplemented();
	return nullptr;
}

int32 FTraceMotionMatchingStateMessage::GetCurrentDatabasePoseIndex() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (const FTraceMotionMatchingStatePoseEntry* PoseEntry = GetCurrentPoseEntry())
	{
		return PoseEntry->DbPoseIdx;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return INDEX_NONE;
}

const FTraceMotionMatchingStatePoseEntry* FTraceMotionMatchingStateMessage::GetCurrentPoseEntry() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (DatabaseEntries.IsValidIndex(CurrentDbEntryIdx))
	{
		const FTraceMotionMatchingStateDatabaseEntry& DbEntry = DatabaseEntries[CurrentDbEntryIdx];
		if (DbEntry.PoseEntries.IsValidIndex(CurrentPoseEntryIdx))
		{
			return &DbEntry.PoseEntries[CurrentPoseEntryIdx];
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return nullptr;
}

} // namespace UE::PoseSearch