// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearch/VPTree.h"

namespace UE::PoseSearch
{

void FVPTree::Reset()
{
	Nodes.Reset();
}

SIZE_T FVPTree::GetAllocatedSize() const
{
	return Nodes.GetAllocatedSize();
}

bool FVPTree::operator==(const FVPTree& Other) const
{
	return Nodes == Other.Nodes;
}

FArchive& operator<<(FArchive& Ar, FVPTree& VPTree)
{
	Ar << VPTree.Nodes;
	return Ar;
}

} // namespace UE::PoseSearch