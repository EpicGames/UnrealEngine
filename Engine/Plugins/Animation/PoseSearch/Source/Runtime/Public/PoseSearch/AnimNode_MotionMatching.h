// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNode_AssetPlayerBase.h"
#include "BlendStack/AnimNode_BlendStack.h"
#include "GameplayTagContainer.h"
#include "PoseSearch/PoseSearchInteractionAvailability.h"
#include "PoseSearch/PoseSearchLibrary.h"
#include "AnimNode_MotionMatching.generated.h"

#define UE_API POSESEARCH_API

class UPoseSearchDatabase;
struct FPoseSearchEvent;

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_MotionMatching : public FAnimNode_BlendStack_Standalone
{
	GENERATED_BODY()

public:
	// Search InDatabase instead of the Database property on this node. Use InterruptMode to control the continuing pose search
	UE_API void SetDatabaseToSearch(UPoseSearchDatabase* InDatabase, EPoseSearchInterruptMode InterruptMode);

	// Search InDatabases instead of the Database property on the node. Use InterruptMode to control the continuing pose search.
	UE_API void SetDatabasesToSearch(TConstArrayView<UPoseSearchDatabase*> InDatabases, EPoseSearchInterruptMode InterruptMode);

	// Reset the effects of SetDatabaseToSearch/SetDatabasesToSearch and use the Database property on this node.
	UE_API void ResetDatabasesToSearch(EPoseSearchInterruptMode InterruptMode);

	// Use InterruptMode to control the continuing pose search
	UE_API void SetInterruptMode(EPoseSearchInterruptMode InterruptMode);

	const FMotionMatchingState& GetMotionMatchingState() const { return MotionMatchingState; }

	UE_API FVector GetEstimatedFutureRootMotionVelocity() const;

	UE_API const FAnimNodeFunctionRef& GetOnUpdateMotionMatchingStateFunction() const;

	// FAnimNode_Base interface
	// @todo: implement CacheBones_AnyThread to rebind the schema bones
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual bool GetIgnoreForRelevancyTest() const override;
	UE_API virtual bool SetIgnoreForRelevancyTest(bool bInIgnoreForRelevancyTest) override;
	UE_API virtual FName GetGroupName() const override;
	UE_API virtual EAnimGroupRole::Type GetGroupRole() const override;
	UE_API virtual EAnimSyncMethod GetGroupMethod() const override;
	UE_API virtual bool GetOverridePositionWhenJoiningSyncGroupAsLeader() const override;
	UE_API virtual bool IsLooping() const override;
	UE_API virtual bool SetGroupName(FName InGroupName) override;
	UE_API virtual bool SetGroupRole(EAnimGroupRole::Type InRole) override;
	UE_API virtual bool SetGroupMethod(EAnimSyncMethod InMethod) override;
	UE_API virtual bool SetOverridePositionWhenJoiningSyncGroupAsLeader(bool InOverridePositionWhenJoiningSyncGroupAsLeader) override;
	// End of FAnimNode_Base interface

	UE_API const FVector& GetBlendspaceParameters() const;
	UE_API float GetBlendspaceParametersDeltaThreshold() const;
	UE_API EBlendStack_BlendspaceUpdateMode GetBlendspaceUpdateMode() const;

	// FAnimNode_AssetPlayerBase interface
	UE_API virtual void UpdateAssetPlayer(const FAnimationUpdateContext& Context) override;
	UE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	// End of FAnimNode_AssetPlayerBase interface

private:

#if WITH_EDITORONLY_DATA
	
	// requested blend space blend parameters (if AnimationAsset is a blend space)
	UPROPERTY(EditAnywhere, Category = Blendspace, meta = (PinHiddenByDefault, FoldProperty))
	FVector BlendParameters = FVector::Zero();

	// Use this to define a threshold to trigger a new blend when blendspace xy input pins change.
	// By default, any delta will trigger a blend.
	UPROPERTY(EditAnywhere, Category = Blendspace, meta = (FoldProperty))
	float BlendParametersDeltaThreshold = 0.0f;

	// The group name that we synchronize with (NAME_None if it is not part of any group). Note that
	// this is the name of the group used to sync the output of this node - it will not force
	// syncing of animations contained by it.
	UPROPERTY(EditAnywhere, Category = Sync, meta = (FoldProperty))
	FName GroupName = NAME_None;

	// The role this node can assume within the group (ignored if GroupName is not set). Note
	// that this is the role of the output of this node, not of animations contained by it.
	UPROPERTY(VisibleAnywhere, Category = Sync, meta = (FoldProperty))
	TEnumAsByte<EAnimGroupRole::Type> GroupRole = EAnimGroupRole::ExclusiveAlwaysLeader;
	
	// When enabled, acting as the leader, and using marker-based sync, this asset player will not sync to the previous leader's sync position when joining a sync group and before becoming the leader but instead force everyone else to match its position.
	UPROPERTY(VisibleAnywhere, Category = Sync, meta = (FoldProperty, EditCondition = "GroupRole != EAnimGroupRole::TransitionFollower && GroupRole != EAnimGroupRole::AlwaysFollower", EditConditionHides))
	bool bOverridePositionWhenJoiningSyncGroupAsLeader = true;
	
	// How we should update individual blend space parameters. See dropdown options tooltips.
	UPROPERTY(EditAnywhere, Category = Blendspace, meta = (FoldProperty))
	EBlendStack_BlendspaceUpdateMode BlendspaceUpdateMode = EBlendStack_BlendspaceUpdateMode::InitialOnly;

	// How this node will synchronize with other animations. Note that this determines how the output
	// of this node is used for synchronization, not of animations contained by it.
	UPROPERTY(EditAnywhere, Category = Sync, meta = (FoldProperty))
	EAnimSyncMethod Method = EAnimSyncMethod::DoNotSync;

	// If true, "Relevant anim" nodes that look for the highest weighted animation in a state will ignore this node
	UPROPERTY(EditAnywhere, Category = Relevancy, meta = (FoldProperty, PinHiddenByDefault))
	bool bIgnoreForRelevancyTest = false;

#endif
	
	// The database to search. This can be overridden by Anim Node Functions such as "On Become Relevant" and "On Update" via SetDatabaseToSearch/SetDatabasesToSearch.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinShownByDefault))
	TObjectPtr<const UPoseSearchDatabase> Database = nullptr;

	// Time in seconds to blend out to the new pose. Uses either inertial blending, requiring an Inertialization node after this node, or the internal blend stack, if MaxActiveBlends is greater than zero.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, ClampMin="0"))
	float BlendTime = 0.2f;

	// Set Blend Profiles (editable in the skeleton) to determine how the blending is distributed among your character's bones. It could be used to differentiate between upper body and lower body to blend timing.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, UseAsBlendProfile = true))
	TObjectPtr<UBlendProfile> BlendProfile;

	// How the blend is applied over time to the bones. Common selections are linear, ease in, ease out, and ease in and out.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	EAlphaBlendOption BlendOption = EAlphaBlendOption::Linear;

	// Don't jump to poses of the same segment that are within the interval this many seconds away from the continuing pose.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	FFloatInterval PoseJumpThresholdTime = FFloatInterval(0.f, 0.f);

	// Prevent re-selection of poses that have been selected previously within this much time (in seconds) in the past. This is across all animation segments that have been selected within this time range.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, ClampMin = "0"))
	float PoseReselectHistory = 0.3f;

	// Minimum amount of time to wait between searching for a new pose segment. It allows users to define how often the system searches, default for locomotion is searching every update, but you may only want to search once for other situations, like jump.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, ClampMin="0"))
	float SearchThrottleTime = 0.f;

	// Effective range of play rate that can be applied to the animations to account for discrepancies in estimated velocity between the movement model and the animation.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, ClampMin = "0.2", ClampMax = "3.0", UIMin = "0.2", UIMax = "3.0"))
	FFloatInterval PlayRate = FFloatInterval(1.f, 1.f);
	
	// Experimental: Multiplier applied to the play rate of the selected animation after Motion Matching State has been updated.
	UPROPERTY(EditAnywhere, Category = "Settings | Experimental", meta = (PinHiddenByDefault))
	float PlayRateMultiplier = 1.0f;
	
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bUseInertialBlend = false;

	// Reset the motion matching selection state if it has become relevant to the graph after not being updated on previous frames.
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bResetOnBecomingRelevant = true;

	// If set to false, the motion matching node will perform a search only if the continuing pose is invalid. This is useful if you want to stagger searches of different nodes for performance reasons
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	bool bShouldSearch = true;

	// If set to true, the search of multiple databases with different schemas will try to share pose features data calculated during query build
	// the idea is to be able to share as much as possible the continuing pose features vector across different schemas (and potentially improve performances)
	// defaulted to false to preserve behavior backward compatibility
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	bool bShouldUseCachedChannelData = false;
	
	// Experimental, this feature might be removed without warning, not for production use
	UPROPERTY(EditAnywhere, Category = "Settings | Experimental", meta = (PinHiddenByDefault))
	FPoseSearchEvent EventToSearch;

	// Encapsulated motion matching algorithm and internal state
	UPROPERTY(Transient)
	FMotionMatchingState MotionMatchingState;

	// Update Counter for detecting being relevant
	FGraphTraversalCounter UpdateCounter;

	// List of databases this node is searching.
	UPROPERTY(Transient)
	TArray<TObjectPtr<const UPoseSearchDatabase>> DatabasesToSearch;

	// Applied EPoseSearchInterruptMode on the next update that controls the continuing pose search evaluation. This is set back to EPoseSearchInterruptMode::DoNotInterrupt after each update.
	EPoseSearchInterruptMode NextUpdateInterruptMode = EPoseSearchInterruptMode::DoNotInterrupt;

	// True if the Database property on this node has been overridden by SetDatabaseToSearch/SetDatabasesToSearch.
	bool bOverrideDatabaseInput = false;

	// Experimental, this feature might be removed without warning, not for production use
	UPROPERTY(EditAnywhere, Category = "Interaction | Experimental", meta = (PinHiddenByDefault))
	TArray<FPoseSearchInteractionAvailability> Availabilities;

	// Experimental, this feature might be removed without warning, not for production use
	UPROPERTY(EditAnywhere, Category = "Interaction | Experimental", meta = (PinHiddenByDefault))
	bool bValidateResultAgainstAvailabilities = true;

	// Experimental, this feature might be removed without warning, not for production use
	UPROPERTY(EditAnywhere, Category = "Interaction | Experimental", meta = (PinHiddenByDefault))
	bool bKeepInteractionAlive = false;

	// Experimental, this feature might be removed without warning, not for production use
	// if bEnableWarp is true and this node is performing an interaction, warp aligment will be caluculated
	// and requested to be applied by the anim root motion provider (IAnimRootMotionProvider interface)
	// set it to false if a different warp aligment strategy is used
	UPROPERTY(EditAnywhere, Category = "Interaction | Experimental", meta = (PinHiddenByDefault))
	bool bEnableWarp = false;

	// Experimental, this feature might be removed without warning, not for production use
	// if bWarpUsingRootBone is true, warping will be calculated using the interacting actors previous frame root bone transforms (effective for setups with OffsetRootBone node allowing root bone drifting from capsule)
	// if bWarpUsingRootBone is true, warping will be calculated using the previous frame root transforms (effective root motion driven for setups)
	UPROPERTY(EditAnywhere, Category = "Interaction | Experimental", meta = (PinHiddenByDefault))
	bool bWarpUsingRootBone = true;

	// Experimental, this feature might be removed without warning, not for production use
	// amount or translation warping to apply
	UPROPERTY(EditAnywhere, Category = "Interaction | Experimental", meta = (PinHiddenByDefault, ClampMin = "0", ClampMax = "1"))
	float WarpingTranslationRatio = 1.f;

	// Experimental, this feature might be removed without warning, not for production use
	// amount or rotation warping to apply
	UPROPERTY(EditAnywhere, Category = "Interaction | Experimental", meta = (PinHiddenByDefault, ClampMin = "0", ClampMax = "1"))
	float WarpingRotationRatio = 1.f;

#if WITH_EDITORONLY_DATA
	
	UPROPERTY(meta=(FoldProperty))
	FAnimNodeFunctionRef OnMotionMatchingStateUpdated;

#endif // WITH_EDITORONLY_DATA

	friend class UAnimGraphNode_MotionMatching;
	friend class UMotionMatchingAnimNodeLibrary;

private:
	// Experimental, this feature might be removed without warning, not for production use
	FTransform MeshWithOffset = FTransform::Identity;
	// Experimental, this feature might be removed without warning, not for production use
	FTransform MeshWithoutOffset = FTransform::Identity;
};

#undef UE_API
