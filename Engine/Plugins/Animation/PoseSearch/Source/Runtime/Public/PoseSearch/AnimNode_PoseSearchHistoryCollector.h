// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNodeBase.h"
#include "PoseSearch/PoseSearchHistory.h"
#include "PoseSearch/PoseSearchTrajectoryTypes.h"
#include "Animation/TrajectoryTypes.h"
#include "AnimNode_PoseSearchHistoryCollector.generated.h"

#define UE_API POSESEARCH_API

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_PoseSearchHistoryCollector_Base : public FAnimNode_Base
{
	GENERATED_BODY()

public:
	
	// The maximum amount of poses that can be stored
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ClampMin="2"))
	int32 PoseCount = 2;
	
	// how often in seconds poses are collected (if 0, it will collect every update)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ClampMin="0"))
	float SamplingInterval = 0.04f;

	UPROPERTY(EditAnywhere, Category = Settings)
	TArray<FBoneReference> CollectedBones;

	UPROPERTY(EditAnywhere, Category = Settings)
	TArray<FName> CollectedCurves;

	UPROPERTY()
	bool bInitializeWithRefPose_DEPRECATED = false;

	// Reset the pose history if it has become relevant to the graph after not being updated on previous frames.
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bResetOnBecomingRelevant = true;

	// if true pose scales will be cached, otherwise implied to be unitary scales
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bStoreScales = false;

	// time in seconds to recover to the reference skeleton root bone transform by RootBoneTranslationRecoveryRatio and RootBoneRotationRecoveryRatio
	// from any eventual root bone modification. if zero the behaviour will be disabled
	// Experimental, this feature might be removed without warning, not for production use
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Experimental, meta = (ClampMin="0"))
	float RootBoneRecoveryTime = 0.f;

	// ratio to recover to the reference skeleton root bone translation from any eventual root bone modification. zero for no recovery, 1 for full recovery
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Experimental, meta = (ClampMin="0", ClampMax="1", EditCondition = "RootBoneRecoveryTime > 0", EditConditionHides))
	float RootBoneTranslationRecoveryRatio = 1.f;

	// ratio to recover to the reference skeleton root bone rotation from any eventual root bone modification. zero for no recovery, 1 for full recovery
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Experimental, meta = (ClampMin="0", ClampMax="1", EditCondition = "RootBoneRecoveryTime > 0", EditConditionHides))
	float RootBoneRotationRecoveryRatio = 1.f;

	// Update Counter for detecting being relevant
	FGraphTraversalCounter UpdateCounter;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = Debug)
	FLinearColor DebugColor = FLinearColor::Red;
#endif // WITH_EDITORONLY_DATA

	// if true Trajectory the pose history node will generate the trajectory using the TrajectoryData parameters instead of relying on the input Trajectory
	// Experimental, this feature might be removed without warning, not for production use
	UPROPERTY(EditAnywhere, Category = Experimental)
	bool bGenerateTrajectory = false;

#if WITH_EDITORONLY_DATA

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.6, "Please use TransformTrajectory variable instead.")
	FPoseSearchQueryTrajectory Trajectory;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

#endif // WITH_EDITORONLY_DATA

	// Input Trajectory samples for pose search queries in Motion Matching. These are expected to be in the world space of the SkeletalMeshComponent.
	// the trajectory sample with AccumulatedSeconds equals to zero (Trajectory.Samples[i].AccumulatedSeconds) is the sample of the previous frame of simulation (since MM works by matching the previous character pose)
	UPROPERTY(EditAnywhere, Transient, Category = Settings, DisplayName="Trajectory", meta = (PinShownByDefault, EditCondition="!bGenerateTrajectory", EditConditionHides))
	FTransformTrajectory TransformTrajectory;

	// Input Trajectory velocity will be multiplied by TrajectorySpeedMultiplier: values below 1 will result in selecting animation slower than requested from the original Trajectory
	UPROPERTY(EditAnywhere, Category = Experimental, meta = (PinHiddenByDefault, ClampMin="0", EditCondition="!bGenerateTrajectory", EditConditionHides))
	float TrajectorySpeedMultiplier = 1.f;

	// if bGenerateTrajectory is true, this is the number of trajectory past (collected) samples
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Experimental, meta = (ClampMin = "2", EditCondition = "bGenerateTrajectory", EditConditionHides))
	int32 TrajectoryHistoryCount = 10;

	// if bGenerateTrajectory is true, this is the number of trajectory future (prediction) samples
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Experimental, meta = (ClampMin="2", EditCondition="bGenerateTrajectory", EditConditionHides))
	int32 TrajectoryPredictionCount = 8;

	// if bGenerateTrajectory is true, this is the sampling interval between trajectory future (prediction) samples
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Experimental, meta = (ClampMin="0.001", EditCondition="bGenerateTrajectory", EditConditionHides))
	float PredictionSamplingInterval = 0.4f;

	// if bGenerateTrajectory is true, TrajectoryData contains the tuning parameters to generate the trajectory
	UPROPERTY(EditAnywhere, Category = Experimental, meta=(EditCondition="bGenerateTrajectory", EditConditionHides))
	FPoseSearchTrajectoryData TrajectoryData;

	bool bCacheBones = false;

	// FAnimNode_Base interface
	virtual bool NeedsOnInitializeAnimInstance() const override { return true; }
	UE_API virtual void OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance) override;
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	UE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	// End of FAnimNode_Base interface

	// GetPoseHistory got deprecated in favor of GetPoseHistoryPtr, since FAnimNode_PoseSearchHistoryCollector_Base::PoseHistoryPtr can be invalid during blue print
	// recompilation when the pose history node exists in the graph but on an inactive branch, for instance on a never activated branch of a blend by bool node
	const UE::PoseSearch::FPoseHistory* GetPoseHistoryPtr() const { return PoseHistoryPtr.Get(); }
	UE::PoseSearch::FPoseHistory* GetPoseHistoryPtr() { return PoseHistoryPtr.Get(); }

	UE_DEPRECATED(5.8, "Use GetPoseHistoryPtr instead")
	const UE::PoseSearch::FPoseHistory& GetPoseHistory() const { check(PoseHistoryPtr.IsValid()); return *PoseHistoryPtr.Get(); }
	
	UE_DEPRECATED(5.8, "Use GetPoseHistoryPtr instead")
	UE::PoseSearch::FPoseHistory& GetPoseHistory() { check(PoseHistoryPtr.IsValid()); return *PoseHistoryPtr.Get(); }
	
	FPoseHistoryReference GetPoseHistoryReference() const { check(PoseHistoryPtr.IsValid()); return FPoseHistoryReference {PoseHistoryPtr}; }

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	// this is a requirement for clang to compile without warnings.
	FAnimNode_PoseSearchHistoryCollector_Base() = default;
	~FAnimNode_PoseSearchHistoryCollector_Base() = default;
	FAnimNode_PoseSearchHistoryCollector_Base(const FAnimNode_PoseSearchHistoryCollector_Base&) = default;
	FAnimNode_PoseSearchHistoryCollector_Base(FAnimNode_PoseSearchHistoryCollector_Base&&) = default;
	FAnimNode_PoseSearchHistoryCollector_Base& operator=(const FAnimNode_PoseSearchHistoryCollector_Base&) = default;
	FAnimNode_PoseSearchHistoryCollector_Base& operator=(FAnimNode_PoseSearchHistoryCollector_Base&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Experimental, this feature might be removed without warning, not for production use
	UE_EXPERIMENTAL(5.8, "This API is experimental and might be removed without warning, not for production use.")
	TArray<FBoneIndexType> GetRequiredBones(const USkeletalMeshComponent* SkeletalMeshComponent) const;
	
protected:
	// returns an array of skeleton bone indexes (encoded as FBoneIndexType)
	TArray<FBoneIndexType> GetRequiredBones(const FAnimInstanceProxy* AnimInstanceProxy) const;

private:
	TSharedPtr<UE::PoseSearch::FGenerateTrajectoryPoseHistory, ESPMode::ThreadSafe> PoseHistoryPtr;
};

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_PoseSearchHistoryCollector : public FAnimNode_PoseSearchHistoryCollector_Base
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Links, meta = (DisplayPriority = 0))
	FPoseLink Source;

	// FAnimNode_Base interface
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	UE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	UE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface
};

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_PoseSearchComponentSpaceHistoryCollector : public FAnimNode_PoseSearchHistoryCollector_Base
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Links, meta = (DisplayPriority = 0))
	FComponentSpacePoseLink Source;

	// FAnimNode_Base interface
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	UE_API virtual void EvaluateComponentSpace_AnyThread(FComponentSpacePoseContext& Output) override;
	UE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface
};

#undef UE_API
