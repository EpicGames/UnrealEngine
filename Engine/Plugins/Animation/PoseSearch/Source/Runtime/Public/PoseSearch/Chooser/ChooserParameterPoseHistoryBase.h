// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <IHasContext.h>

#include "CoreMinimal.h"
#include "IChooserParameterBase.h"
#include "PoseSearch/PoseSearchHistory.h"
#include "ChooserParameterPoseHistoryBase.generated.h"

USTRUCT()
struct FChooserParameterPoseHistoryBase : public FChooserParameterBase
{
	GENERATED_BODY()
	virtual bool GetValue(FChooserEvaluationContext& Context, FPoseHistoryReference& OutResult) const { return false; }
	virtual bool GetValue(FChooserEvaluationContext& Context, FPoseSearchHistory& OutResult) const { return false; }
	virtual bool IsBound() const { return false; }
};