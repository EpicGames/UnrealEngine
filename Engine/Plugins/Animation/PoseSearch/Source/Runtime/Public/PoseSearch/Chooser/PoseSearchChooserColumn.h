// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChooserPropertyAccess.h"
#include "CoreMinimal.h"
#include "IChooserColumn.h"
#include "PoseSearch/Chooser/ChooserParameterPoseHistoryBase.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "StructUtils/InstancedStruct.h"
#include "PoseSearchChooserColumn.generated.h"

#define UE_API POSESEARCH_API

class UAnimationAsset;

namespace UE::PoseSearch
{
	// Experimental, this feature might be removed without warning, not for production use
	struct FActiveColumnCost
	{
		int32 RowIndex = INDEX_NONE;
		float RowCost = 0.f;
	};
	FArchive& operator<<(FArchive& Ar, FActiveColumnCost& ActiveColumnCost);

	// Experimental, this feature might be removed without warning, not for production use
	struct FMapping 
	{
	private:
		struct FTableCell 
		{
			FPoseSearchColumn* Column = nullptr;
			int32 RowIndex = INDEX_NONE;

			bool operator == (const FTableCell& Other) const
			{
				return Column == Other.Column && RowIndex == Other.RowIndex;
			}
		};

		typedef TArray<FTableCell> FTableCells;
		TMap<UPoseSearchDatabase*, FTableCells> Map;
	
		
	public:
		explicit FMapping(UChooserTable* Chooser = nullptr);
		void Init(UChooserTable* Chooser);
		bool Equals(const FMapping& Other) const;
		int32 GetNumIndexesInDatabase(const UPoseSearchDatabase* Database) const;
		int32 FindIndexInDatabase(int32 RowIndex, const FPoseSearchColumn* Column, const UPoseSearchDatabase* Database) const;
		const FPoseSearchDatabaseAnimationAsset* FindDatabaseAnimationAsset(int32 IndexInDatabase, const UPoseSearchDatabase* Database) const;

#if WITH_EDITORONLY_DATA
		void SetAnimationAssetAt(const FPoseSearchDatabaseAnimationAsset& AnimationAsset, int32 IndexInDatabase, const UPoseSearchDatabase* Database);
#endif // WITH_EDITORONLY_DATA

		int32 GetRowIndex(int32 IndexInDatabase, const UPoseSearchDatabase* Database) const;
		void ApplyMappingToAllColumns();

		static FMapping* FindMapping(UChooserTable* Chooser);
	};

} // namespace UE::PoseSearch


USTRUCT(Experimental, DisplayName = "Pose History Property Binding")
struct FPoseHistoryContextProperty :  public FChooserParameterPoseHistoryBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Meta = (BindingType = "FPoseHistoryReference", BindingAllowFunctions = "true", BindingColor = "StructPinTypeColor"), Category = "Binding")
	FChooserPropertyBinding Binding;
	
	virtual bool GetValue(FChooserEvaluationContext& Context, FPoseHistoryReference& OutResult) const override
	{
		return Binding.GetStructValue(Context, OutResult);
	}

	virtual bool GetValue(FChooserEvaluationContext& Context, FPoseSearchHistory& OutResult) const override
	{
		return Binding.GetStructValue(Context, OutResult);
	}

	virtual bool IsBound() const override
	{
		return Binding.IsBoundToRoot || !Binding.PropertyBindingChain.IsEmpty();
	}

	CHOOSER_PARAMETER_BOILERPLATE();
};

USTRUCT(Experimental)
struct FPoseSearchColumnRow
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Data")
	FPoseSearchDatabaseAnimationAsset Data;

	UPROPERTY(EditAnywhere, Category = "Data")
	TObjectPtr<UPoseSearchDatabase> Database;

	friend bool operator==(const FPoseSearchColumnRow& A, const FPoseSearchColumnRow& B)
	{
		return A.Data == B.Data && A.Database == B.Database;
	}
};
template<> struct TStructOpsTypeTraits<FPoseSearchColumnRow> : public TStructOpsTypeTraitsBase2<FPoseSearchColumnRow>
{
	enum { WithIdenticalViaEquality = true };
};

USTRUCT(Experimental)
struct FPoseSearchColumnRowReflection : public FPoseSearchColumnRow
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category = "Data")
	TObjectPtr<UChooserTable> RootChooser;

#endif // WITH_EDITORONLY_DATA
};

USTRUCT(DisplayName = "Pose Match", Meta = (Experimental, Category = "Experimental", Tooltip = "This column filters out all assets except the one which is selected by motion matching query.  Results must be AnimationAssets with a PoseSearchBranchIn notify state.  It also outputs OutputStartTime to specify the frame which matched pose best.  To work as intended it must be placed last (furthest right) in the Chooser so that other filters are applied first."))
struct FPoseSearchColumn : public FChooserColumnBase
{
	GENERATED_BODY()
	
private:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UPoseSearchDatabase> InternalDatabase_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY(VisibleAnywhere, Category = "Data")
	TArray<FPoseSearchColumnRow> RowValues;

	UPROPERTY(VisibleAnywhere, Category = "Data")
	FPoseSearchColumnRow FallbackValue;

	UPROPERTY(VisibleAnywhere, Category = "Data")
	TObjectPtr<UChooserTable> Chooser;

	// @todo: currently Mapping is duplicated in every FPoseSearchColumn and it'd be nice to store it somewhere in the RootChooserTable only. 
	//        That would also speed up the complexity of FMapping::FindMapping.
	//
	// structure containing the mapping between (FPoseSearchColumn, RowIndex) to (UPoseSearchDatabase, IndexInDatabase). 
	// Not serialized, but generated on PostLoad and regenerated when Chooser->GetRootChooser() properties changes
	UE::PoseSearch::FMapping Mapping;

public:
	// @todo: expose it as pinnable parameter
	// Prevent re-selection of poses that have been selected previously within this much time (in seconds) in the past. This is across all animation segments that have been selected within this time range.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, ClampMin = "0"))
	float PoseReselectHistory = 0.3f;

	// @todo: expose it as pinnable parameter
	// MaxNumberOfResults represent the maximum number of results from the motion matching search. if MaxNumberOfResults <= 0 the column will add ALL the results to the output
	// if bEnableMultiSelection is true, this FPoseSearchColumn will be set as cost column, retrieving multiple results from the MM search (one per chooser table entry)
	// and setting their relative costs to be used with subsequent column such as the FRandomizeColumn for additional results refinements.
	// if bEnableMultiSelection is false, this FPoseSearchColumn will return ONLY the best result, the one with the lowest cost
	UPROPERTY(EditAnywhere, Category = "Data")
	int32 MaxNumberOfResults = 1;

	// @todo: filter to allow only enums of type EPoseSearchInterruptMode
	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterEnumBase", ToolTip="Applied EPoseSearchInterruptMode controlling the continuing pose search evaluation. Defaulted to EPoseSearchInterruptMode::DoNotInterrupt if not set"), Category = "Data")
	FInstancedStruct InterruptMode;

	// Pose History
	UPROPERTY(EditAnywhere, DisplayName = "Pose History", NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/PoseSearch.ChooserParameterPoseHistoryBase"), Category = "Data")
	FInstancedStruct InputValue;
	
	// Float output for the start time with the best matching pose
	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterFloatBase"), Category = "Output")
	FInstancedStruct OutputStartTime;
	
	// Bool output for if asset should be mirrored
	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterBoolBase"), Category = "Output")
	FInstancedStruct OutputMirror;

	// Bool output for suggesting a chooser player to force a blend into the newly selected asset
	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterBoolBase"), Category = "Output")
	FInstancedStruct OutputForceBlendTo;

	// Float output for the cost of the selected pose
	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterFloatBase"), Category = "Output")
	FInstancedStruct OutputCost;

	FPoseSearchColumn() = default;
	UE_API FPoseSearchColumn(const FPoseSearchColumn& Other);
	UE_API FPoseSearchColumn& operator=(const FPoseSearchColumn& Other);

	UE_API virtual void PostLoad() override;

	UE_API UPoseSearchDatabase* GetDatabase(int32 RowIndex) const;
#if WITH_EDITOR
	UE_API void SwitchDatabase(UPoseSearchDatabase* NewDatabase, int32 RowIndex);
#endif // WITH_EDITOR

	const FPoseSearchColumnRow& GetRowValue(int32 RowIndex) const;
	FPoseSearchColumnRow& EditRowValue(int32 RowIndex);

#if WITH_EDITOR
	static void InsertAnimationAssetAt(const FPoseSearchDatabaseAnimationAsset& AnimationAsset, int32 AnimationAssetIndex, UChooserTable* Chooser, const UPoseSearchDatabase* Database);
	static void RemoveAnimationAssetAt(int32 AnimationAssetIndex, UChooserTable* Chooser, const UPoseSearchDatabase* Database);
	static void SetAnimationAssetAt(const FPoseSearchDatabaseAnimationAsset& AnimationAsset, int32 AnimationAssetIndex, UChooserTable* Chooser, const UPoseSearchDatabase* Database);
	const UChooserTable* GetChooser() const { return Chooser.Get(); }
#endif // WITH_EDITOR

	static const FPoseSearchDatabaseAnimationAsset* GetDatabaseAnimationAsset(int32 AnimationAssetIndex, UChooserTable* Chooser, const UPoseSearchDatabase* Database);
	static int32 GetNumAnimationAssets(UChooserTable* Chooser, const UPoseSearchDatabase* Database);

	UE::PoseSearch::FMapping& GetMapping() { return Mapping; }
	void UpdateMapping(UE::PoseSearch::FMapping& NewMapping);

	int32 GetNumRows() const { return RowValues.Num(); }
private:
	virtual bool HasFilters() const override { return true; }
	virtual void Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut, TArrayView<uint8> ScratchArea) const override;
	virtual bool HasCosts() const override { return true; }
	virtual bool HasOutputs() const { return true; }
	virtual void SetOutputs(FChooserEvaluationContext& Context, int RowIndex, TArrayView<uint8> ScratchArea) const override;

	virtual int32 GetScratchAreaSize() const override;
	virtual void InitializeScratchArea(TArrayView<uint8> ScratchArea) const override;
	virtual void DeinitializeScratchArea(TArrayView<uint8> ScratchArea) const override;

#if WITH_EDITOR
	virtual void Initialize(UChooserTable* OuterChooser) override;
	
	mutable TArray<UE::PoseSearch::FActiveColumnCost> ActiveColumnCosts;

	virtual bool EditorTestFilter(int32 RowIndex) const override;
	virtual float EditorTestCost(int32 RowIndex) const override;
	virtual void SetTestValue(TArrayView<const uint8> Value) override;

	virtual void AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;

	virtual void ReplaceReferences(UObject* ReferenceToReplace, UObject* ReplaceWith) override;

	virtual FName RowValuesPropertyName() override;
	virtual void SetNumRows(int32 NumRows) override;
	virtual void InsertRows(int Index, int Count);
	virtual void DeleteRows(TArrayView<int> RowIndices);
	virtual void MoveRow(int SourceRowIndex, int TargetRowIndex);
	virtual void CopyRow(FChooserColumnBase& SourceColumn, int SourceRowIndex, int TargetRowIndex) override;
	virtual void CopyFallback(FChooserColumnBase& SourceColumn) override;
	
	virtual bool AutoPopulates() const override { return true; }
	virtual void AutoPopulate(int32 RowIndex, UObject* OutputObject) override;

	virtual UScriptStruct* GetInputBaseType() const override;
	virtual const UScriptStruct* GetInputType() const override;
	virtual FInstancedStruct* GetInputValuePtr() override;
	virtual void SetInputType(const UScriptStruct* Type) override;
#endif // WITH_EDITOR

	virtual FChooserParameterBase* GetInputValue() override;
};

#undef UE_API
