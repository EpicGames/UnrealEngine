// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/IsSorted.h"
#include "Animation/AnimTypes.h"
#include "Containers/ArrayView.h"
#include "HAL/Platform.h"

#ifndef UE_POSE_SEARCH_ENABLE_ITERATIVE_KDTREE_CONSTRUCTION_VALIDATION
#define UE_POSE_SEARCH_ENABLE_ITERATIVE_KDTREE_CONSTRUCTION_VALIDATION ENABLE_ANIM_DEBUG
#endif

class FArchive;

namespace UE::PoseSearch
{
enum { AccessorTypeMax = INT_MAX };
typedef int32 AccessorType;

struct FKDTreeImplementation;

struct FKDTree
{
	struct FDataSource
	{
		FDataSource(AccessorType pointCount, AccessorType pointDim, const float* data)
			: PointCount((pointDim > 0 && data) ? pointCount : 0)
			, PointDim(pointDim)
			, Data(data)
		{
		}

		~FDataSource()
		{
			PointCount = 0;
			PointDim = 0;
			Data = nullptr;
		}

		FDataSource() = default;
		FDataSource(const FDataSource& Other) = default;
		FDataSource(FDataSource&& Other) = default;
		FDataSource& operator=(const FDataSource& Other) = default;
		FDataSource& operator=(FDataSource&& Other) = default;
		bool operator==(const FDataSource& Other) const
		{
			if (PointCount != Other.PointCount)
			{
				return false;
			}

			if (PointDim != Other.PointDim)
			{
				return false;
			}

			const AccessorType DataSize = PointCount * PointDim;
			for (AccessorType DataIndex = 0; DataIndex < DataSize; ++DataIndex)
			{
				if (Data[DataIndex] != Other.Data[DataIndex])
				{
					return false;
				}
			}

			return true;
		}

		// Must return the number of data points
		inline AccessorType kdtree_get_point_count() const { return PointCount; }

		// this method is called by nanoflann::findNeighbors -> nanoflann::searchLevel, so it has to be as fast as possible
		inline float kdtree_get_pt(const AccessorType idx, const AccessorType dim) const { return Data[idx * PointDim + dim]; }

		// Optional bounding-box computation: return false to default to a standard
		// bbox computation loop.
		//   Return true if the BBOX was already computed by the class and returned
		//   in "bb" so it can be avoided to redo it again. Look at bb.size() to
		//   find out the expected dimensionality (e.g. 2 or 3 for point clouds)
		template <class BBOX> bool kdtree_get_bbox(BBOX& /* bb */) const { return false; }

		AccessorType PointCount = 0;
		AccessorType PointDim = 0;
		const float* Data = nullptr;

#if UE_POSE_SEARCH_ENABLE_ITERATIVE_KDTREE_CONSTRUCTION_VALIDATION
		bool bUseRecursiveDivideTree = false;
#endif // UE_POSE_SEARCH_ENABLE_ITERATIVE_KDTREE_CONSTRUCTION_VALIDATION
	};

	struct FKNNResultSet
	{
		inline FKNNResultSet(AccessorType InNumNeighbors, TArrayView<AccessorType> InIndexes, TArrayView<float> InDistances, float InitValue = UE_BIG_NUMBER)
		: Indexes(InIndexes)
		, Distances(InDistances)
		, NumNeighbors(InNumNeighbors)
		, Count(0)
		{
			// by having IndexesView and DistancesView cardinality bigger than NumNeighbors, we can skip some if statements in the addPoint method
			check(NumNeighbors > 0);
			check(InIndexes.Num() > NumNeighbors);
			check(InDistances.Num() > NumNeighbors);

			Distances[NumNeighbors - 1] = InitValue;
		}

		inline AccessorType Num() const
		{
			return Count;
		}

		inline bool full() const
		{
			return Count == NumNeighbors;
		}

		inline bool addPoint(float dist, AccessorType index)
		{
			// shifting Distances[i] and Indexes[i] to make space for "dist" and "index" at the right "i"th slot
			AccessorType i;
			for (i = Count; (i > 0) && (Distances[i - 1] > dist); --i)
			{
				// no need to check "if (i < capacity)" since dists and indices can contains more items than capacity_ 
				Distances[i] = Distances[i - 1];
				Indexes[i] = Indexes[i - 1];
			}
			
			// inserting "dist" and "index" in a sorted manner
			// no need to check "if (i < capacity)" since dists and indices can contains more items than capacity_ 
			Distances[i] = dist;
			Indexes[i] = index;
			
			if (Count < NumNeighbors)
			{
				Count++;
			}

			// tell caller that the search shall continue
			return true;
		}

		inline float worstDist() const { return Distances[NumNeighbors - 1]; }

	protected:
		TArrayView<AccessorType> Indexes;
		TArrayView<float> Distances;
		AccessorType NumNeighbors;
		AccessorType Count;
	};

	struct FFilteredKNNResultSet : public FKNNResultSet
	{
		inline FFilteredKNNResultSet(AccessorType InNumNeighbors, TArrayView<AccessorType> InIndexes, TArrayView<float> InDistances, TConstArrayView<AccessorType> InExcludeFromSearchIndexes = TConstArrayView<AccessorType>())
		: FKNNResultSet(InNumNeighbors, InIndexes, InDistances)
		, ExcludeFromSearchIndexes(InExcludeFromSearchIndexes)
		{
			check(Algo::IsSorted(InExcludeFromSearchIndexes));
		}

		inline bool addPoint(float dist, AccessorType index)
		{
			if (Algo::BinarySearch(ExcludeFromSearchIndexes, index) == INDEX_NONE)
			{
				FKNNResultSet::addPoint(dist, index);
			}
			return true;
		}

	protected:
		TConstArrayView<AccessorType> ExcludeFromSearchIndexes; // sorted array view
	};

	struct FRadiusResultSet : public FKNNResultSet
	{
		inline FRadiusResultSet(float Radius, AccessorType InNumNeighbors, TArrayView<AccessorType> InIndexes, TArrayView<float> InDistances)
		: FKNNResultSet(InNumNeighbors, InIndexes, InDistances, Radius)
		{
		}

		inline bool addPoint(float dist, AccessorType index)
		{
			if (dist < worstDist())
			{
				FKNNResultSet::addPoint(dist, index);
			}
			return true;
		}
	};

	// Experimental, this feature might be removed without warning, not for production use
	struct FKNNMaxHeapResultSet
	{
		struct FResult
		{
			float Distance;
			AccessorType Index;

			inline bool operator()(const FResult& A, const FResult& B) const
			{
				// making it a max heap
				return A.Distance > B.Distance;
			}
		};

		inline FKNNMaxHeapResultSet(TArrayView<FResult> Results, float InitWorstDist = UE_BIG_NUMBER)
		: MaxHeap(Results)
		, MaxHeapNum(0)
		, WorstDist(InitWorstDist)
		{
		}

		inline int32 Num() const
		{
			return MaxHeapNum;
		}
	
		inline bool full() const
		{
			return MaxHeap.Num() == MaxHeapNum;
		}

		inline bool addPoint(float dist, AccessorType index)
		{
			if (full())
			{
				if (dist < MaxHeap[0].Distance)
				{
					MaxHeap[0].Distance = dist;
					MaxHeap[0].Index = index;

					AlgoImpl::HeapSiftDown(MaxHeap.GetData(), 0, MaxHeapNum, FIdentityFunctor(), FResult());
					WorstDist = MaxHeap[0].Distance;
				}
			}
			else
			{
				MaxHeap[MaxHeapNum] = { dist, index };
				// calling HeapSiftUp every addPoint seems to be faster than calling HeapifyInternal the first time MaxHeap is full
				// @todo: do more profiling! (comment out AlgoImpl::HeapSiftUp and uncomment below AlgoImpl::HeapifyInternal)
				AlgoImpl::HeapSiftUp(MaxHeap.GetData(), 0, MaxHeapNum, FIdentityFunctor(), FResult());
				++MaxHeapNum;

				if (full())
				{
					//AlgoImpl::HeapifyInternal(MaxHeap.GetData(), MaxHeapNum, FIdentityFunctor(), FResult());
					WorstDist = MaxHeap[0].Distance;
				}
			}

			// tell caller that the search shall continue
			return true;
		}

		inline float worstDist() const
		{
			return WorstDist;
		}

	protected:
		TArrayView<FResult> MaxHeap;
		int32 MaxHeapNum;
		float WorstDist;
	};

	// Experimental, this feature might be removed without warning, not for production use
	struct FFilteredKNNMaxHeapResultSet : public FKNNMaxHeapResultSet
	{
		inline FFilteredKNNMaxHeapResultSet(TArrayView<FResult> Results, TConstArrayView<AccessorType> InExcludeFromSearchIndexes)
		: FKNNMaxHeapResultSet(Results)
		, ExcludeFromSearchIndexes(InExcludeFromSearchIndexes)
		{
			check(Algo::IsSorted(InExcludeFromSearchIndexes));
		}

		inline bool addPoint(float dist, AccessorType index)
		{
			if (Algo::BinarySearch(ExcludeFromSearchIndexes, index) == INDEX_NONE)
			{
				FKNNMaxHeapResultSet::addPoint(dist, index);
			}
			return true;
		}

	private:
		TConstArrayView<AccessorType> ExcludeFromSearchIndexes; // sorted array view
	};

	// Experimental, this feature might be removed without warning, not for production use
	struct FRadiusMaxHeapResultSet : public FKNNMaxHeapResultSet
	{
		inline FRadiusMaxHeapResultSet(TArrayView<FResult> Results, float Radius)
		: FKNNMaxHeapResultSet(Results, Radius)
		{
		}

		inline bool addPoint(float dist, AccessorType index)
		{
			if (dist < worstDist())
			{
				FKNNMaxHeapResultSet::addPoint(dist, index);
			}
			return true;
		}
	};
	

	FKDTree(AccessorType Count, AccessorType Dim, const float* Data, AccessorType MaxLeafSize = 16
#if UE_POSE_SEARCH_ENABLE_ITERATIVE_KDTREE_CONSTRUCTION_VALIDATION
		, bool bUseRecursiveDivideTree = false
#endif // UE_POSE_SEARCH_ENABLE_ITERATIVE_KDTREE_CONSTRUCTION_VALIDATION
	);
	FKDTree();
	FKDTree(const FKDTree& Other);
	FKDTree(FKDTree&& Other);
	
	~FKDTree();

	FKDTree& operator=(const FKDTree& Other);
	FKDTree& operator=(FKDTree&& Other);
	bool operator==(const FKDTree& Other) const;
	
	void Reset();
	void Construct(AccessorType Count, AccessorType Dim, const float* Data, AccessorType MaxLeafSize = 16);

	int32 FindNeighbors(FKNNResultSet& Result, TConstArrayView<float> Query) const;
	int32 FindNeighbors(FFilteredKNNResultSet& Result, TConstArrayView<float> Query) const;
	int32 FindNeighbors(FRadiusResultSet& Result, TConstArrayView<float> Query) const;
	
	// Experimental, this feature might be removed without warning, not for production use
	int32 FindNeighbors(FKNNMaxHeapResultSet& Result, TConstArrayView<float> Query) const;
	// Experimental, this feature might be removed without warning, not for production use
	int32 FindNeighbors(FFilteredKNNMaxHeapResultSet& Result, TConstArrayView<float> Query) const;
	// Experimental, this feature might be removed without warning, not for production use
	int32 FindNeighbors(FRadiusMaxHeapResultSet& Result, TConstArrayView<float> Query) const;

	POSESEARCH_API SIZE_T GetAllocatedSize() const;

	FDataSource DataSource;
	FKDTreeImplementation* KDTreeImplementation = nullptr;
};

FArchive& Serialize(FArchive& Ar, FKDTree& KDTree, const float* data);
} // namespace UE::PoseSearch