// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimExecutionContext.h"
#include "Animation/AnimNodeReference.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "MotionMatchingAnimNodeLibrary.generated.h"

#define UE_API POSESEARCH_API

struct FAnimNode_MotionMatching;
enum class EAlphaBlendOption : uint8;
class UBlendProfile;
class UPoseSearchDatabase;

USTRUCT(BlueprintType)
struct FMotionMatchingBlueprintBlendSettings
{
	GENERATED_BODY()

	FMotionMatchingBlueprintBlendSettings();

	bool Serialize(FArchive& Ar);
	bool Serialize(FStructuredArchive::FSlot Slot);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	float BlendTime;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	TObjectPtr<UBlendProfile> BlendProfile;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	EAlphaBlendOption BlendOption;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bUseInertialBlend;
};

template<> struct TStructOpsTypeTraits<FMotionMatchingBlueprintBlendSettings> : public TStructOpsTypeTraitsBase2<FMotionMatchingBlueprintBlendSettings>
{
	enum
	{
		WithSerializer = true,
		WithStructuredSerializer = true,
	};
};

USTRUCT(BlueprintType)
struct FMotionMatchingAnimNodeReference : public FAnimNodeReference
{
	GENERATED_BODY()

	typedef FAnimNode_MotionMatching FInternalNodeType;
};

// Exposes operations that can be run on a Motion Matching node via Anim Node Functions such as "On Become Relevant" and "On Update".
UCLASS(MinimalAPI)
class UMotionMatchingAnimNodeLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Get a motion matching node context from an anim node context */
	UFUNCTION(BlueprintCallable, Category = "Animation|MotionMatching", meta = (BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static UE_API FMotionMatchingAnimNodeReference ConvertToMotionMatchingNode(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result);

	/** Get a motion matching node context from an anim node context (pure) */
	UFUNCTION(BlueprintPure, Category = "Animation|MotionMatching", meta = (BlueprintThreadSafe, DisplayName = "Convert to Motion Matching Node"))
	static void ConvertToMotionMatchingNodePure(const FAnimNodeReference& Node, FMotionMatchingAnimNodeReference& MotionMatchingNode, bool& Result)
	{
		EAnimNodeReferenceConversionResult ConversionResult;
		MotionMatchingNode = ConvertToMotionMatchingNode(Node, ConversionResult);
		Result = (ConversionResult == EAnimNodeReferenceConversionResult::Succeeded);
	}

	UFUNCTION(BlueprintPure, Category = "Animation|MotionMatching", meta = (BlueprintThreadSafe, DisplayName = "Get Motion Matching Search Result"))
	static UE_API void GetMotionMatchingSearchResult(const FMotionMatchingAnimNodeReference& MotionMatchingNode, FPoseSearchBlueprintResult& Result, bool& bIsResultValid);

	// Get current blend settings used when blending into a new asset
	UFUNCTION(BlueprintPure, Category = "Animation|MotionMatching", meta = (BlueprintThreadSafe, DisplayName = "Get Motion Matching Blend Settings"))
	static UE_API void GetMotionMatchingBlendSettings(const FMotionMatchingAnimNodeReference& MotionMatchingNode, FMotionMatchingBlueprintBlendSettings& BlendSettings, bool& bIsResultValid);

	// Override current blend settings for motion matching. Note that any pinned parameters will stomp this override on the next update.
	UFUNCTION(BlueprintCallable, Category = "Animation|MotionMatching", meta = (BlueprintThreadSafe, DisplayName = "Override Motion Matching Blend Settings"))
	static UE_API void OverrideMotionMatchingBlendSettings(const FMotionMatchingAnimNodeReference& MotionMatchingNode, const FMotionMatchingBlueprintBlendSettings& BlendSettings);

	/**
	 * Set the database to search on the motion matching node. This overrides the Database property on the motion matching node.
	 * @param MotionMatchingNode - The motion matching node to operate on.
	 * @param Database - The database for the motion matching node to search.
	 * @param InterruptMode - mode to control the continuing pose search (the current animation that's playing)
	 */
	UFUNCTION(BlueprintCallable, Category = "Animation|MotionMatching", meta = (BlueprintThreadSafe))
	static UE_API void SetDatabaseToSearch(const FMotionMatchingAnimNodeReference& MotionMatchingNode, UPoseSearchDatabase* Database, EPoseSearchInterruptMode InterruptMode);

	/**
	 * Set the database to search on the motion matching node. This overrides the Database property on the motion matching node.
	 * @param MotionMatchingNode - The motion matching node to operate on.
	 * @param Databases - Array of databases for the motion matching node to search.
	 * @param InterruptMode - mode to control the continuing pose search (the current animation that's playing)
	 */
	UFUNCTION(BlueprintCallable, Category = "Animation|MotionMatching", meta = (BlueprintThreadSafe))
	static UE_API void SetDatabasesToSearch(const FMotionMatchingAnimNodeReference& MotionMatchingNode, const TArray<UPoseSearchDatabase*>& Databases, EPoseSearchInterruptMode InterruptMode);

	/**
	 * Clear the effects of SetDatabaseToSearch/SetDatabasesToSearch and resume searching the Database property on the motion matching node.
	 * @param MotionMatchingNode - The motion matching node to operate on.
	 * @param InterruptMode - mode to control the continuing pose search (the current animation that's playing)
	 */
	UFUNCTION(BlueprintCallable, Category = "Animation|MotionMatching", meta = (BlueprintThreadSafe))
	static UE_API void ResetDatabasesToSearch(const FMotionMatchingAnimNodeReference& MotionMatchingNode, EPoseSearchInterruptMode InterruptMode);

	/**
	 * Ignore the continuing pose (the current clip that's playing) and force a new search.
	 * @param MotionMatchingNode - The motion matching node to operate on.
	 * @param InterruptMode - mode to control the continuing pose search (the current animation that's playing)
	 */
	UFUNCTION(BlueprintCallable, Category = "Animation|MotionMatching", meta = (BlueprintThreadSafe))
	static UE_API void SetInterruptMode(const FMotionMatchingAnimNodeReference& MotionMatchingNode, EPoseSearchInterruptMode InterruptMode);

	/**
	 * returns true if any new BlendTo API has been called so far during the current frame. BlendTo is the BlendStack API used by the motion matching node to blend into a newly selected animation
	 * @param MotionMatchingNode - The motion matching node to operate on.
	 */
	UFUNCTION(Experimental, BlueprintPure, Category = "Animation|MotionMatching", meta = (BlueprintThreadSafe))
	static UE_API bool AnyNewBlendToThisFrame(const FMotionMatchingAnimNodeReference& MotionMatchingNode);
};

#undef UE_API
