// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// @todo: move UMultiAnimAsset as well as IMultiAnimAssetEditor to Engine or a base plugin for multi character animation assets

#include "MultiAnimAsset.generated.h"

#define UE_API POSESEARCH_API

class UMirrorDataTable;
class UAnimationAsset;
class USkeletalMesh;

// UObject defining tuples of UAnimationAsset(s) with associated Role(s) and relative transforms from a shared reference system via GetOrigin
UCLASS(MinimalAPI, Abstract, Experimental, BlueprintType, Category = "Animation")
class UMultiAnimAsset : public UObject
{
	GENERATED_BODY()
public:

	[[nodiscard]] virtual bool IsLooping() const PURE_VIRTUAL(UMultiAnimAsset::IsLooping, return false;);
	[[nodiscard]] virtual bool HasRootMotion() const PURE_VIRTUAL(UMultiAnimAsset::HasRootMotion, return false;);
	[[nodiscard]] virtual float GetPlayLength(const FVector& BlendParameters) const PURE_VIRTUAL(UMultiAnimAsset::GetPlayLength, return 0.f;);

#if WITH_EDITOR
	[[nodiscard]] virtual USkeletalMesh* GetPreviewMesh(const FName& Role) const PURE_VIRTUAL(UMultiAnimAsset::GetPreviewMesh, return nullptr;);
#endif // WITH_EDITOR

	[[nodiscard]] virtual int32 GetNumRoles() const PURE_VIRTUAL(UMultiAnimAsset::GetNumRoles, return 0;);
	[[nodiscard]] virtual FName GetRole(int32 RoleIndex) const PURE_VIRTUAL(UMultiAnimAsset::GetRole, return FName(););
	[[nodiscard]] virtual UAnimationAsset* GetAnimationAsset(const FName& Role) const PURE_VIRTUAL(UMultiAnimAsset::GetAnimationAsset, return nullptr;);
	[[nodiscard]] virtual FTransform GetOrigin(const FName& Role) const PURE_VIRTUAL(UMultiAnimAsset::GetOrigin, return FTransform::Identity;);

	// calculates the full aligned root bone transforms (FullAlignedActorRootBoneTransforms) at time "Time + TimeOffset" given the current actors root bone transforms (ActorRootBoneTransforms) at "Time"
	// the mirror data tables. DebugAnimContexts can be used to draw debug lines
	// For example if the actors are playing the associated GetAnimationAsset(Role) at time Time, by providing the ActorRootBoneTransforms and an TimeOffset of zero, CalculateWarpTransforms will return the
	// FullAlignedActorRootBoneTransforms should be NOW to be in full alignment. Any TimeOffset > 0 will have the method returning a future FullAlignedActorRootBoneTransforms. This is useful in synergy with
	// Motion Warping to provide a transform to use as full aligment at the ned of the notify state time
	virtual void CalculateWarpTransforms(float Time, float TimeOffset, const TConstArrayView<const FTransform> ActorRootBoneTransforms, TArrayView<FTransform> FullAlignedActorRootBoneTransforms, 
		const TConstArrayView<const UMirrorDataTable*> MirrorDataTables = TConstArrayView<const UMirrorDataTable*>(), const TConstArrayView<TObjectPtr<const UObject>> DebugAnimContexts = TConstArrayView<TObjectPtr<const UObject>>()) const 
		PURE_VIRTUAL(UMultiAnimAsset::CalculateWarpTransforms, );

	UFUNCTION(BlueprintPure, Category = "Animation", meta=(BlueprintThreadSafe, DisplayName = "Get Animation Asset"))
	UAnimationAsset* BP_GetAnimationAsset(const FName& Role) const { return GetAnimationAsset(Role); }

	UFUNCTION(BlueprintPure, Category = "Animation", meta=(BlueprintThreadSafe, DisplayName = "Get Origin"))
	FTransform BP_GetOrigin(const FName& Role) const { return GetOrigin(Role); }
};

#undef UE_API
