// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNodeMessages.h"

#define UE_API POSESEARCH_API

struct FAnimNode_PoseSearchHistoryCollector_Base;

namespace UE::PoseSearch
{

struct IPoseHistory;

class FPoseHistoryProvider : public UE::Anim::IGraphMessage
{
	DECLARE_ANIMGRAPH_MESSAGE_API(FPoseHistoryProvider, UE_API);

public:
	FPoseHistoryProvider(const FAnimNode_PoseSearchHistoryCollector_Base* InHistoryCollector) : HistoryCollector(InHistoryCollector) { check(HistoryCollector); }
	
	UE_DEPRECATED(5.8, "Use GetPoseHistoryPtr instead")
	UE_API const IPoseHistory& GetPoseHistory() const;

	UE_API const IPoseHistory* GetPoseHistoryPtr() const;
	
private:
	const FAnimNode_PoseSearchHistoryCollector_Base* HistoryCollector = nullptr;
};

} // namespace UE::PoseSearch

#undef UE_API
