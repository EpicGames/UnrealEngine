// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "BoneContainer.h"
#include "GameplayTagContainer.h"
#include "PoseSearchAnimNotifies.generated.h"

#define UE_API POSESEARCH_API

class UPoseSearchDatabase;

// Base class for ALL pose search anim notify states
UCLASS(MinimalAPI, Abstract)
class UAnimNotifyState_PoseSearchBase : public UAnimNotifyState
{
	GENERATED_BODY()
};

// Use this notify state to remove animation segments from the database completely, they will never play or return from
// a search result. PSD indexing will report errors in case the sampled animation asset contains conflicting UAnimNotifyState_PoseSearchSamplingRange(s)
UCLASS(MinimalAPI, Blueprintable, meta = (DisplayName = "Pose Search: Exclude From Database"))
class UAnimNotifyState_PoseSearchExcludeFromDatabase : public UAnimNotifyState_PoseSearchBase
{
	GENERATED_BODY()
};

// Use this notify state to include animation segments into the database, whatever excluded will never play or return from
// a search result. PSD indexing will report errors in case the sampled animation asset contains conflicting UAnimNotifyState_PoseSearchExcludeFromDatabase(s)
UCLASS(MinimalAPI, Experimental, Blueprintable, meta = (DisplayName = "Pose Search: Sampling Range"))
class UAnimNotifyState_PoseSearchSamplingRange : public UAnimNotifyState_PoseSearchBase
{
	GENERATED_BODY()
};

// A pose search search will not return results that overlap this notify, but the animation segment can still play
// if a previous search result advances into it.
UCLASS(MinimalAPI, Blueprintable, meta = (DisplayName = "Pose Search: Block Transition In"))
class UAnimNotifyState_PoseSearchBlockTransition : public UAnimNotifyState_PoseSearchBase
{
	GENERATED_BODY()
};

// Pose search cost will be affected by this, making the animation segment more or less likely to be selected based
// on the notify parameters
UCLASS(MinimalAPI, Blueprintable, meta = (DisplayName = "Pose Search: Override Base Cost Bias"))
// @todo: rename into UAnimNotifyState_PoseSearchOverrideBaseCostBias
class UAnimNotifyState_PoseSearchModifyCost : public UAnimNotifyState_PoseSearchBase
{
	GENERATED_BODY()

public:

	// A negative value reduces the cost and makes the segment more likely to be chosen. A positive value, conversely,
	// makes the segment less likely to be chosen
	UPROPERTY(EditAnywhere, Category = Config, meta = (DisplayName = "Modifier"))
	float CostAddend = -1.0f;
};

// Pose search cost for the continuing pose will be affected by this, making the animation segment more or less 
// likely to be continuing playing based on the notify parameters
UCLASS(MinimalAPI, Blueprintable, meta = (DisplayName = "Pose Search: Override Continuing Pose Cost Bias"))
class UAnimNotifyState_PoseSearchOverrideContinuingPoseCostBias : public UAnimNotifyState_PoseSearchBase
{
	GENERATED_BODY()

public:

	// A negative value reduces the cost and makes the segment more likely to continuing playing. A positive value, conversely,
	// makes the segment less likely to continuing playing
	UPROPERTY(EditAnywhere, Category = Config, meta = (DisplayName = "Modifier"))
	float CostAddend = -1.0f;
};

// Pose search cost for the continuing interaction (same anim contexts with the same assigned roles), making the animation segment more or less 
// likely to be continuing playing based on the notify parameters
UCLASS(MinimalAPI, Experimental, Blueprintable, meta = (DisplayName = "Pose Search: Override Continuing Interaction Cost Bias"))
class UAnimNotifyState_PoseSearchOverrideContinuingInteractionPoseCostBias : public UAnimNotifyState_PoseSearchBase
{
	GENERATED_BODY()

public:

	// A negative value reduces the cost and makes the segment more likely to continuing playing. A positive value, conversely,
	// makes the segment less likely to continuing playing
	UPROPERTY(EditAnywhere, Category = Config, meta = (DisplayName = "Modifier"))
	float CostAddend = -1.0f;
};

// Pose search cost for the continuing context interaction (same anim contexts, assigned roles may differ), making the animation segment more or less 
// likely to be continuing playing based on the notify parameters
UCLASS(MinimalAPI, Experimental, Blueprintable, meta = (DisplayName = "Pose Search: Override Continuing Context Interaction Cost Bias"))
class UAnimNotifyState_PoseSearchOverrideContinuingContextInteractionCostBias : public UAnimNotifyState_PoseSearchBase
{
	GENERATED_BODY()

public:

	// A negative value reduces the cost and makes selecting a new interaction less likely
	UPROPERTY(EditAnywhere, Category = Config, meta = (DisplayName = "Modifier"))
	float CostAddend = -1.0f;
};

// UPoseSearchFeatureChannel(s) can use this UAnimNotifyState_PoseSearchSamplingEvent to demarcate events identified by SamplingAttributeId
// during database indexing by specifying their SamplingAttributeId property to match UAnimNotifyState_PoseSearchSamplingAttribute::SamplingAttributeId
UCLASS(MinimalAPI, Blueprintable, meta = (DisplayName = "Pose Search: Sampling Event"))
class UAnimNotifyState_PoseSearchSamplingEvent : public UAnimNotifyState_PoseSearchBase
{
	GENERATED_BODY()

public:

#if WITH_EDITORONLY_DATA
	
	UPROPERTY(EditAnywhere, Category = Config, meta = (ClampMin=0))
	int32 SamplingAttributeId = 0;

#endif // WITH_EDITORONLY_DATA
};

// UPoseSearchFeatureChannel(s) can use this UAnimNotifyState_PoseSearchSamplingAttribute as animation space position, rotation, and linear velocity provider 
// during database indexing by specifying their SamplingAttributeId property to match UAnimNotifyState_PoseSearchSamplingAttribute::SamplingAttributeId
UCLASS(MinimalAPI, Blueprintable, meta = (DisplayName = "Pose Search: Sampling Attribute"))
class UAnimNotifyState_PoseSearchSamplingAttribute : public UAnimNotifyState_PoseSearchSamplingEvent
{
	GENERATED_BODY()

public:

#if WITH_EDITORONLY_DATA
	
	UPROPERTY(EditAnywhere, Category = Config)
	FBoneReference Bone;

	UPROPERTY(EditAnywhere, Category = Config)
	FVector Position = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, Category = Config)
	FQuat Rotation = FQuat::Identity;

	UPROPERTY(EditAnywhere, Category = Config)
	FVector LinearVelocity = FVector::ZeroVector;

#endif // WITH_EDITORONLY_DATA
};

UCLASS(MinimalAPI, Blueprintable, meta = (DisplayName = "Pose Search: Motion Matched Branch In"))
class UAnimNotifyState_PoseSearchBranchIn : public UAnimNotifyState_PoseSearchBase
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = Config)
	TObjectPtr<UPoseSearchDatabase> Database;

	UE_API uint32 GetBranchInId() const;
};

// multi character interaction constraint notify state: used to calculate relative bone positions in UPoseSearchInteractionAsset items
UCLASS(MinimalAPI, Experimental, Blueprintable, meta = (DisplayName = "Pose Search: Constraint"))
class UAnimNotifyState_PoseSearchConstraint : public UAnimNotifyState_PoseSearchBase
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Config)
	FName FromSocketName;

	UPROPERTY(EditAnywhere, Category = Config)
	FName FromSocketRole;
	
	UPROPERTY(EditAnywhere, Category = Config)
	FName ToSocketName;
	
	UPROPERTY(EditAnywhere, Category = Config)
	FName ToSocketRole;

	// who should translate to reach full alignment? FromSocket or ToSocket? If TranslationWeight is 0 FromSocket will move 100% and ToSocket 0%, if TranslationWeight is 1 FromSocket will move 0% and ToSocket 100%,
	UPROPERTY(EditAnywhere, Category = Config, meta = (ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1"))
	float TranslationWeight = 0.f;
	
	// who should rotate to reach full alignment? FromSocket or ToSocket? If RotationWeight is 0 FromSocket will move 100% and ToSocket 0%, if RotationWeight is 1 FromSocket will move 0% and ToSocket 100%,
	UPROPERTY(EditAnywhere, Category = Config, meta = (ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1"))
	float RotationWeight = 0.f;

	// ramp up time used to calculate the desired reach for this constraint 
	// (it doesn't start at the beginning of this notify state, BUT when this notify state is first seen, since MM can jump into a random frame of the animation containing the notify state)
	UPROPERTY(EditAnywhere, Category = Config, meta = (ClampMin = "0", UIMin = "0"))
	float RampUpTime = 0.2f;

	// cool down time used to calculate the desired reach for this constraint
	// (it doesn't start at the end of this notify state, BUT when this notify state gets out of scope
	// (ultimately getting out of scope when reaching the notify state end time minus CoolDownTime, since MM can jump into a random frame of the animation containing the notify state)
	UPROPERTY(EditAnywhere, Category = Config, meta = (ClampMin = "0", UIMin = "0"))
	float CoolDownTime = 0.2f;
};

// Base class for ALL pose search anim notifies
UCLASS(MinimalAPI, Abstract)
class UAnimNotify_PoseSearchBase : public UAnimNotify
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI, Experimental, Blueprintable, meta = (DisplayName = "Pose Search: Event"))
class UAnimNotify_PoseSearchEvent : public UAnimNotify_PoseSearchBase
{
	GENERATED_BODY()

public:
	// Tag identifying this event
	UPROPERTY(EditAnywhere, Category = Config)
	FGameplayTag EventTag;
};

#undef UE_API
