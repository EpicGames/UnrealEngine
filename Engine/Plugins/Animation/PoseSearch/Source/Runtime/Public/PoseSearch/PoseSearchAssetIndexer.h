// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Animation/AnimCurveTypes.h"
#include "BonePose.h"
#include "PoseSearch/PoseSearchIndex.h"
#include "PoseSearch/PoseSearchMirrorDataCache.h"
#include "PoseSearch/PoseSearchSchema.h"

struct FAnimNotifyContext;
class UAnimNotifyState_PoseSearchBase;
class UPoseSearchDatabase;

namespace UE::PoseSearch
{

struct FAnimationAssetSampler;
struct FSearchIndexAsset;
struct FPoseMetadata;

struct FAssetSamplingContext
{
	float BaseCostBias = 0.f;
	float LoopingCostBias = 0.f;

	FAssetSamplingContext(const UPoseSearchDatabase& Database);
};

struct FAnimationAssetSamplers
{
	void Reset();
	int32 Num() const;

	float GetPlayLength() const;
	bool IsLoopable() const;
	void ExtractAnimNotifyStates(float Time, FAnimNotifyContext& PreAllocatedNotifyContext, const TFunction<bool(UAnimNotifyState*)>& ProcessAnimNotifyState) const;
	bool ProcessAllAnimNotifyEvents(const TFunction<bool(TConstArrayView<FAnimNotifyEvent>)>& ProcessAnimNotifyEvents) const;
	const FString GetAssetName() const;

	FTransform ExtractRootTransform(float Time, int32 RoleIndex) const;
	FTransform GetTotalRootTransform(int32 RoleIndex) const;
	void ExtractPose(float Time, FCompactPose& OutPose, int32 RoleIndex) const;
	void ExtractPose(float Time, FCompactPose& OutPose, FBlendedCurve& OutCurve, int32 RoleIndex) const;
	FTransform MirrorTransform(const FTransform& InTransform, int32 RoleIndex) const;
	void MirrorPose(FCompactPose& Pose, int32 RoleIndex) const;

	TArray<const FAnimationAssetSampler*, TInlineAllocator<PreallocatedRolesNum>> AnimationAssetSamplers;
	TArray<const FMirrorDataCache*, TInlineAllocator<PreallocatedRolesNum>> MirrorDataCaches;
};

class FAssetIndexer
{
public:
	struct FStats
	{
		int32 NumAccumulatedSamples = 0;
		float AccumulatedSpeed = 0.f;
		float MaxSpeed = 0.f;
		float AccumulatedAcceleration = 0.f;
		float MaxAcceleration = 0.f;
	};

	POSESEARCH_API FAssetIndexer(const TConstArrayView<FBoneContainer> InBoneContainers, const FSearchIndexAsset& InSearchIndexAsset, const FAssetSamplingContext& InSamplingContext,
		const UPoseSearchSchema& InSchema, const FAnimationAssetSamplers& InAssetSamplers, const FRoleToIndex& InRoleToIndex, const FFloatInterval& InExtrapolationTimeInterval);
	POSESEARCH_API void AssignWorkingData(int32 InStartPoseIdx, TArrayView<float> InOutFeatureVectorTable, TArrayView<FPoseMetadata> InOutPoseMetadata);
	POSESEARCH_API void Process(int32 AssetIdx);
	const FStats& GetStats() const { return Stats; }
	// Experimental, this feature might be removed without warning, not for production use
	const FEventDataCollector& GetEventDataCollector() const { return EventDataCollector; }

	// Returns the value of float curve CurveName at time CalculateSampleTime(SampleIdx) + SampleTimeOffset.
	POSESEARCH_API bool GetSampleCurveValue(float& OutCurveValue, float SampleTimeOffset, int32 SampleIdx, const FName& CurveName, const FRole& SampleRole);

	// Returns OutSampleRotation as the rotation of the bone Schema.BoneReferences[SchemaSampleBoneIdx] at time CalculateSampleTime(SampleIdx) + SampleTimeOffset relative to the
	// transform of the bone Schema.BoneReferences[SchemaOriginBoneIdx] at time CalculateSampleTime(SampleIdx) + OriginTimeOffset 
	// Times will be processed by GetPermutationTimeOffsets(PermutationTimeType, ...)
	POSESEARCH_API bool GetSampleRotation(FQuat& OutSampleRotation, float SampleTimeOffset, float OriginTimeOffset, int32 SampleIdx, int8 SchemaSampleBoneIdx, int8 SchemaOriginBoneIdx, const FRole& SampleRole, const FRole& OriginRole, EPermutationTimeType PermutationTimeType = EPermutationTimeType::UseSampleTime, int32 SamplingAttributeId = INDEX_NONE);

	// Returns OutSamplePosition as the position of the bone Schema.BoneReferences[SchemaSampleBoneIdx] at time CalculateSampleTime(SampleIdx) + SampleTimeOffset relative to the
	// transform of the bone Schema.BoneReferences[SchemaOriginBoneIdx] at time CalculateSampleTime(SampleIdx) + OriginTimeOffset.
	// Times will be processed by GetPermutationTimeOffsets(PermutationTimeType, ...)
	POSESEARCH_API bool GetSamplePosition(FVector& OutSamplePosition, float SampleTimeOffset, float OriginTimeOffset, int32 SampleIdx, int8 SchemaSampleBoneIdx, int8 SchemaOriginBoneIdx, const FRole& SampleRole, const FRole& OriginRole, EPermutationTimeType PermutationTimeType = EPermutationTimeType::UseSampleTime, int32 SamplingAttributeId = INDEX_NONE);

	// Returns OutSampleVelocity as the delta velocity of the bone Schema.BoneReferences[SchemaSampleBoneIdx] velocity at time CalculateSampleTime(SampleIdx) + SampleTimeOffset minus
	// the velocity of the bone Schema.BoneReferences[SchemaOriginBoneIdx] at time CalculateSampleTime(SampleIdx) + OriginTimeOffset.
	// Times will be processed by GetPermutationTimeOffsets(PermutationTimeType, ...)
	// if bUseCharacterSpaceVelocities is true, velocities will be computed in root bone space, rather than animation (world) space
	POSESEARCH_API bool GetSampleVelocity(FVector& OutSampleVelocity, float SampleTimeOffset, float OriginTimeOffset, int32 SampleIdx, int8 SchemaSampleBoneIdx, int8 SchemaOriginBoneIdx, const FRole& SampleRole, const FRole& OriginRole, bool bUseCharacterSpaceVelocities = true, EPermutationTimeType PermutationTimeType = EPermutationTimeType::UseSampleTime, int32 SamplingAttributeId = INDEX_NONE);

	POSESEARCH_API bool ProcessAllAnimNotifyEvents(const TFunction<bool(TConstArrayView<FAnimNotifyEvent>)>& ProcessAnimNotifyEvents) const;
	POSESEARCH_API const FString GetAssetName() const;
	POSESEARCH_API float GetPlayLength() const;

	POSESEARCH_API int32 GetBeginSampleIdx() const;
	POSESEARCH_API int32 GetEndSampleIdx() const;
	POSESEARCH_API int32 GetNumIndexedPoses() const;
	
	POSESEARCH_API TArrayView<float> GetPoseVector(int32 SampleIdx) const;
	POSESEARCH_API const UPoseSearchSchema* GetSchema() const;
	POSESEARCH_API float CalculateSampleTime(int32 SampleIdx) const;
	bool IsProcessFailed() const { return bProcessFailed; }

#if WITH_EDITOR
	POSESEARCH_API float CalculatePermutationTimeOffset() const;
#endif //WITH_EDITOR

#if ENABLE_ANIM_DEBUG
	POSESEARCH_API void CompareCachedEntries(const FAssetIndexer& Other) const;
#endif // ENABLE_ANIM_DEBUG

private:
	int32 GetVectorIdx(int32 SampleIdx) const;

	// Returns the animation (world) space transform of the bone Schema.BoneReferences[SchemaBoneIdx] at time SampleTime
	FTransform GetTransform(float SampleTime, const FRole& Role, int8 SchemaBoneIdx = RootSchemaBoneIdx);
	FTransform GetTransform(float SampleTime, int32 RoleIndex, const FBoneReference& BoneReference);
	
	// Returns the value of float curve CurveName at time SampleTime
	float GetSampleCurveValueInternal(float SampleTime, const FName& CurveName, const FRole& Role);

	// Returns the component space transform of the bone Schema.BoneReferences[SchemaBoneIdx] at time SampleTime
	FTransform GetComponentSpaceTransform(float SampleTime, const FRole& Role, int8 SchemaBoneIdx = RootSchemaBoneIdx);

	// Returns OutSamplePosition as the displacement of the bone Schema.BoneReferences[SchemaSampleBoneIdx] position at time SampleTime relative to the
	// position of the bone Schema.BoneReferences[SchemaOriginBoneIdx] at time OriginTime in root bone space
	bool GetSamplePositionInternal(FVector& OutSamplePosition, float SampleTime, float OriginTime, int8 SchemaSampleBoneIdx, int8 SchemaOriginBoneIdx, const FRole& SampleRole, const FRole& OriginRole, int32 SamplingAttributeId);

	struct FCachedCSPose : public FCSPose<FCompactHeapPose>
	{
		void InitPose(const FCompactPose& SrcPose)
		{
			Pose.CopyBonesFrom(SrcPose);
			ComponentSpaceFlags.Empty(Pose.GetNumBones());
			ComponentSpaceFlags.AddZeroed(Pose.GetNumBones());
			ComponentSpaceFlags[0] = 1;
		}
	};

	struct FCachedEntry
	{
		float SampleTime = 0.f;

		// RootTransform and ComponentSpacePose are stored mirrored in case SearchIndexAsset.IsMirrored
		TArray<FTransform, TInlineAllocator<PreallocatedRolesNum>> RootTransform;
		TArray<FCachedCSPose, TInlineAllocator<PreallocatedRolesNum>> ComponentSpacePose;
		TArray<FBlendedHeapCurve, TInlineAllocator<PreallocatedRolesNum>> Curves;
	};

	FTransform MirrorTransform(const FTransform& Transform, int32 RoleIndex) const;
	FCachedEntry& GetEntry(float SampleTime);
	FTransform CalculateComponentSpaceTransform(FCachedEntry& Entry, const FBoneReference& BoneReference, int32 RoleIndex);

	void ComputeStats();

	TArray<FBoneContainer> BoneContainers;
	TMap<float, FCachedEntry> CachedEntries;
	const FSearchIndexAsset& SearchIndexAsset;
	const FAssetSamplingContext& SamplingContext;
	const UPoseSearchSchema& Schema;
	const FAnimationAssetSamplers AssetSamplers;

	// NoTe: mapping Role to the index of the associated asset that this FAssetIndexer is indexing.
	// NOT the index of the UPoseSearchSchema::Skeletons! Use UPoseSearchSchema::GetRoledSkeleton API to resolve that Role to FPoseSearchRoledSkeleton 
	const FRoleToIndex RoleToIndex;

	FFloatInterval ExtrapolationTimeInterval = FFloatInterval(-UE_BIG_NUMBER, UE_BIG_NUMBER);
	
	int32 StartPoseIdx = 0;
	
	TArrayView<float> FeatureVectorTable;
	TArrayView<FPoseMetadata> PoseMetadata;

	FEventDataCollector EventDataCollector;
	FStats Stats;

	bool bProcessFailed = false;
};

template <class TAnimNotifyState>
struct FPoseSearchTimedNotifies
{
	static_assert(TIsDerivedFrom<TAnimNotifyState, UAnimNotifyState_PoseSearchBase>::IsDerived, "The template class TAnimNotifyState must be a subclass of UAnimNotifyState_PoseSearchBase.");

	struct FItem
	{
		float Time = 0.f;
		const TAnimNotifyState* NotifyState = nullptr;
	};

	FPoseSearchTimedNotifies()
	{
	}

	FPoseSearchTimedNotifies(int32 SamplingAttributeId, FAssetIndexer& Indexer)
	{
		Initialize(SamplingAttributeId, Indexer);
	}

	void Initialize(int32 SamplingAttributeId, FAssetIndexer& Indexer)
	{
		Items.Reset();

		if (SamplingAttributeId >= 0)
		{
			Indexer.ProcessAllAnimNotifyEvents([SamplingAttributeId, this](const TConstArrayView<FAnimNotifyEvent> AnimNotifyEvents)
				{
					for (const FAnimNotifyEvent& AnimNotifyEvent : AnimNotifyEvents)
					{
						if (const TAnimNotifyState* SamplingEvent = Cast<TAnimNotifyState>(AnimNotifyEvent.NotifyStateClass))
						{
							if (SamplingEvent->SamplingAttributeId == SamplingAttributeId)
							{
								Items.Add({ AnimNotifyEvent.GetTime(), SamplingEvent });
							}
						}
					}

					return true;
				});
		
			Items.Sort([](const FItem& A, const FItem& B) { return A.Time < B.Time; });
		}

		CachedPlayLength = Items.IsEmpty() ? Indexer.GetPlayLength() : 0.f;
	}

	FItem GetClosestFutureEvent(float SampleTime) const
	{
		const int32 NumItems = Items.Num();
		if (NumItems == 0)
		{
			FItem Item;
			Item.Time = CachedPlayLength;
			return Item;
		}

		if (NumItems == 1)
		{
			return Items[0];
		}

		const int32 LowerBoundIdx = Algo::LowerBound(Items, SampleTime, [](const FItem& Item, float Value)
		{
			return Value > Item.Time;
		});

		const int32 ClampedLowerBoundIdx = FMath::Min(LowerBoundIdx, NumItems - 1);
		return Items[ClampedLowerBoundIdx];
	}

private:
	TArray<FItem, TInlineAllocator<128>> Items;
	float CachedPlayLength = 0.f;
};

} // namespace UE::PoseSearch

#endif // WITH_EDITOR
