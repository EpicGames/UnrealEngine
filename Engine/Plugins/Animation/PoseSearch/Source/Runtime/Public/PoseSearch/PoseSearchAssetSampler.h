// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BonePose.h"
#if WITH_EDITOR
#include "IO/IoHash.h"
#endif // WITH_EDITOR

struct FAnimNotifyContext;
struct FAnimExtractContext;
struct FAnimationPoseData;
class UAnimNotifyState_PoseSearchBase;
class UAnimationAsset;

namespace UE::PoseSearch
{
/**
 * Helper for sampling data from animation assets
 */
struct FAnimationAssetSampler
{
	enum { DefaultRootTransformSamplingRate = 30 };
	POSESEARCH_API FAnimationAssetSampler();
	POSESEARCH_API FAnimationAssetSampler(const UAnimationAsset* InAnimationAsset, const FTransform& InRootTransformOrigin = FTransform::Identity, const FVector& InBlendParameters = FVector::ZeroVector, int32 InRootTransformSamplingRate = DefaultRootTransformSamplingRate, bool bPreProcessRootTransform = true, bool bInEnforceCompressedDataSampling = true);
	POSESEARCH_API ~FAnimationAssetSampler();
	POSESEARCH_API void Init(const UAnimationAsset* InAnimationAsset, const FTransform& InRootTransformOrigin = FTransform::Identity, const FVector& InBlendParameters = FVector::ZeroVector, int32 InRootTransformSamplingRate = DefaultRootTransformSamplingRate, bool bPreProcessRootTransform = true, bool bInEnforceCompressedDataSampling = true);

	POSESEARCH_API bool IsInitialized() const;
	POSESEARCH_API float GetPlayLength() const;
	POSESEARCH_API float ToRealTime(float NormalizedTime) const;
	POSESEARCH_API float ToNormalizedTime(float RealTime) const;
	POSESEARCH_API bool IsLoopable() const;

	// Gets the final root transformation at the end of the asset's playback time
	POSESEARCH_API FTransform GetTotalRootTransform() const;

	UE_DEPRECATED(5.8, "This ExtractPose signatures will be made private. Use other signatures")
	POSESEARCH_API void ExtractPose(const FAnimExtractContext& ExtractionCtx, FAnimationPoseData& OutAnimPoseData) const;

	// Extracts pose for this asset at a given Time
	POSESEARCH_API void ExtractPose(float Time, FCompactPose& OutPose) const;
	POSESEARCH_API void ExtractPose(float Time, FCompactPose& OutPose, FBlendedCurve& OutCurve) const;

	// Extracts root transform at the given time, using the extremities of the sequence to extrapolate beyond the 
	// sequence limits when Time is less than zero or greater than the sequence length.
	POSESEARCH_API FTransform ExtractRootTransform(float Time) const;
	
	// Populates FAnimNotifyContext from the animation asset at Time
	POSESEARCH_API void ExtractAnimNotifyContext(float Time, FAnimNotifyContext& PreAllocatedNotifyContext) const;
	// Extracts notify states present in the animation asset at Time
	POSESEARCH_API void ExtractAnimNotifyStates(float Time, FAnimNotifyContext& PreAllocatedNotifyContext, const TFunction<bool(UAnimNotifyState*)>& ProcessAnimNotifyState) const;
	// Extracts notify events present in the animation asset at Time
	POSESEARCH_API void ExtractAnimNotifyEvents(float Time, FAnimNotifyContext& PreAllocatedNotifyContext, const TFunction<bool(const FAnimNotifyEvent*)>& ProcessAnimNotifyEvent) const;
	
	POSESEARCH_API TConstArrayView<FAnimNotifyEvent> GetAllAnimNotifyEvents() const;

	POSESEARCH_API const UAnimationAsset* GetAsset() const;

	POSESEARCH_API void Process();

	POSESEARCH_API static float GetPlayLength(const UAnimationAsset* AnimAsset, const FVector& BlendParameters = FVector::ZeroVector);
	POSESEARCH_API static float WrapOrClampTime(float Time, float PlayLength, bool bLoopable);

	void SetRootTransformOrigin(const FTransform& InRootTransformOrigin) { RootTransformOrigin = InRootTransformOrigin; }
	const FTransform& GetRootTransformOrigin() const { return RootTransformOrigin; }

protected:
	TWeakObjectPtr<const UAnimationAsset> AnimationAssetPtr;
	FTransform RootTransformOrigin = FTransform::Identity;

	// members used to sample blend spaces only!
	FVector BlendParameters = FVector::ZeroVector;
	int32 RootTransformSamplingRate = DefaultRootTransformSamplingRate;
	float CachedPlayLength = -1.f;
	TArray<FTransform> AccumulatedRootTransform;

	const float ExtrapolationSampleTime = 1.f / 30.f;
	const float ExtractionInterval = 1.0f / 120.0f;

	bool bEnforceCompressedDataSampling = true;
#if WITH_EDITOR	
	FIoHash PlatformHash;
#endif // WITH_EDITOR
};

} // namespace UE::PoseSearch

