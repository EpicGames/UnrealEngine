// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimCurveTypes.h"
#include "DrawDebugHelpers.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "PoseSearch/PoseSearchAssetSampler.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "VisualLogger/VisualLogger.h"
#include "PoseSearchAssetSamplerLibrary.generated.h"

#define UE_API POSESEARCH_API

USTRUCT(Experimental, BlueprintType, Category="Animation|Pose Search")
struct FPoseSearchAssetSamplerInput
{
	GENERATED_BODY()

	// Animation to sample
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	TObjectPtr<const UAnimationAsset> Animation;

	// Sampling time for Animation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	float AnimationTime = 0.f;

	// origin used to start sampling Animation at time of zero
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	FTransform RootTransformOrigin = FTransform::Identity;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	bool bMirrored = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	TObjectPtr<const UMirrorDataTable> MirrorDataTable;

	// blend parameters if Animation is a blend space
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	FVector BlendParameters = FVector::ZeroVector;
};

USTRUCT(Experimental, BlueprintType, Category="Animation|Pose Search")
struct FPoseSearchAssetSamplerPose
{
	GENERATED_BODY()

	FPoseSearchAssetSamplerPose() = default;

	FPoseSearchAssetSamplerPose(const FPoseSearchAssetSamplerPose& Other)
	: RootTransform(Other.RootTransform)
	, BoneContainer(Other.BoneContainer)
	, Pose(Other.Pose)
	, Curve(Other.Curve)
	, ComponentSpacePose(Other.ComponentSpacePose)
	{
		Pose.SetBoneContainer(&BoneContainer);
		const_cast<FCompactHeapPose&>(ComponentSpacePose.GetPose()).SetBoneContainer(&BoneContainer);
	}

	FPoseSearchAssetSamplerPose& operator=(const FPoseSearchAssetSamplerPose& Other)
	{
		if (this != &Other)
		{
			this->~FPoseSearchAssetSamplerPose();
			new(this) FPoseSearchAssetSamplerPose(Other);
		}
		return *this;
	}

	FTransform RootTransform = FTransform::Identity;
	FBoneContainer BoneContainer;
	FCompactHeapPose Pose;
	FBlendedHeapCurve Curve;
	// @todo: add Attribute(s)
	//FHeapAttributeContainer Attribute;

	FCSPose<FCompactHeapPose> ComponentSpacePose;
};

UENUM()
enum class EPoseSearchAssetSamplerSpace : uint8
{
	Local,
	Component,
	World
};

UCLASS(MinimalAPI, Experimental)
class UPoseSearchAssetSamplerLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintPure, Category = "Animation|Pose Search|Experimental", meta = (BlueprintThreadSafe, AdvancedDisplay = "AnimInstance"))
	static UE_API FPoseSearchAssetSamplerPose SamplePose(const FPoseSearchAssetSamplerInput Input, const UAnimInstance* AnimInstance = nullptr);

	static UE_API FPoseSearchAssetSamplerPose SamplePose(const FBoneContainer& BoneContainer, const FPoseSearchAssetSamplerInput Input);

	UFUNCTION(BlueprintPure, Category = "Animation|Pose Search|Experimental", meta = (BlueprintThreadSafe))
	static UE_API FTransform GetTransformByName(UPARAM(ref) FPoseSearchAssetSamplerPose& AssetSamplerPose, FName BoneName, EPoseSearchAssetSamplerSpace Space = EPoseSearchAssetSamplerSpace::World);

	static UE_API FTransform GetTransform(FPoseSearchAssetSamplerPose& AssetSamplerPose, FCompactPoseBoneIndex CompactPoseBoneIndex = FCompactPoseBoneIndex(INDEX_NONE), EPoseSearchAssetSamplerSpace Space = EPoseSearchAssetSamplerSpace::World);

	// @todo: it'd be nice if it was threadsafe...
	UFUNCTION(BlueprintCallable, Category="Animation|Pose Search|Experimental")
	static UE_API void Draw(const UAnimInstance* AnimInstance, UPARAM(ref) FPoseSearchAssetSamplerPose& AssetSamplerPose);

	template<typename FComponentSpacePoseType>
	static FTransform GetTransform(FComponentSpacePoseType& ComponentSpacePose, const FTransform& RootTransform, FCompactPoseBoneIndex CompactPoseBoneIndex = FCompactPoseBoneIndex(INDEX_NONE), EPoseSearchAssetSamplerSpace Space = EPoseSearchAssetSamplerSpace::World)
	{
		if (!ComponentSpacePose.GetPose().IsValid())
		{
			UE_LOGF(LogPoseSearch, Error, "UPoseSearchAssetSamplerLibrary::GetTransform invalid Pose");
			return FTransform::Identity;
		}

		if (CompactPoseBoneIndex.GetInt() == INDEX_NONE)
		{
			if (Space != EPoseSearchAssetSamplerSpace::World)
			{
				UE_LOGF(LogPoseSearch, Error, "UPoseSearchAssetSamplerLibrary::GetTransform invalid Space %ls to get the RootTransform. Expected space: %ls", *UEnum::GetDisplayValueAsText(Space).ToString(), *UEnum::GetDisplayValueAsText(EPoseSearchAssetSamplerSpace::World).ToString());
			}
			return RootTransform;
		}
	
		if (!ComponentSpacePose.GetPose().IsValidIndex(CompactPoseBoneIndex))
		{
			UE_LOGF(LogPoseSearch, Error, "UPoseSearchAssetSamplerLibrary::GetTransform invalid CompactPoseBoneIndex %d", CompactPoseBoneIndex.GetInt());
			return FTransform::Identity;
		}
	
		switch (Space)
		{
		case EPoseSearchAssetSamplerSpace::Local:
			return ComponentSpacePose.GetPose()[CompactPoseBoneIndex];

		case EPoseSearchAssetSamplerSpace::Component:
			return ComponentSpacePose.GetComponentSpaceTransform(CompactPoseBoneIndex);

		case EPoseSearchAssetSamplerSpace::World:
			return ComponentSpacePose.GetComponentSpaceTransform(CompactPoseBoneIndex) * RootTransform;
		}

		checkNoEntry();
		return FTransform::Identity;
	}

#if ENABLE_DRAW_DEBUG
	template<typename FComponentSpacePoseType>
	static void Draw(const UWorld* World, FComponentSpacePoseType& ComponentSpacePose, const FTransform& RootTransform, const FColor Color = FColor::Red, float DebugDrawSamplerRootAxisLength = 20.f, float DebugDrawSamplerSize = 6.f)
	{
		if (World)
		{
			if (DebugDrawSamplerRootAxisLength > 0.f)
			{
				const FTransform AxisWorldTransform = GetTransform(ComponentSpacePose, RootTransform);
				DrawDebugLine(World, AxisWorldTransform.GetTranslation(), AxisWorldTransform.GetTranslation() + AxisWorldTransform.GetScaledAxis(EAxis::X) * DebugDrawSamplerRootAxisLength, FColor::Red, false, 0.f, SDPG_Foreground);
				DrawDebugLine(World, AxisWorldTransform.GetTranslation(), AxisWorldTransform.GetTranslation() + AxisWorldTransform.GetScaledAxis(EAxis::Y) * DebugDrawSamplerRootAxisLength, FColor::Green, false, 0.f, SDPG_Foreground);
				DrawDebugLine(World, AxisWorldTransform.GetTranslation(), AxisWorldTransform.GetTranslation() + AxisWorldTransform.GetScaledAxis(EAxis::Z) * DebugDrawSamplerRootAxisLength, FColor::Blue, false, 0.f, SDPG_Foreground);
			}

			for (int32 BoneIndex = 0; BoneIndex < ComponentSpacePose.GetPose().GetNumBones(); ++BoneIndex)
			{
				const FTransform BoneWorldTransform = GetTransform(ComponentSpacePose, RootTransform, FCompactPoseBoneIndex(BoneIndex), EPoseSearchAssetSamplerSpace::World);
				DrawDebugPoint(World, BoneWorldTransform.GetTranslation(), DebugDrawSamplerSize, Color, false, 0.f, SDPG_Foreground);
			}
		}
	}
#endif // ENABLE_DRAW_DEBUG

#if ENABLE_VISUAL_LOG
	template<typename FComponentSpacePoseType>
	static void VLogDraw(const UObject* VLogContext, FComponentSpacePoseType& ComponentSpacePose, const FTransform& RootTransform, const TCHAR* VLogName, const FColor Color = FColor::Red, float DebugDrawSamplerRootAxisLength = 20.f)
	{
		check(IsInGameThread());

		check(VLogName);
		if (VLogContext)
		{
			if (DebugDrawSamplerRootAxisLength > 0.f)
			{
				const FTransform AxisWorldTransform = GetTransform(ComponentSpacePose, RootTransform);
				UE_VLOG_SEGMENT(VLogContext, VLogName, Display, AxisWorldTransform.GetTranslation(), AxisWorldTransform.GetTranslation() + AxisWorldTransform.GetScaledAxis(EAxis::X) * DebugDrawSamplerRootAxisLength, FColor::Red, TEXT(""));
				UE_VLOG_SEGMENT(VLogContext, VLogName, Display, AxisWorldTransform.GetTranslation(), AxisWorldTransform.GetTranslation() + AxisWorldTransform.GetScaledAxis(EAxis::Y) * DebugDrawSamplerRootAxisLength, FColor::Green, TEXT(""));
				UE_VLOG_SEGMENT(VLogContext, VLogName, Display, AxisWorldTransform.GetTranslation(), AxisWorldTransform.GetTranslation() + AxisWorldTransform.GetScaledAxis(EAxis::Z) * DebugDrawSamplerRootAxisLength, FColor::Blue, TEXT(""));
			}

			for (int32 BoneIndex = 0; BoneIndex < ComponentSpacePose.GetPose().GetNumBones(); ++BoneIndex)
			{
				const FCompactPoseBoneIndex CompactPoseBoneIndex(BoneIndex);
				const FCompactPoseBoneIndex ParentCompactPoseBoneIndex = ComponentSpacePose.GetPose().GetParentBoneIndex(CompactPoseBoneIndex);
				
				const FTransform BoneWorldTransform = GetTransform(ComponentSpacePose, RootTransform, CompactPoseBoneIndex, EPoseSearchAssetSamplerSpace::World);
				const FTransform ParentBoneWorldTransform = GetTransform(ComponentSpacePose, RootTransform, ParentCompactPoseBoneIndex, EPoseSearchAssetSamplerSpace::World);
				UE_VLOG_SEGMENT(VLogContext, VLogName, Display, BoneWorldTransform.GetTranslation(), ParentBoneWorldTransform.GetTranslation(), Color, TEXT(""));
			}
		}
	}

	static UE_API void VLogDraw(const UObject* VLogContext, const USkeletalMeshComponent* Mesh, const TCHAR* VLogName, const FColor Color = FColor::Red, float DebugDrawSamplerRootAxisLength = 20.f);

#endif // ENABLE_VISUAL_LOG
};

#undef UE_API
