// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchRole.h"
#include "PoseSearchConstraints.generated.h"

#define UE_API POSESEARCH_API

class UAnimNotifyState_PoseSearchConstraint;
class UMultiAnimAsset;

namespace UE::PoseSearch
{
	struct FSearchResult;
	struct FInteractionSearchContext;
}

USTRUCT(Experimental, BlueprintType, Category = "Animation|Pose Search")
struct FPoseSearchConstraint
{
	GENERATED_BODY()

	void Initialize(const UAnimNotifyState_PoseSearchConstraint* ConstraintNotifyState);
	const FTransform* GetSocketTransform(UE::PoseSearch::FRole Role, FName SocketName) const;

	template<typename T>
	bool HasTheSameRoledSockets(const T& Other) const
	{
		// same order of From and To properties
		if (FromSocketRole == Other.FromSocketRole && FromSocketName == Other.FromSocketName && ToSocketRole == Other.ToSocketRole && ToSocketName == Other.ToSocketName)
		{
			return true;
		}

		// opposite order of From and To properties
		if (FromSocketRole == Other.ToSocketRole && FromSocketName == Other.ToSocketName && ToSocketRole == Other.FromSocketRole && ToSocketName == Other.FromSocketName)
		{
			return true;
		}

		return false;
	}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	FName FromSocketName;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	FName FromSocketRole;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	FName ToSocketName;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	FName ToSocketRole;

	// who should translate to reach full alignment? FromBone or ToBone? If TranslationWeight is 0 FromBone will move 100% and ToBone 0%, if TranslationWeight is 1 FromBone will move 0% and ToBone 100%,
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	float TranslationWeight = 0.f;
	
	// who should rotate to reach full alignment? FromBone or ToBone? If RotationWeight is 0 FromBone will move 100% and ToBone 0%, if RotationWeight is 1 FromBone will move 0% and ToBone 100%,
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	float RotationWeight = 0.f;

	// ramp up time used to calculate the desired reach for this constraint 
	// (it doesn't start at the beginning of this notify state, BUT when this notify state is first seen, since MM can jump into a random frame of the animation containing the notify state)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	float RampUpTime = 0.2f;

	// cool down time used to calculate the desired reach for this constraint
	// (it doesn't start at the end of this notify state, BUT when this notify state gets out of scope
	// (ultimately getting out of scope when reaching the notify state end time minus CoolDownTime, since MM can jump into a random frame of the animation containing the notify state)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	float CoolDownTime = 0.2f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	FTransform FromSocketTransform;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	FTransform ToSocketTransform;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	float DesiredReach = 0.f;
};

#undef UE_API
