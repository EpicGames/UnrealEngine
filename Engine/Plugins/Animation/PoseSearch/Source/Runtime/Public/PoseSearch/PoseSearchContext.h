// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimInstance.h"
#include "DrawDebugHelpers.h"
#include "IObjectChooser.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "PoseSearch/PoseSearchEvent.h"
#include "PoseSearch/PoseSearchFeatureChannel.h"
#include "PoseSearch/PoseSearchIndex.h"
#include "PoseSearch/PoseSearchResult.h"

#define UE_API POSESEARCH_API

struct FTransformTrajectory;
class UPoseSearchDatabase;
class UPoseSearchFeatureChannel_Position;

namespace UE::PoseSearch
{

// usually used as set of asset to consider during a search of a database 
struct FStackAssetSet : public TSet<const UObject*, DefaultKeyFuncs<const UObject*>, TInlineSetAllocator<8, TMemStackSetAllocator<>>>
{
	bool ContainsAnyOf(TConstArrayView<const UObject*> Objects) const
	{
		for (const UObject* Object : Objects)
		{
			if (Contains(Object))
			{
				return true;
			}
		}
		return false;
	}
};

// Experimental, this feature might be removed without warning, not for production use
// usually used as set of asset indexes to consider (per database)
// Experimental, this feature might be removed without warning, not for production use
typedef TSet<int32, DefaultKeyFuncs<int32>, TInlineSetAllocator<8, TMemStackSetAllocator<>>> FStackAssetIndexes;
// usually used as container to map database to what asset indexes to consider
typedef TMap<const UPoseSearchDatabase*, FStackAssetIndexes, TInlineSetAllocator<8, TMemStackSetAllocator<>>> FStackDatabaseToAssetIndexes;

// Utils functions
const UObject* GetAnimContext(const FChooserEvaluationContext* Context);
// Experimental, this feature might be removed without warning, not for production use
UE_API const FTransform& GetContextTransform(const UObject* AnimContext, bool bCheckIsInGameThread = true);
const USkeleton* GetContextSkeleton(const UObject* AnimContext, bool bCheckIsInGameThread = true);
const AActor* GetContextOwningActor(const UObject* AnimContext, bool bCheckIsInGameThread = true);
FVector GetContextLocation(const UObject* AnimContext, bool bCheckIsInGameThread = true);

// Experimental, this feature might be removed without warning, not for production use
const USkeletalMeshComponent* GetContextSkeletalMeshComponent(const UObject* AnimContext, bool bCheckIsInGameThread = true);
// Experimental, this feature might be removed without warning, not for production use
const FBoneContainer GetBoneContainer(const UObject* AnimContext, bool bCheckIsInGameThread = true);

struct FPoseIndicesHistory;
struct IPoseHistory;

enum class EPoseCandidateFlags : uint32
{
	None = 0,

	Valid_Pose = 1 << 0,
	Valid_ContinuingPose = 1 << 1,
	Valid_CurrentPose = 1 << 2,

	AnyValidMask = Valid_Pose | Valid_ContinuingPose | Valid_CurrentPose,

	DiscardedBy_PoseJumpThresholdTime = 1 << 3,
	DiscardedBy_PoseReselectHistory = 1 << 4,
	DiscardedBy_BlockTransition = 1 << 5,
	DiscardedBy_PoseFilter = 1 << 6,
	DiscardedBy_AssetIdxFilter = 1 << 7,
	DiscardedBy_Search = 1 << 8,
	DiscardedBy_AssetReselection = 1 << 9,

	AnyDiscardedMask = DiscardedBy_PoseJumpThresholdTime | DiscardedBy_PoseReselectHistory | DiscardedBy_BlockTransition | DiscardedBy_PoseFilter | DiscardedBy_AssetIdxFilter | DiscardedBy_Search | DiscardedBy_AssetReselection,
};
ENUM_CLASS_FLAGS(EPoseCandidateFlags);

#if ENABLE_DRAW_DEBUG
struct FDebugDrawParams
{
	enum EDrawContext
	{
		DrawQuery,
		DrawCandidate
	};

	UE_API FDebugDrawParams(TConstArrayView<FChooserEvaluationContext*> InAnimContexts, TConstArrayView<const IPoseHistory*> InPoseHistories, const FRoleToIndex& InRoleToIndex, const UPoseSearchDatabase* InDatabase, EDrawContext InDrawContext = EDrawContext::DrawQuery);
	FDebugDrawParams() = default;

	UE_API FDebugDrawParams(const FDebugDrawParams& Other);
	UE_API FDebugDrawParams(FDebugDrawParams&& Other);
	UE_API FDebugDrawParams& operator=(const FDebugDrawParams& Other);
	UE_API FDebugDrawParams& operator=(FDebugDrawParams&& Other);

	UE_API void Init(TConstArrayView<FChooserEvaluationContext*> InAnimContexts, TConstArrayView<const IPoseHistory*> InPoseHistories, const FRoleToIndex& InRoleToIndex, const UPoseSearchDatabase* InDatabase, EDrawContext InDrawContext = EDrawContext::DrawQuery);

	UE_API const FSearchIndex* GetSearchIndex() const;
	UE_API const UPoseSearchSchema* GetSchema() const;

	UE_API float ExtractPermutationTime(TConstArrayView<float> PoseVector) const;
	UE_API FVector ExtractPosition(TConstArrayView<float> PoseVector, float SampleTimeOffset, int8 SchemaBoneIdx, const FRole& Role, EPermutationTimeType PermutationTimeType = EPermutationTimeType::UseSampleTime, int32 SamplingAttributeId = INDEX_NONE, float PermutationSampleTimeOffset = 0.f) const;
	UE_API FQuat ExtractRotation(TConstArrayView<float> PoseVector, float SampleTimeOffset, int8 SchemaBoneIdx, const FRole& Role, EPermutationTimeType PermutationTimeType = EPermutationTimeType::UseSampleTime, int32 SamplingAttributeId = INDEX_NONE, float PermutationSampleTimeOffset = 0.f) const;
	UE_API FTransform GetRootBoneTransform(const FRole& Role, float SampleTimeOffset = 0.f) const;

	UE_API void DrawLine(const FVector& LineStart, const FVector& LineEnd, const FColor& Color, float Thickness = 0.f) const;
	UE_API void DrawPoint(const FVector& Position, const FColor& Color, float Thickness = 6.f) const;
	UE_API void DrawCircle(const FVector& Center, const FVector& UpVector, float Radius, int32 Segments, const FColor& Color, float Thickness = 0.f) const;
	UE_API void DrawCircle(const FMatrix& TransformMatrix, float Radius, int32 Segments, const FColor& Color, float Thickness = 0.f) const;
	UE_API void DrawWedge(const FVector& Origin, const FVector& Direction, float InnerRadius, float OuterRadius, float WidthInDegrees, int32 Segments, const FColor& Color, float Thickness = 0.f) const;
	UE_API void DrawSphere(const FVector& Center, float Radius, int32 Segments, const FColor& Color, float Thickness = 0.f) const;
	UE_API void DrawCentripetalCatmullRomSpline(TConstArrayView<FVector> Points, TConstArrayView<FColor> Colors, float Alpha, int32 NumSamplesPerSegment, float Thickness = 1.f) const;
	
	UE_API void DrawFeatureVector(TConstArrayView<float> PoseVector);
	UE_API void DrawFeatureVector(int32 PoseIdx);

	UE_API bool IsAnyWeightRelevant(const UPoseSearchFeatureChannel* Channel) const;

	EDrawContext GetDrawContext() const { return DrawContext; }

private:
	bool CanDraw() const;

	TConstArrayView<FChooserEvaluationContext*> AnimContexts;
	TConstArrayView<const IPoseHistory*> PoseHistories;

	// NoTe: mapping Role to the index of the associated asset that this FDebugDrawParams is drawing.
	// NOT the index of the UPoseSearchSchema::Skeletons! Use UPoseSearchSchema::GetRoledSkeleton API to resolve that Role to FPoseSearchRoledSkeleton 
	FRoleToIndex RoleToIndex;

	const UPoseSearchDatabase* Database = nullptr;
	EDrawContext DrawContext = EDrawContext::DrawCandidate;

	TAlignedArray<float> DynamicWeightsSqrtBuffer;
	TConstArrayView<float> DynamicWeightsSqrt;
};

#endif // ENABLE_DRAW_DEBUG

// float buffer of features according to a UPoseSearchSchema layout. Used to build search queries at runtime
struct FCachedQuery
{
public:
	explicit FCachedQuery(const UPoseSearchSchema* InSchema);
	const UPoseSearchSchema* GetSchema() const { return Schema; }
	TArrayView<float> EditValues() { return Values; }
	TConstArrayView<float> GetValues() const { return Values; }

private:
	TStackAlignedArray<float> Values;
	
	// no need for a TWeakObjectPtr since it doesn't persist across multiple frames (same lifespan as FSearchContext)
	const UPoseSearchSchema* Schema;
};

// Experimental, this feature might be removed without warning, not for production use
struct FCachedPCAQuery
{
public:
	explicit FCachedPCAQuery(const UPoseSearchDatabase* InDatabase);
	const UPoseSearchDatabase* GetDatabase() const { return Database; }
	TArrayView<float> EditValues() { return Values; }
	TConstArrayView<float> GetValues() const { return Values; }

private:
	TStackAlignedArray<float> Values;
	
	// no need for a TWeakObjectPtr since it doesn't persist across multiple frames (same lifespan as FSearchContext)
	const UPoseSearchDatabase* Database;
};

// CachedChannels uses hashed unique identifiers to determine channels that can share feature vector data during the building of the query
struct FCachedChannel
{
	// no need for a TWeakObjectPtr since it doesn't persist across multiple frames (same lifespan as FSearchContext)
	const UPoseSearchFeatureChannel* Channel = nullptr;

	// index of the associated query in FSearchContext::CachedQueries
	int32 CachedQueryIndex = INDEX_NONE;
};

struct FSearchContext
{
	UE_DEPRECATED(5.7, "Use other signaure instead")
	UE_API FSearchContext(float InDesiredPermutationTimeOffset, const FPoseIndicesHistory* InPoseIndicesHistory,
		const FSearchResult& InCurrentResult, const FFloatInterval& InPoseJumpThresholdTime, const FPoseSearchEvent& InEventToSearch);

	UE_API FSearchContext(float InDesiredPermutationTimeOffset, const FFloatInterval& InPoseJumpThresholdTime, const FPoseSearchEvent& InEventToSearch);

	// deleting copies and moves since members could reference other members (e.g.: ContinuingPoseValues could point to ContinuingPoseValuesData, so it'll require proper copies and movers implementations)
	FSearchContext(const FSearchContext& Other) = delete;
	FSearchContext(FSearchContext&& Other) = delete;
	FSearchContext& operator=(const FSearchContext& Other) = delete;
	FSearchContext& operator=(FSearchContext&& Other) = delete;

	UE_API void AddRole(const FRole& Role, FChooserEvaluationContext* AnimContext, const IPoseHistory* PoseHistory);

	// Returns the curve value of name CurveName at an offset time of SampleTimeOffset.
	// If the curve is not found, assume value of 0, which is consistent with curve behavior in the animation update.
	UE_API float GetSampleCurveValue(float SampleTimeOffset, const FName& CurveName, const FRole& SampleRole);

	// Returns the rotation of the bone with index Schema.BoneReferences[SchemaSampleBoneIdx] at an offset time of SampleTimeOffset relative to the
	// transform of the bone with index Schema.BoneReferences[SchemaOriginBoneIdx] at an offset time of time OriginTimeOffset 
	// Times will be processed by GetPermutationTimeOffsets(PermutationTimeType, ...)
	UE_API FQuat GetSampleRotation(float SampleTimeOffset, float OriginTimeOffset, int8 SchemaSampleBoneIdx, int8 SchemaOriginBoneIdx, const FRole& SampleRole, const FRole& OriginRole, EPermutationTimeType PermutationTimeType = EPermutationTimeType::UseSampleTime, const FQuat* SampleBoneRotationWorldOverride = nullptr);
	
	// Returns the displacement of the bone with index Schema.BoneReferences[SchemaSampleBoneIdx] at an offset time of SampleTimeOffset relative to the
	// position of the bone with index Schema.BoneReferences[SchemaOriginBoneIdx] at an offset time of OriginTimeOffset in root bone space
	// Times will be processed by GetPermutationTimeOffsets(PermutationTimeType, ...)
	UE_API FVector GetSamplePosition(float SampleTimeOffset, float OriginTimeOffset, int8 SchemaSampleBoneIdx, int8 SchemaOriginBoneIdx, const FRole& SampleRole, const FRole& OriginRole, EPermutationTimeType PermutationTimeType = EPermutationTimeType::UseSampleTime, const FVector* SampleBonePositionWorldOverride = nullptr);
	
	// Returns the delta velocity of the velocity of the bone with index Schema.BoneReferences[SchemaSampleBoneIdx] at an offset time of SampleTimeOffset minus
	// the velocity of the bone with index Schema.BoneReferences[SchemaOriginBoneIdx] at an offset time of OriginTimeOffset 
	// Times will be processed by GetPermutationTimeOffsets(PermutationTimeType, ...)
	// if bUseCharacterSpaceVelocities is true, velocities will be computed in root bone space, rather than world space
	UE_API FVector GetSampleVelocity(float SampleTimeOffset, float OriginTimeOffset, int8 SchemaSampleBoneIdx, int8 SchemaOriginBoneIdx, const FRole& SampleRole, const FRole& OriginRole, bool bUseCharacterSpaceVelocities = true, EPermutationTimeType PermutationTimeType = EPermutationTimeType::UseSampleTime, const FVector* SampleBoneVelocityWorldOverride = nullptr);

	UE_DEPRECATED(5.7, "No longer necessary API")
	UE_API void ResetCurrentBestCost();

	UE_DEPRECATED(5.7, "No longer necessary API")
	UE_API void UpdateCurrentBestCost(const FPoseSearchCost& PoseSearchCost);

	UE_DEPRECATED(5.7, "No longer necessary API. Use the FSearchResult::PoseCost from UPoseSearchDatabase::Search if a search cost is needed")
	float GetCurrentBestTotalCost() const { return MAX_FLT; }

	UE_API TConstArrayView<float> GetCachedQuery(const UPoseSearchSchema* Schema) const;
	UE_API TConstArrayView<float> GetOrBuildQuery(const UPoseSearchSchema* Schema);
	// Experimental, this feature might be removed without warning, not for production use
	UE_API TConstArrayView<float> GetOrBuildPCAQuery(const UPoseSearchDatabase* Database);

	UE_DEPRECATED(5.7, "compare Database against GetContinuingPoseSearchResult().Database instead (the internal replacement IsContinuingPoseDatabase is not going to be dll exported)")
	UE_API bool IsCurrentResultFromDatabase(const UPoseSearchDatabase* Database) const;
	bool IsContinuingPoseDatabase(const UPoseSearchDatabase* Database) const;
	
	UE_DEPRECATED(5.7, "Use CanUseContinuingPoseValues instead")
	UE_API bool CanUseCurrentResult() const;
	UE_API bool CanUseContinuingPoseValues() const;

	UE_DEPRECATED(5.7, "Use GetContinuingPoseValues")
	TConstArrayView<float> GetCurrentResultPoseVector() const { return ContinuingPoseValues; }
	TConstArrayView<float> GetContinuingPoseValues() const { return ContinuingPoseValues; }

	UE_DEPRECATED(5.7, "Use UpdateContinuingPoseSearchResult to update the continuing pose search result AND values - previously referenced as 'CurrentResultPoseVector' ")
	UE_API void UpdateCurrentResultPoseVector();
	UE_API void UpdateContinuingPoseSearchResult(const FSearchResult& InContinuingPoseSearchResult, const FDatabasePoseIdx& InContinuingPoseValuesDatabasePoseIdx);
	
	UE_DEPRECATED(5.7, "Use GetContinuingPoseSearchResult")
	const FSearchResult& GetCurrentResult() const { return ContinuingPoseSearchResult; }
	const FSearchResult& GetContinuingPoseSearchResult() const { return ContinuingPoseSearchResult; }

	const FFloatInterval& GetPoseJumpThresholdTime() const { return PoseJumpThresholdTime; }
	
	UE_DEPRECATED(5.7, "use GetPoseHistory(Role)->GetPoseIndicesHistory() with the appropriate Role instead")
	const FPoseIndicesHistory* GetPoseIndicesHistory() const;
	
	UE_DEPRECATED(5.7, "ArePoseHistoriesValid is no longer necessary, since PoseHistories MUST be valid since it contains necessary information for MM to function like trajectory, historical poses, etc")
	UE_API bool ArePoseHistoriesValid() const;

	TConstArrayView<const IPoseHistory*> GetPoseHistories() const { return PoseHistories; }
	UE_API const IPoseHistory* GetPoseHistory(const FRole& Role) const;

	float GetDesiredPermutationTimeOffset() const { return DesiredPermutationTimeOffset; }
	
	UE_API const FChooserEvaluationContext* GetContext(const FRole& Role) const;
	UE_API FChooserEvaluationContext* GetContext(const FRole& Role);
	
	TArrayView<FChooserEvaluationContext*> GetContexts() { return AnimContexts; }

	const FRoleToIndex& GetRoleToIndex() const { return RoleToIndex; }

	UE_DEPRECATED(5.7, "Use SetAssetsToConsiderSet instead")
	void SetAssetsToConsider(TConstArrayView<const UObject*> InAssetsToConsider)
	{
		// you cannot call GetAssetsToConsiderSet / SetAssetsToConsiderSet with GetAssetsToConsider / SetAssetsToConsider
		check(AssetsToConsider == nullptr);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		AssetsToConsider_DEPRECATED = InAssetsToConsider;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.7, "Use GetAssetsToConsiderSet instead")
	TConstArrayView<const UObject*> GetAssetsToConsider() const
	{
		// you cannot call GetAssetsToConsiderSet / SetAssetsToConsiderSet with GetAssetsToConsider / SetAssetsToConsider
		check(AssetsToConsider == nullptr);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return AssetsToConsider_DEPRECATED;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.7, "Unnecessary once deprecated GetAssetsToConsiderSet / SetAssetsToConsiderSet have been removed")
	const TArray<const UObject*>& GetInternalDeprecatedAssetsToConsider() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return AssetsToConsider_DEPRECATED;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.7, "Unnecessary once deprecated GetAssetsToConsiderSet / SetAssetsToConsiderSet have been removed")
	void SetInternalDeprecatedAssetsToConsider(TArray<const UObject*>& InAssetsToConsider)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		AssetsToConsider_DEPRECATED = InAssetsToConsider;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetAssetsToConsiderSet(const FStackAssetSet* InAssetsToConsider)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// you cannot call GetAssetsToConsiderSet / SetAssetsToConsiderSet with GetAssetsToConsider / SetAssetsToConsider
		check(AssetsToConsider_DEPRECATED.IsEmpty());
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		AssetsToConsider = InAssetsToConsider;
	}

	const FStackAssetSet* GetAssetsToConsiderSet() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// you cannot call GetAssetsToConsiderSet / SetAssetsToConsiderSet with GetAssetsToConsider / SetAssetsToConsider
		check(AssetsToConsider_DEPRECATED.IsEmpty());
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return AssetsToConsider;
	}

	void SetAssetIndexesToConsiderPerDatabase(const FStackDatabaseToAssetIndexes* InAssetIndexesToConsiderPerDatabase)
	{
		AssetIndexesToConsiderPerDatabase = InAssetIndexesToConsiderPerDatabase;
	}

	const FStackDatabaseToAssetIndexes* GetAssetIndexesToConsiderPerDatabase() const
	{
		return AssetIndexesToConsiderPerDatabase;
	}
	
	// Experimental, this feature might be removed without warning, not for production use
	const FPoseSearchEvent& GetEventToSearch() const { return EventToSearch; }
	
	// returns the world space transform of the bone SchemaBoneIdx at time SampleTime
	UE_API FTransform GetWorldBoneTransformAtTime(float SampleTime, const FRole& SampleRole, int8 SchemaBoneIdx);

#if WITH_EDITOR
	void SetAsyncBuildIndexInProgress() { bAsyncBuildIndexInProgress = true; }
	void ResetAsyncBuildIndexInProgress() { bAsyncBuildIndexInProgress = false; }
	bool IsAsyncBuildIndexInProgress() const { return bAsyncBuildIndexInProgress; }
#endif // WITH_EDITOR

	bool AnyCachedQuery() const { return !CachedQueries.IsEmpty(); }
	void AddNewFeatureVectorBuilder(const UPoseSearchSchema* Schema) { CachedQueries.Emplace(Schema); }
	UE_API TArrayView<float> EditFeatureVector();
	
	UE_API const UPoseSearchFeatureChannel* GetCachedChannelData(uint32 ChannelUniqueIdentifier, const UPoseSearchFeatureChannel* Channel, TConstArrayView<float>& CachedChannelData);
	bool IsUseCachedChannelData() const { return bUseCachedChannelData; }
	void SetUseCachedChannelData(bool bInUseCachedChannelData) { bUseCachedChannelData = bInUseCachedChannelData; }
	
	// Experimental, this feature might be removed without warning, not for production use
	bool IsContinuingInteraction() const { return bIsContinuingInteraction; }
	// Experimental, this feature might be removed without warning, not for production use
	void SetContinuingInteraction(bool bInIsContinuingInteraction) { bIsContinuingInteraction = bInIsContinuingInteraction; }

	// Experimental, this feature might be removed without warning, not for production use
	bool IsContinuingContextInteraction() const { return bIsContinuingContextInteraction; }
	// Experimental, this feature might be removed without warning, not for production use
	void SetContinuingContextInteraction(bool bInIsContinuingContextInteraction) { bIsContinuingContextInteraction = bInIsContinuingContextInteraction; }

private:
	float GetSampleCurveValueInternal(float SampleTime, const FName& CurveName, const FRole& SampleRole);
	FVector GetSamplePositionInternal(float SampleTime, float OriginTime, int8 SchemaSampleBoneIdx, int8 SchemaOriginBoneIdx, const FRole& SampleRole, const FRole& OriginRole, const FVector* SampleBonePositionWorldOverride = nullptr);
	FQuat GetSampleRotationInternal(float SampleTime, float OriginTime, int8 SchemaSampleBoneIdx, int8 SchemaOriginBoneIdx, const FRole& SampleRole, const FRole& OriginRole, const FQuat* SampleBoneRotationWorldOverride = nullptr);
	FTransform GetWorldRootBoneTransformAtTime(float SampleTime, const FRole& SampleRole) const;
	void UpdateContinuingPoseValues();
	
	TArray<FChooserEvaluationContext*, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> AnimContexts;
	TArray<const IPoseHistory*, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> PoseHistories;
	FRoleToIndex RoleToIndex;
	
	// if AssetsToConsider is not empty, we'll search only for poses from UObject(s) that are in the AssetsToConsider
	const FStackAssetSet* AssetsToConsider = nullptr;
	
	const FStackDatabaseToAssetIndexes* AssetIndexesToConsiderPerDatabase = nullptr;

	UE_DEPRECATED(5.7, "Unnecessary once deprecated GetAssetsToConsiderSet / SetAssetsToConsiderSet have been removed")
	TArray<const UObject*> AssetsToConsider_DEPRECATED;
	
	const FPoseSearchEvent EventToSearch;

	const float DesiredPermutationTimeOffset = 0.f;
	
	// search result used to extract the continuing pose properties, except for gathering the continuing pose values, where we use ContinuingPoseValuesDatabasePoseIdx.
	// More details in ContinuingPoseValuesDatabasePoseIdx comments
	FSearchResult ContinuingPoseSearchResult;

	// database - pose index pair used to extract the continuing pose vector properties (values)
	// the reason it can differ from the ContinuingPoseSearchResult (database - pose index pair) is because we build and cache a query only using a schema, 
	// NOT a database (for performance reasons), BUT we want to be able to calculate the continuing pose from different databases sharing the same animation assets AND same schema,
	// and evaluate the continuing pose feature data across both of them. When this happens, the query gets created using the FIRST database pose value that could slightly differs, 
	// because of quantization of time, numerical errors, not common normalization sets, etc, BUT we still want to keep this logic consistent to the fact we created 
	// the query and cached it for the common schema, NOT per database, so we need to "lie" to the system by providing a ContinuingPoseSearchResult to gather continuing 
	// pose properties such as pose index, database etc, and rely on "ContinuingPoseValuesDatabasePoseIdx" to gather the continuing pose feature values vector (ContinuingPoseValues),
	// used in the channels as cached version of the data that could have been caluclated from the pose history
	FDatabasePoseIdx ContinuingPoseValuesDatabasePoseIdx;

	const FFloatInterval PoseJumpThresholdTime;
	bool bUseCachedChannelData = false;

	// if ContinuingPoseValuesDatabasePoseIdx is valid, this array view references the continuing pose values (feature vector data) 
	TConstArrayView<float> ContinuingPoseValues;
	// buffer optionally used in case the continuing pose needed to be reconstructed. ContinuingPoseValues will reference it
	TStackAlignedArray<float> ContinuingPoseValuesData;

	// transforms cached in world space
	TMap<uint32, FTransform, TInlineSetAllocator<64, TMemStackSetAllocator<>>> CachedTransforms;

	TArray<FCachedQuery, TInlineAllocator<PreallocatedCachedQueriesNum, TMemStackAllocator<>>> CachedQueries;
	TArray<FCachedPCAQuery, TInlineAllocator<PreallocatedCachedQueriesNum, TMemStackAllocator<>>> CachedPCAQueries;

	// mapping channel unique identifier (hash) to FCachedChannel
	TMap<uint32, FCachedChannel, TInlineSetAllocator<PreallocatedCachedChannelDataNum, TMemStackSetAllocator<>>> CachedChannels;

	// Experimental, this feature might be removed without warning, not for production use
	bool bIsContinuingInteraction = false;

	// Experimental, this feature might be removed without warning, not for production use
	bool bIsContinuingContextInteraction = false;

#if WITH_EDITOR
	bool bAsyncBuildIndexInProgress = false;
#endif // WITH_EDITOR

#if UE_POSE_SEARCH_TRACE_ENABLED

public:

	struct FPoseCandidate
	{
		FPoseCandidate(int32 InPoseIdx = 0, FPoseSearchCost InCost = FPoseSearchCost(), EPoseCandidateFlags InPoseCandidateFlags = EPoseCandidateFlags::None)
			: PoseIdx(InPoseIdx)
			, Cost(InCost)
			, PoseCandidateFlags(InPoseCandidateFlags)
		{
		}

		int32 PoseIdx = 0;
		FPoseSearchCost Cost;
		EPoseCandidateFlags PoseCandidateFlags = EPoseCandidateFlags::None;
	};

	struct FBestPoseCandidates
	{
		UE_API FBestPoseCandidates();
		UE_API void Add(int32 PoseIdx, EPoseCandidateFlags PoseCandidateFlags, const FPoseSearchCost& Cost);
		UE_API bool IterateOverBestPoseCandidates(const TFunctionRef<bool(const FPoseCandidate& PoseCandidate)> IterateOverBestPoseCandidatesFunction) const;

		UE_DEPRECATED(5.7, "Use IterateOverBestPoseCandidates to iterate over FBestPoseCandidates instead")
		UE_API int32 Num() const;
		UE_DEPRECATED(5.7, "Use IterateOverBestPoseCandidates to iterate over FBestPoseCandidates instead")
		UE_API FPoseCandidate GetUnsortedCandidate(int32 Index) const;
		
	private:

		struct FPoseCandidateHeapCompare
		{
			bool operator()(const FPoseCandidate& A, const FPoseCandidate& B) const
			{
				// using > to create a max heap where TransientPoseCandidates.HeapTop().Cost is the greatest Cost (to behave like the std::priority_queue)
				return A.Cost > B.Cost;
			}
		};

		// max heap of Valid_Pose(s). It could contains duplicate poses, that we filter out in IterateOverBestPoseCandidates!
		TArray<FPoseCandidate> TransientPoseCandidates;

		struct FPoseCandidatePoseIdxKeyFunc : BaseKeyFuncs<FPoseCandidate, FPoseCandidate, false>
		{
			static KeyInitType GetSetKey(ElementInitType& Element) { return Element; }
			static bool Matches(KeyInitType A, KeyInitType B) { return A.PoseIdx == B.PoseIdx; }
			static uint32 GetKeyHash(KeyInitType Key) { return GetTypeHash(Key.PoseIdx); }
		};

		// set of Valid_ContinuingPose(s) and Valid_CurrentPose(s). the set contains unique poses with the lowest cost and an union of their EPoseCandidateFlags
		TSet<FPoseCandidate, FPoseCandidatePoseIdxKeyFunc> PermanentPoseCandidates;
	};
	
	UE_API void Track(const UPoseSearchDatabase* Database, int32 PoseIdx = INDEX_NONE, EPoseCandidateFlags PoseCandidateFlags = EPoseCandidateFlags::None, const FPoseSearchCost& Cost = FPoseSearchCost());
	UE_API void Track(const UPoseSearchDatabase* Database, int32 PoseIdx, EPoseCandidateFlags PoseCandidateFlags, TConstArrayView<float> DynamicWeightsSqrt, float ContinuingPoseCostAddend, float ContinuingInteractionCostAddend, float ContinuingContextInteractionCostAddend);

	const TMap<const UPoseSearchDatabase*, FBestPoseCandidates>& GetBestPoseCandidatesMap() const
	{
		return BestPoseCandidatesMap;
	}

private:
	TMap<const UPoseSearchDatabase*, FBestPoseCandidates> BestPoseCandidatesMap;
#endif // UE_POSE_SEARCH_TRACE_ENABLED
};

} // namespace UE::PoseSearch

#undef UE_API
