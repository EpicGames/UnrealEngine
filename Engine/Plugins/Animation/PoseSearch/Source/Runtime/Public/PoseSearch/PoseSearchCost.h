// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearchCost.generated.h"

USTRUCT()
struct FPoseSearchCost
{
	GENERATED_BODY()
public:

	FPoseSearchCost() = default;
	~FPoseSearchCost() = default;
	FPoseSearchCost(const FPoseSearchCost&) = default;
	FPoseSearchCost(FPoseSearchCost&&) = default;
	FPoseSearchCost& operator=(const FPoseSearchCost&) = default;
	FPoseSearchCost& operator=(FPoseSearchCost&&) = default;

	FPoseSearchCost(float DissimilarityCost, float InNotifyCostAddend = 0.f, float InContinuingPoseCostAddend = 0.f, float InContinuingInteractionCostAddend = 0.f, float InContinuingContextInteractionCostAddend = 0.f)
		: TotalCost(DissimilarityCost + InNotifyCostAddend + InContinuingPoseCostAddend + InContinuingInteractionCostAddend + InContinuingContextInteractionCostAddend)
#if WITH_EDITORONLY_DATA
		, NotifyCostAddend(InNotifyCostAddend)
		, ContinuingPoseCostAddend(InContinuingPoseCostAddend)
		, ContinuingInteractionCostAddend(InContinuingInteractionCostAddend)
		, ContinuingContextInteractionCostAddend(InContinuingContextInteractionCostAddend)
#endif // WITH_EDITORONLY_DATA
	{
	}

	static bool IsCostValid(const float Cost) { return Cost != MAX_flt; }
	bool IsValid() const { return IsCostValid(TotalCost); }
	
	operator float() const { return TotalCost; }

#if WITH_EDITORONLY_DATA
	float GetCostAddend() const { return NotifyCostAddend + ContinuingPoseCostAddend + ContinuingInteractionCostAddend + ContinuingContextInteractionCostAddend; }

	float GetNotifyCostAddend() const { return NotifyCostAddend; }
	float GetContinuingPoseCostAddend() const { return ContinuingPoseCostAddend; }
	float GetContinuingInteractionCostAddend() const { return ContinuingInteractionCostAddend; }
	float GetContinuingContextInteractionCostAddend() const { return ContinuingContextInteractionCostAddend; }
#endif // WITH_EDITORONLY_DATA

	friend FArchive& operator<<(FArchive& Ar, FPoseSearchCost& PoseSearchCost)
	{
		Ar << PoseSearchCost.TotalCost;

		// this method is used by rewind debugger to archive data for pose search debugger and shared across game and editor.
		// when captures come from the game the cost addend breakdowns will be invalid and displaying incorrect information when previewed by the editor
		// @todo: should we make *CostAddend !WITH_EDITORONLY_DATA?
#if WITH_EDITORONLY_DATA
		
		Ar << PoseSearchCost.NotifyCostAddend;
		Ar << PoseSearchCost.ContinuingPoseCostAddend;
		Ar << PoseSearchCost.ContinuingInteractionCostAddend;
		Ar << PoseSearchCost.ContinuingContextInteractionCostAddend;

#else // WITH_EDITORONLY_DATA
		
		float NotifyCostAddend = 0.f;
		float ContinuingPoseCostAddend = 0.f;
		float ContinuingInteractionCostAddend = 0.f;
		float ContinuingContextInteractionCostAddend = 0.f;

		Ar << NotifyCostAddend;
		Ar << ContinuingPoseCostAddend;
		Ar << ContinuingInteractionCostAddend;
		Ar << ContinuingContextInteractionCostAddend;

#endif // WITH_EDITORONLY_DATA
			
		return Ar;
	}

protected:
	// TotalCost is the sum of all the Cost contributions (dissimilarity, notifies, continuing pose, continuing interaction, and continuing context interaction costs)
	UPROPERTY()
	float TotalCost = MAX_flt;

#if WITH_EDITORONLY_DATA

	// Notify Cost Bias contribution
	UPROPERTY()
	float NotifyCostAddend = 0.f;
	
	// Continuing Pose Cost Bias contribution
	UPROPERTY()
	float ContinuingPoseCostAddend = 0.f;

	// Experimental, this feature might be removed without warning, not for production use
	// Continuing Interaction Cost Bias contribution
	UPROPERTY()
	float ContinuingInteractionCostAddend = 0.f;

	// Experimental, this feature might be removed without warning, not for production use
	// Continuing Context Interaction Cost Bias contribution
	UPROPERTY()
	float ContinuingContextInteractionCostAddend = 0.f;

#endif // WITH_EDITORONLY_DATA
};

