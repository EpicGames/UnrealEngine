// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

#define UE_API POSESEARCH_API

// Custom serialization version for all packages containing PoseSearch dependent asset types
struct FPoseSearchCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,

		// Deprecated FPoseSearchQueryTrajectory and FPoseSearchQueryTrajectorySample. This version matches the RouteId_MotionMatchingState2 event.
		DeprecatedTrajectoryTypes,

		// Added InterruptMode to posesearch debugger. This version matches the RouteId_MotionMatchingState3 event.
		AddedInterruptModeToDebugger,

		// Strip out disabled Pose Search Database animations during cook
		StripOutDisabledDatabaseAnimationsDuringCook,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	UE_API const static FGuid GUID;

private:
	FPoseSearchCustomVersion() {}
};
 

#undef UE_API
