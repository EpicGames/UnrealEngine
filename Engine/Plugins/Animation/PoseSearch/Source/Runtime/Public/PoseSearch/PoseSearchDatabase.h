// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "PoseSearch/PoseSearchAssetSampler.h"
#include "PoseSearch/PoseSearchContext.h"
#include "StructUtils/InstancedStruct.h"
#include "PoseSearchDatabase.generated.h"

#define UE_API POSESEARCH_API

class UAnimationAsset;
class UAnimComposite;
class UAnimMontage;
class UBlendSpace;
class UMultiAnimAsset;

#if WITH_EDITORONLY_DATA
class UPoseSearchNormalizationSet;
#endif // WITH_EDITORONLY_DATA

namespace UE::PoseSearch
{
	UE_DEPRECATED(5.7, "UpgradeToDatabaseAnimationAsset will be removed once all the classes inheriting from FPoseSearchDatabaseAnimationAssetBase are removed")
	UE_API FPoseSearchDatabaseAnimationAsset UpgradeToDatabaseAnimationAsset(FInstancedStruct& AnimationAsset);
} // namespace UE::PoseSearch

UENUM()
enum class EPoseSearchMode : int32
{
	// Database searches will be evaluated extensively. the system will evaluate all the indexed poses to search for the best one.
	BruteForce,

	// Optimized search mode: the database projects the poses into a PCA space using only the most significant "NumberOfPrincipalComponents" dimensions, and construct a kdtree to facilitate the search.
	PCAKDTree,

	// Experimental, this feature might be removed without warning, not for production use
	// Optimized search mode using a vantage point tree
	VPTree UMETA(DisplayName = "VPTree (Experimental)"),

	// Experimental, this feature might be removed without warning, not for production use
	// search will only be performed on events.
	EventOnly UMETA(DisplayName = "EventOnly (Experimental)")
};

UENUM()
enum class EPoseSearchMirrorOption : int32
{
	UnmirroredOnly UMETA(DisplayName = "Original Only"),
	MirroredOnly UMETA(DisplayName = "Mirrored Only"),
	UnmirroredAndMirrored UMETA(DisplayName = "Original and Mirrored")
};

USTRUCT()
struct FPoseSearchDatabaseAnimationAssetBase
{
	GENERATED_BODY()
	virtual ~FPoseSearchDatabaseAnimationAssetBase() = default;
	virtual UObject* GetAnimationAsset() const { return nullptr; }
	
	UE_API virtual float GetPlayLength(const FVector& BlendParameters) const;

	virtual int32 GetNumRoles() const { return 1; }
	virtual UE::PoseSearch::FRole GetRole(int32 RoleIndex) const { check(GetNumRoles() == 1);  return UE::PoseSearch::DefaultRole; }
	UE_API virtual UAnimationAsset* GetAnimationAssetForRole(const UE::PoseSearch::FRole& Role) const;
	UE_API virtual FTransform GetRootTransformOriginForRole(const UE::PoseSearch::FRole& Role) const;
#if WITH_EDITOR
	UE_API virtual bool IsSkeletonCompatible(TObjectPtr<const UPoseSearchSchema> InSchema) const;
	// Experimental, this feature might be removed without warning, not for production use
	UE_API virtual bool IsMirrorOptionCompatible(TObjectPtr<const UPoseSearchSchema> InSchema) const;
	// Experimental, this feature might be removed without warning, not for production use
	UE_API virtual USkeletalMesh* GetPreviewMeshForRole(const UE::PoseSearch::FRole& Role) const;
	UE_API virtual void IterateOverSamplingParameter(const TFunction<void(const FVector& BlendParameters)>& ProcessSamplingParameter) const;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
    UE_API virtual bool UpdateFrom(const FPoseSearchDatabaseAnimationAssetBase& Source);
	virtual bool IsDisableReselection() const { return bDisableReselection; }
	virtual void SetDisableReselection(bool bValue) { bDisableReselection = bValue; }
	virtual UClass* GetAnimationAssetStaticClass() const { return nullptr; }
	virtual bool IsLooping() const { return false; }
	UE_API virtual const FString GetName() const;
	virtual bool IsEnabled() const { return bEnabled; }
	virtual void SetIsEnabled(bool bValue) { bEnabled = bValue; }
	virtual bool IsRootMotionEnabled() const { return false; }
	virtual EPoseSearchMirrorOption GetMirrorOption() const { return MirrorOption; }

	// [0, 0] represents the entire frame range of the original animation.
	virtual FFloatInterval GetSamplingRange() const { return FFloatInterval(0.f, 0.f); }
	virtual void SetSamplingRange(const FFloatInterval& NewRange) { }
	UE_API virtual FFloatInterval GetEffectiveSamplingRange(const FVector& BlendParameters) const;
	
	static UE_API FFloatInterval GetEffectiveSamplingRange(float PlayLength, const FFloatInterval& SamplingRange);

	UE_API virtual int64 GetEditorMemSize() const;
	virtual int64 GetApproxCookedSize() const { return GetEditorMemSize(); }

	bool IsSynchronizedWithExternalDependency() const { return BranchInId != 0; }

	// This allows users to enable or exclude animations from this database. Useful for debugging.
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (DisplayPriority = 1))
	bool bEnabled = true;

	// if bDisableReselection is true, poses from the same asset cannot be reselected. Useful to avoid jumping on frames on the same looping animations
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (DisplayPriority = 2))
	bool bDisableReselection = false;

	// This allows users to set if this animation is original only (no mirrored data), original and mirrored, or only the mirrored version of this animation.
	// It requires the mirror table to be set up in the database Schema.
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (DisplayPriority = 3))
	EPoseSearchMirrorOption MirrorOption = EPoseSearchMirrorOption::UnmirroredOnly;

	// BranchInId != 0 when this asset has been added via SynchronizeWithExternalDependencies.
	// To delete it, remove the PoseSearchBranchIn notify state
	UPROPERTY(VisibleAnywhere, Category = "Settings", meta = (DisplayPriority = 20))
	uint32 BranchInId = 0;
#endif // WITH_EDITORONLY_DATA

	friend bool operator==(const FPoseSearchDatabaseAnimationAssetBase& A, const FPoseSearchDatabaseAnimationAssetBase& B)
	{
#if WITH_EDITORONLY_DATA
		return A.bEnabled == B.bEnabled &&
			A.bDisableReselection == B.bDisableReselection &&
			A.MirrorOption == B.MirrorOption &&
			A.BranchInId == B.BranchInId;
#else // WITH_EDITORONLY_DATA
		return true;
#endif // WITH_EDITORONLY_DATA
	}
};

template<> struct TStructOpsTypeTraits<FPoseSearchDatabaseAnimationAssetBase> : public TStructOpsTypeTraitsBase2<FPoseSearchDatabaseAnimationAssetBase>
{
	enum { WithIdenticalViaEquality = true };
};

USTRUCT(BlueprintType, Category = "Animation|Pose Search")
struct FPoseSearchDatabaseAnimationAsset : public FPoseSearchDatabaseAnimationAssetBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (DisplayPriority = 0))
	TObjectPtr<UObject> AnimAsset;

#if WITH_EDITORONLY_DATA

	// If true this BlendSpace will output a single segment in the database.
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (DisplayPriority = 4))
	bool bUseSingleSample = false;

	// When turned on, this will use the set grid samples of the blend space asset for sampling. This will override the Number of Horizontal/Vertical Samples.
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "!bUseSingleSample", EditConditionHides, DisplayPriority = 5))
	bool bUseGridForSampling = false;

	// Sets the number of horizontal samples in the blend space to pull the animation data coverage from. The larger the samples the more the data, but also the more memory and performance it takes.
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "!bUseSingleSample && !bUseGridForSampling", EditConditionHides, ClampMin = "1", UIMin = "1", UIMax = "25", DisplayPriority = 6))
	int32 NumberOfHorizontalSamples = 9;
	
	// Sets the number of vertical samples in the blend space to pull the animation data coverage from.The larger the samples the more the data, but also the more memory and performance it takes.
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "!bUseSingleSample && !bUseGridForSampling", EditConditionHides, ClampMin = "1", UIMin = "1", UIMax = "25", DisplayPriority = 7))
	int32 NumberOfVerticalSamples = 2;

	// BlendParams used to sample this BlendSpace
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "bUseSingleSample", EditConditionHides, DisplayPriority = 8))
	float BlendParamX = 0.f;

	// BlendParams used to sample this BlendSpace
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "bUseSingleSample", EditConditionHides, DisplayPriority = 9))
	float BlendParamY = 0.f;

	// It allows users to set a time range to an individual blend space in the database.
	// This is effectively trimming the beginning and end of the animation in the database (not in the original blend space).
	// If set to [0, 0] it will be the entire frame range of the original blend space.
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampToMinMaxLimits, DisplayPriority = 2))
	FFloatInterval SamplingRange = FFloatInterval(0.f, 0.f);

	UE_API virtual UClass* GetAnimationAssetStaticClass() const override;
	UE_API static bool IsLooping(const UObject* AnimAsset, const FFloatInterval& SamplingRange);
	UE_API virtual bool IsLooping() const override;

	UE_API static bool IsRootMotionEnabled(const UObject* AnimAsset);
	UE_API virtual bool IsRootMotionEnabled() const override;

	virtual FFloatInterval GetSamplingRange() const override { return SamplingRange; }
	UE_API virtual FFloatInterval GetEffectiveSamplingRange(const FVector& BlendParameters) const override;
	UE_API virtual void IterateOverSamplingParameter(const TFunction<void(const FVector& BlendParameters)>& ProcessSamplingParameter) const override;
	virtual void SetSamplingRange(const FFloatInterval& NewRange) override { SamplingRange = NewRange; }
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	UE_API USkeletalMesh* GetPreviewMeshForRole(const UE::PoseSearch::FRole& Role) const;
#endif // WITH_EDITOR

	UE_API virtual UObject* GetAnimationAsset() const override;
	UE_API virtual float GetPlayLength(const FVector& BlendParameters) const override;

	UE_API virtual int32 GetNumRoles() const override;
	UE_API virtual UE::PoseSearch::FRole GetRole(int32 RoleIndex) const override;
	UE_API virtual UAnimationAsset* GetAnimationAssetForRole(const UE::PoseSearch::FRole& Role) const override;
	UE_API virtual FTransform GetRootTransformOriginForRole(const UE::PoseSearch::FRole& Role) const override;

	friend bool operator==(const FPoseSearchDatabaseAnimationAsset& A, const FPoseSearchDatabaseAnimationAsset& B)
	{
		return static_cast<const FPoseSearchDatabaseAnimationAssetBase&>(A) == static_cast<const FPoseSearchDatabaseAnimationAssetBase&>(B)
			&& A.AnimAsset == B.AnimAsset
#if WITH_EDITORONLY_DATA
			&& A.bUseSingleSample == B.bUseSingleSample
			&& A.bUseGridForSampling == B.bUseGridForSampling
			&& A.NumberOfHorizontalSamples == B.NumberOfHorizontalSamples
			&& A.NumberOfVerticalSamples == B.NumberOfVerticalSamples
			&& A.BlendParamX == B.BlendParamX
			&& A.BlendParamY == B.BlendParamY
			&& A.SamplingRange == B.SamplingRange
#endif // WITH_EDITORONLY_DATA
			;
	}
};

template<> struct TStructOpsTypeTraits<FPoseSearchDatabaseAnimationAsset> : public TStructOpsTypeTraitsBase2<FPoseSearchDatabaseAnimationAsset>
{
	enum { WithIdenticalViaEquality = true };
};


// A sequence entry in a UPoseSearchDatabase.
USTRUCT(BlueprintType, Category = "Animation|Pose Search")
struct UE_DEPRECATED(5.7, "Use FPoseSearchDatabaseAnimationAsset instead. To convert your struct to support the new APIs use ") FPoseSearchDatabaseSequence : public FPoseSearchDatabaseAnimationAssetBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (DisplayPriority = 0))
	TObjectPtr<UAnimSequence> Sequence;

#if WITH_EDITORONLY_DATA
	// It allows users to set a time range to an individual animation sequence in the database. 
	// This is effectively trimming the beginning and end of the animation in the database (not in the original sequence).
	// If set to [0, 0] it will be the entire frame range of the original sequence.
	// Set to readonly if this asset is synchronized via PoseSearchBranchIn notify state.
	// To edit its value update the associated PoseSearchBranchIn in Sequence
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition="BranchInId == 0", ClampToMinMaxLimits, DisplayPriority = 2))
	FFloatInterval SamplingRange = FFloatInterval(0.f, 0.f);

	UE_API virtual UClass* GetAnimationAssetStaticClass() const override;
	UE_API virtual bool IsLooping() const override;
	UE_API virtual bool IsRootMotionEnabled() const override;
	virtual FFloatInterval GetSamplingRange() const override { return SamplingRange; }
	virtual void SetSamplingRange(const FFloatInterval& NewRange) override { SamplingRange = NewRange; }
#endif // WITH_EDITORONLY_DATA
	
	UE_API virtual UObject* GetAnimationAsset() const override;

	friend bool operator==(const FPoseSearchDatabaseSequence& A, const FPoseSearchDatabaseSequence& B)
	{
		return static_cast<const FPoseSearchDatabaseAnimationAssetBase&>(A) == static_cast<const FPoseSearchDatabaseAnimationAssetBase&>(B)
			&& A.Sequence == B.Sequence
#if WITH_EDITORONLY_DATA
			&& A.SamplingRange == B.SamplingRange
#endif // WITH_EDITORONLY_DATA
			;
	}
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
template<> struct TStructOpsTypeTraits<FPoseSearchDatabaseSequence> : public TStructOpsTypeTraitsBase2<FPoseSearchDatabaseSequence>
{
	enum { WithIdenticalViaEquality = true };
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

// An blend space entry in a UPoseSearchDatabase.
USTRUCT(BlueprintType, Category = "Animation|Pose Search")
struct UE_DEPRECATED(5.7, "Use FPoseSearchDatabaseAnimationAsset instead. To convert your struct to support the new APIs use ") FPoseSearchDatabaseBlendSpace : public FPoseSearchDatabaseAnimationAssetBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (DisplayPriority = 0))
	TObjectPtr<UBlendSpace> BlendSpace;

#if WITH_EDITORONLY_DATA

	// If true this BlendSpace will output a single segment in the database.
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (DisplayPriority = 4))
	bool bUseSingleSample = false;

	// When turned on, this will use the set grid samples of the blend space asset for sampling. This will override the Number of Horizontal/Vertical Samples.
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "!bUseSingleSample", EditConditionHides, DisplayPriority = 5))
	bool bUseGridForSampling = false;

	// Sets the number of horizontal samples in the blend space to pull the animation data coverage from. The larger the samples the more the data, but also the more memory and performance it takes.
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "!bUseSingleSample && !bUseGridForSampling", EditConditionHides, ClampMin = "1", UIMin = "1", UIMax = "25", DisplayPriority = 6))
	int32 NumberOfHorizontalSamples = 9;
	
	// Sets the number of vertical samples in the blend space to pull the animation data coverage from.The larger the samples the more the data, but also the more memory and performance it takes.
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "!bUseSingleSample && !bUseGridForSampling", EditConditionHides, ClampMin = "1", UIMin = "1", UIMax = "25", DisplayPriority = 7))
	int32 NumberOfVerticalSamples = 2;

	// BlendParams used to sample this BlendSpace
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "bUseSingleSample", EditConditionHides, DisplayPriority = 8))
	float BlendParamX = 0.f;

	// BlendParams used to sample this BlendSpace
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "bUseSingleSample", EditConditionHides, DisplayPriority = 9))
	float BlendParamY = 0.f;

	// It allows users to set a time range to an individual blend space in the database.
	// This is effectively trimming the beginning and end of the animation in the database (not in the original blend space).
	// If set to [0, 0] it will be the entire frame range of the original blend space.
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampToMinMaxLimits, DisplayPriority = 2))
	FFloatInterval SamplingRange = FFloatInterval(0.f, 0.f);

	UE_API virtual UClass* GetAnimationAssetStaticClass() const override;
	UE_API virtual bool IsLooping() const override;
	UE_API virtual bool IsRootMotionEnabled() const override;
	virtual FFloatInterval GetSamplingRange() const override { return SamplingRange; }
	UE_API virtual FFloatInterval GetEffectiveSamplingRange(const FVector& BlendParameters) const override;
	UE_API virtual void IterateOverSamplingParameter(const TFunction<void(const FVector& BlendParameters)>& ProcessSamplingParameter) const override;
	virtual void SetSamplingRange(const FFloatInterval& NewRange) override { SamplingRange = NewRange; }
#endif // WITH_EDITORONLY_DATA

	UE_API virtual UObject* GetAnimationAsset() const override;
	UE_API virtual float GetPlayLength(const FVector& BlendParameters) const override;

	friend bool operator==(const FPoseSearchDatabaseBlendSpace& A, const FPoseSearchDatabaseBlendSpace& B)
	{
		return static_cast<const FPoseSearchDatabaseAnimationAssetBase&>(A) == static_cast<const FPoseSearchDatabaseAnimationAssetBase&>(B)
			&& A.BlendSpace == B.BlendSpace
#if WITH_EDITORONLY_DATA
			&& A.bUseSingleSample == B.bUseSingleSample
			&& A.bUseGridForSampling == B.bUseGridForSampling
			&& A.NumberOfHorizontalSamples == B.NumberOfHorizontalSamples
			&& A.NumberOfVerticalSamples == B.NumberOfVerticalSamples
			&& A.BlendParamX == B.BlendParamX
			&& A.BlendParamY == B.BlendParamY
			&& A.SamplingRange == B.SamplingRange
#endif // WITH_EDITORONLY_DATA
			;
	}
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
template<> struct TStructOpsTypeTraits<FPoseSearchDatabaseBlendSpace> : public TStructOpsTypeTraitsBase2<FPoseSearchDatabaseBlendSpace>
{
	enum { WithIdenticalViaEquality = true };
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

// An entry in a UPoseSearchDatabase.
USTRUCT(BlueprintType, Category = "Animation|Pose Search")
struct UE_DEPRECATED(5.7, "Use FPoseSearchDatabaseAnimationAsset instead. To convert your struct to support the new APIs use ") FPoseSearchDatabaseAnimComposite : public FPoseSearchDatabaseAnimationAssetBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (DisplayPriority = 0))
	TObjectPtr<UAnimComposite> AnimComposite;

#if WITH_EDITORONLY_DATA
	// It allows users to set a time range to an individual animation composite in the database. 
	// This is effectively trimming the beginning and end of the animation in the database (not in the original composite).
	// If set to [0, 0] it will be the entire frame range of the original composite.
	// Set to readonly if this asset is synchronized via PoseSearchBranchIn notify state.
	// To edit its value update the associated PoseSearchBranchIn in AnimComposite
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition="BranchInId == 0", ClampToMinMaxLimits, DisplayPriority = 3))
	FFloatInterval SamplingRange = FFloatInterval(0.f, 0.f);

	UE_API virtual UClass* GetAnimationAssetStaticClass() const override;
	UE_API virtual bool IsLooping() const override;
	UE_API virtual bool IsRootMotionEnabled() const override;
	virtual FFloatInterval GetSamplingRange() const override { return SamplingRange; }
	virtual void SetSamplingRange(const FFloatInterval& NewRange) override { SamplingRange = NewRange; }
#endif // WITH_EDITORONLY_DATA

	UE_API virtual UObject* GetAnimationAsset() const override;

	friend bool operator==(const FPoseSearchDatabaseAnimComposite& A, const FPoseSearchDatabaseAnimComposite& B)
	{
		return static_cast<const FPoseSearchDatabaseAnimationAssetBase&>(A) == static_cast<const FPoseSearchDatabaseAnimationAssetBase&>(B)
			&& A.AnimComposite == B.AnimComposite
#if WITH_EDITORONLY_DATA
			&& A.SamplingRange == B.SamplingRange
#endif // WITH_EDITORONLY_DATA
			;
	}
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
template<> struct TStructOpsTypeTraits<FPoseSearchDatabaseAnimComposite> : public TStructOpsTypeTraitsBase2<FPoseSearchDatabaseAnimComposite>
{
	enum { WithIdenticalViaEquality = true };
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

// An anim montage entry in a UPoseSearchDatabase.
USTRUCT(BlueprintType, Category = "Animation|Pose Search")
struct UE_DEPRECATED(5.7, "Use FPoseSearchDatabaseAnimationAsset instead. To convert your struct to support the new APIs use ") FPoseSearchDatabaseAnimMontage : public FPoseSearchDatabaseAnimationAssetBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (DisplayPriority = 0))
	TObjectPtr<UAnimMontage> AnimMontage;

#if WITH_EDITORONLY_DATA
	// It allows users to set a time range to an individual animation montage in the database. 
	// This is effectively trimming the beginning and end of the animation in the database (not in the original montage).
	// If set to [0, 0] it will be the entire frame range of the original montage.
	// Set to readonly if this asset is synchronized via PoseSearchBranchIn notify state.
	// To edit its value update the associated PoseSearchBranchIn in AnimMontage
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition="BranchInId == 0", ClampToMinMaxLimits, DisplayPriority = 2))
	FFloatInterval SamplingRange = FFloatInterval(0.f, 0.f);

	UE_API virtual UClass* GetAnimationAssetStaticClass() const override;
	UE_API virtual bool IsLooping() const override;
	UE_API virtual bool IsRootMotionEnabled() const override;
	virtual FFloatInterval GetSamplingRange() const override { return SamplingRange; }
	virtual void SetSamplingRange(const FFloatInterval& NewRange) override { SamplingRange = NewRange; }
#endif // WITH_EDITORONLY_DATA

	UE_API virtual UObject* GetAnimationAsset() const override;

	friend bool operator==(const FPoseSearchDatabaseAnimMontage& A, const FPoseSearchDatabaseAnimMontage& B)
	{
		return static_cast<const FPoseSearchDatabaseAnimationAssetBase&>(A) == static_cast<const FPoseSearchDatabaseAnimationAssetBase&>(B)
			&& A.AnimMontage == B.AnimMontage
#if WITH_EDITORONLY_DATA
			&& A.SamplingRange == B.SamplingRange
#endif // WITH_EDITORONLY_DATA
			;
	}
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
template<> struct TStructOpsTypeTraits<FPoseSearchDatabaseAnimMontage> : public TStructOpsTypeTraitsBase2<FPoseSearchDatabaseAnimMontage>
{
	enum { WithIdenticalViaEquality = true };
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

USTRUCT(Experimental, BlueprintType, Category = "Animation|Pose Search")
struct UE_DEPRECATED(5.7, "Use FPoseSearchDatabaseAnimationAsset instead. To convert your struct to support the new APIs use ") FPoseSearchDatabaseMultiAnimAsset : public FPoseSearchDatabaseAnimationAssetBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (DisplayPriority = 0))
	TObjectPtr<UMultiAnimAsset> MultiAnimAsset;

#if WITH_EDITORONLY_DATA
	// Sets the number of horizontal samples in the blend spaces referenced by the UMultiAnimAsset to pull the animation data coverage from. The larger the samples the more the data, but also the more memory and performance it takes.
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "1", UIMin = "1", UIMax = "25", DisplayPriority = 6))
	int32 NumberOfHorizontalSamples = 1;
	
	// Sets the number of vertical samples for the blend spaces referenced by the UMultiAnimAsset to pull the animation data coverage from. The larger the samples the more the data, but also the more memory and performance it takes.
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "1", UIMin = "1", UIMax = "25", DisplayPriority = 7))
	int32 NumberOfVerticalSamples = 1;

	// It allows users to set a time range to an individual UMultiAnimAsset in the database. 
	// This is effectively trimming the beginning and end of the animation in the database (not in the original UMultiAnimAsset).
	// If set to [0, 0] it will be the entire frame range of the original UMultiAnimAsset.
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampToMinMaxLimits, DisplayPriority = 2))
	FFloatInterval SamplingRange = FFloatInterval(0.f, 0.f);

	UE_API virtual UClass* GetAnimationAssetStaticClass() const override;
	UE_API virtual bool IsLooping() const override;
	UE_API virtual bool IsRootMotionEnabled() const override;
	virtual FFloatInterval GetSamplingRange() const override { return SamplingRange; }
	UE_API virtual void IterateOverSamplingParameter(const TFunction<void(const FVector& BlendParameters)>& ProcessSamplingParameter) const override;
	virtual void SetSamplingRange(const FFloatInterval& NewRange) override { SamplingRange = NewRange; }
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	UE_API USkeletalMesh* GetPreviewMeshForRole(const UE::PoseSearch::FRole& Role) const;
#endif // WITH_EDITOR

	UE_API virtual UObject* GetAnimationAsset() const override;
	UE_API virtual float GetPlayLength(const FVector& BlendParameters) const override;
	UE_API virtual int32 GetNumRoles() const override;
	UE_API virtual UE::PoseSearch::FRole GetRole(int32 RoleIndex) const override;
	UE_API virtual UAnimationAsset* GetAnimationAssetForRole(const UE::PoseSearch::FRole& Role) const override;
	UE_API virtual FTransform GetRootTransformOriginForRole(const UE::PoseSearch::FRole& Role) const override;

	friend bool operator==(const FPoseSearchDatabaseMultiAnimAsset& A, const FPoseSearchDatabaseMultiAnimAsset& B)
	{
		return static_cast<const FPoseSearchDatabaseAnimationAssetBase&>(A) == static_cast<const FPoseSearchDatabaseAnimationAssetBase&>(B)
			&& A.MultiAnimAsset == B.MultiAnimAsset
#if WITH_EDITORONLY_DATA
			&& A.NumberOfHorizontalSamples == B.NumberOfHorizontalSamples
			&& A.NumberOfVerticalSamples == B.NumberOfVerticalSamples
			&& A.SamplingRange == B.SamplingRange
#endif // WITH_EDITORONLY_DATA
			;
	}
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
template<> struct TStructOpsTypeTraits<FPoseSearchDatabaseMultiAnimAsset> : public TStructOpsTypeTraitsBase2<FPoseSearchDatabaseMultiAnimAsset>
{
	enum { WithIdenticalViaEquality = true };
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

// A data asset for indexing a collection of animation sequences.
UCLASS(MinimalAPI, BlueprintType, Category = "Animation|Pose Search", meta = (DisplayName = "Pose Search Database"))
class UPoseSearchDatabase : public UDataAsset
{
	GENERATED_BODY()
public:

	// The Schema sets what channels this database will use to match against (bones, trajectory and what properties of those you’re interested in, such as position and velocity).
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Database")
	TObjectPtr<const UPoseSearchSchema> Schema;

	// Cost added to the continuing pose from this database. This allows users to apply a cost bias (positive or negative) to the continuing pose.
	// This is useful to help the system stay in one animation segment longer, or shorter depending on how you set this bias.
	// Negative values make it more likely to be picked, or stayed in, positive values make it less likely to be picked or stay in.
	// Note: excluded from DDC hash, since used only at runtime in SearchContinuingPose
	UPROPERTY(EditAnywhere, Category = "Database", meta = (ExcludeFromHash)) 
	float ContinuingPoseCostBias = -0.01f;

	// Base Cost added or removed to all poses from this database. It can be overridden by Anim Notify: Pose Search Modify Cost at the frame level of animation data.
	// Negative values make it more likely to be picked, or stayed in, Positive values make it less likely to be picked or stay in.
	UPROPERTY(EditAnywhere, Category = "Database")
	float BaseCostBias = 0.f;

	// Cost added to all looping animation assets in this database. This allows users to make it more or less likely to pick the looping animation segments.
	// Negative values make it more likely to be picked, or stayed in, Positive values make it less likely to be picked or stay in.
	UPROPERTY(EditAnywhere, Category = "Database")
	float LoopingCostBias = -0.005f;

	// Cost added to poses from this database (if setup with UMultiAnimAsset assets in "interaction mode") as continuation of a previous interaction (same anim contexts with the same assigned roles)
	// This allows users to apply a cost bias (positive or negative) to poses following up an interaction.
	// This is useful to help the system stay in one animation segment longer, or shorter depending on how you set this bias.
	// Negative values make it more likely to be picked, or stayed in, positive values make it less likely to be picked or stay in.
	// Note: excluded from DDC hash, since used only at runtime
	// Experimental, this feature might be removed without warning, not for production use
	UPROPERTY(EditAnywhere, Category = "Database", meta = (ExcludeFromHash))
	float ContinuingInteractionCostBias = 0.f;

	// behaving like ContinuingInteractionCostBias, but for continuing interactions with the same anim contexts (e.g. same actors)
	// Experimental, this feature might be removed without warning, not for production use
	UPROPERTY(EditAnywhere, Category = "Database", meta = (ExcludeFromHash))
	float ContinuingContextInteractionCostBias = 0.f;
	
#if WITH_EDITORONLY_DATA
	// These settings allow users to trim the start and end of animations in the database to preserve start/end frames for blending, and prevent the system from selecting the very last frames before it blends out.
	// valid animation frames will be AnimationAssetTimeStart + ExcludeFromDatabaseParameters.Min, AnimationAssetTimeEnd + ExcludeFromDatabaseParameters.Max
	UPROPERTY(EditAnywhere, Category = "Database", meta = (AllowInvertedInterval))
	FFloatInterval ExcludeFromDatabaseParameters = FFloatInterval(0.f, -0.3f);

	// extrapolation of animation assets will be clamped by AnimationAssetTimeStart + AdditionalExtrapolationTime.Min, AnimationAssetTimeEnd + AdditionalExtrapolationTime.Max
	UPROPERTY(EditAnywhere, Category = "Database", meta = (AllowInvertedInterval))
	FFloatInterval AdditionalExtrapolationTime = FFloatInterval(-100.f, 100.f);
#endif // WITH_EDITORONLY_DATA

private:
	UE_DEPRECATED(5.7, "Please reference DatabaseAnimationAssets instead")
	UPROPERTY()
	TArray<FInstancedStruct> AnimationAssets_DEPRECATED;

	UPROPERTY()
	TArray<FPoseSearchDatabaseAnimationAsset> DatabaseAnimationAssets;


public:
	/** Array of tags that can be used as metadata. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Database")
	TArray<FName> Tags;

#if WITH_EDITORONLY_DATA
	// This optional asset defines a list of databases you want to normalize together. Without it, it would be difficult to compare costs from separately normalized databases containing different types of animation,
	// like only idles versus only runs animations, given that the range of movement would be dramatically different.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Database")
	TObjectPtr<const UPoseSearchNormalizationSet> NormalizationSet;

	// If null, the default preview mesh for the skeleton will be used. Otherwise, this will be used in preview scenes.
	// @todo: Move this to be a setting in the Pose Search Database editor. 
	UPROPERTY(EditAnywhere, Category = "Preview", meta = (ExcludeFromHash))
	TObjectPtr<USkeletalMesh> PreviewMesh = nullptr;
#endif // WITH_EDITORONLY_DATA

	// This dictates how the database will perform the search.
	UPROPERTY(EditAnywhere, Category = "Performance")
	EPoseSearchMode PoseSearchMode = EPoseSearchMode::PCAKDTree;

#if WITH_EDITORONLY_DATA
	// Number of dimensions used to create the kdtree. More dimensions allows a better explanation of the variance of the dataset that usually translates in better search results, but will imply more memory usage and worse performances.
	UPROPERTY(EditAnywhere, Category = "Performance", meta = (EditCondition = "PoseSearchMode == EPoseSearchMode::PCAKDTree", EditConditionHides, ClampMin = "1", ClampMax = "64", UIMin = "1", UIMax = "64"))
	int32 NumberOfPrincipalComponents = 4;

	UPROPERTY(EditAnywhere, Category = "Performance", meta = (EditCondition = "PoseSearchMode == EPoseSearchMode::PCAKDTree", EditConditionHides, ClampMin = "1", ClampMax = "256", UIMin = "1", UIMax = "256"))
	int32 KDTreeMaxLeafSize = 16;
#endif // WITH_EDITORONLY_DATA
	
	// @todo: rename to KNNQueryNumNeighbors to be usable with the VPTree as well
	// Out of a kdtree search, results will have only an approximate cost, so the database search will select the best “KDTree Query Num Neighbors” poses to perform the full cost analysis, and be able to elect the best pose.
	// Memory & Performance Optimization! If KDTreeQueryNumNeighbors is 1 all the SearchIndexPrivate::Values will be stripped away, and the search will exclusively rely on the KDTree query result from the PCA space encoded values (SearchIndexPrivate::PCAValues).
	UPROPERTY(EditAnywhere, Category = "Performance", meta = (DisplayName = "KNNQueryNumNeighbors", EditCondition = "PoseSearchMode == EPoseSearchMode::PCAKDTree || PoseSearchMode == EPoseSearchMode::VPTree", EditConditionHides, ClampMin = "1", ClampMax = "600", UIMin = "1"))
	int32 KDTreeQueryNumNeighbors = 200;

#if WITH_EDITORONLY_DATA
	// if two poses values (multi dimensional point with the schema cardinality) are closer than PosePruningSimilarityThreshold,
	// only one will be saved into the database FSearchIndexBase (to save memory) and accessed by the two different pose indexes
	UPROPERTY(EditAnywhere, Category = "Performance")
	float PosePruningSimilarityThreshold = 0.f;

	// if two PCA values (multi dimensional point with the GetNumberOfPrincipalComponents cardinality) are closer than PCAValuesPruningSimilarityThreshold,
	// only one will be saved into the database FSearchIndex (to save memory).
	UPROPERTY(EditAnywhere, Category = "Performance", meta = (EditCondition = "PoseSearchMode == EPoseSearchMode::PCAKDTree", EditConditionHides))
	float PCAValuesPruningSimilarityThreshold = 0.f;
#endif // WITH_EDITORONLY_DATA

	// @todo: rename to KNNQueryNumNeighborsWithDuplicates to be usable with the VPTree as well
	// if PCAValuesPruningSimilarityThreshold > 0 the kdtree will remove duplicates, every result out of the KDTreeQueryNumNeighbors could potentially references multiple poses.
	// KDTreeQueryNumNeighborsWithDuplicates is the upper bound number of poses the system will perform the full cost evaluation. if KDTreeQueryNumNeighborsWithDuplicates is zero then there's no upper bound
	UPROPERTY(EditAnywhere, Category = "Performance", meta = (DisplayName = "KNNQueryNumNeighborsWithDuplicates", EditCondition = "PoseSearchMode == EPoseSearchMode::PCAKDTree && PCAValuesPruningSimilarityThreshold > 0", EditConditionHides, ClampMin = "0", ClampMax = "600", UIMin = "1"))
	int32 KDTreeQueryNumNeighborsWithDuplicates = 0;
	
private:
	// Do not use it directly. Use GetSearchIndex / SetSearchIndex interact with it and validate that is ok to do so.
	UE::PoseSearch::FSearchIndex SearchIndexPrivate;
	
	// CachedAssetMap is NOT serialized in operator<< but recalculated by UpdateCachedProperties every time SearchIndexPrivate changes
	// accessing the key const UObject* is protected by FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex so it doesn't have to be TWeakObjectPtr<const UObject> / FObjectKey for GC protection
	typedef TMap<const UObject*, TArray<int32>> FCachedAssetMap;
	typedef TPair<const UObject*, TArray<int32>> FCachedAssetMapPair;
	FCachedAssetMap CachedAssetMap;

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE(FOnDerivedDataRebuildMulticaster);
	FOnDerivedDataRebuildMulticaster OnDerivedDataRebuild;

	DECLARE_MULTICAST_DELEGATE(FOnSynchronizeWithExternalDependenciesMulticaster);
	FOnSynchronizeWithExternalDependenciesMulticaster OnSynchronizeWithExternalDependencies;
#endif // WITH_EDITOR

public:
	UE_API virtual ~UPoseSearchDatabase();

	UE_API void SetSearchIndex(const UE::PoseSearch::FSearchIndex& SearchIndex);
	UE_API const UE::PoseSearch::FSearchIndex& GetSearchIndex() const;
	
	UE_DEPRECATED(5.8, "no longer necessary API")
	UE_API bool GetSkipSearchIfPossible() const;

	// given an AnimationAsset, a normalized time AnimationAssetTime (in [0-1] range for blend spaces), and mirror state bMirrored
	// will retrieve the associated pose index from this database closest in distance to BlendParameters.
	// It'll return INDEX_NONE if it cannot find a pose
	UE_API int32 GetPoseIndex(const UObject* AnimationAsset, float AnimationAssetTime, bool bMirrored, const FVector& BlendParameters, const UE::PoseSearch::FStackAssetIndexes* AssetIndexesToConsider = nullptr) const;

#if WITH_EDITOR
	UE_DEPRECATED(5.7, "Use AddAnimationAsset(const FPoseSearchDatabaseAnimationAsset& AnimationAsset) instead")
	UE_API void AddAnimationAsset(FInstancedStruct AnimationAsset);

	UE_API void AddAnimationAsset(const FPoseSearchDatabaseAnimationAsset& AnimationAsset);
	// Experimental, this feature might be removed without warning, not for production use
	UE_API void InsertAnimationAssetAt(const FPoseSearchDatabaseAnimationAsset& AnimationAsset, int32 AnimationAssetIndex);
	UE_API void RemoveAnimationAssetAt(int32 AnimationAssetIndex);

	UE_API void SetAnimationAssetAt(const FPoseSearchDatabaseAnimationAsset& AnimationAsset, int32 AnimationAssetIndex);
#endif // WITH_EDITOR

	UE_API const FPoseSearchDatabaseAnimationAsset* GetDatabaseAnimationAsset(int32 AnimationAssetIndex) const;

	UE_DEPRECATED(5.8, "Use GetDatabaseAnimationAsset(SearchIndexAsset.GetSourceAssetIdx()) instead")
	UE_API const FPoseSearchDatabaseAnimationAsset* GetDatabaseAnimationAsset(const UE::PoseSearch::FSearchIndexAsset& SearchIndexAsset) const;

	UE_DEPRECATED(5.8, "Use SetAnimationAssetAt(AnimationAsset, AnimationAssetIndex) instead")
	UE_API FPoseSearchDatabaseAnimationAsset* GetMutableDatabaseAnimationAsset(int32 AnimationAssetIndex);
	UE_DEPRECATED(5.8, "Use SetAnimationAssetAt(AnimationAsset, SearchIndexAsset.GetSourceAssetIdx()) instead")
	UE_API FPoseSearchDatabaseAnimationAsset* GetMutableDatabaseAnimationAsset(const UE::PoseSearch::FSearchIndexAsset& SearchIndexAsset);

	template<typename TDatabaseAnimationAsset> UE_DEPRECATED(5.7, "Use the non templated version of this API") const TDatabaseAnimationAsset* GetDatabaseAnimationAsset(int32 AnimationAssetIndex) const;
	template<typename TDatabaseAnimationAsset> UE_DEPRECATED(5.7, "Use the non templated version of this API") const TDatabaseAnimationAsset* GetDatabaseAnimationAsset(const UE::PoseSearch::FSearchIndexAsset& SearchIndexAsset) const;
	template<typename TDatabaseAnimationAsset> UE_DEPRECATED(5.7, "Use the non templated version of this API") TDatabaseAnimationAsset* GetMutableDatabaseAnimationAsset(int32 AnimationAssetIndex);
	template<typename TDatabaseAnimationAsset> UE_DEPRECATED(5.7, "Use the non templated version of this API") TDatabaseAnimationAsset* GetMutableDatabaseAnimationAsset(const UE::PoseSearch::FSearchIndexAsset& SearchIndexAsset);

	UE_API float GetRealAssetTime(int32 PoseIdx) const;
	UE_API float GetNormalizedAssetTime(int32 PoseIdx) const;

	// Begin UObject
	UE_API virtual void PostLoad() override;
	UE_API virtual void PreSaveRoot(FObjectPreSaveRootContext ObjectSaveContext) override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	// End UObject
	
	// deprecate in favor of Search(UE::PoseSearch::FSearchContext& SearchContext, UE::PoseSearch::FSearchResults& SearchResults) once we settle on FSearchResults API signatures
	UE_API UE::PoseSearch::FSearchResult Search(UE::PoseSearch::FSearchContext& SearchContext) const;
	// deprecate in favor of SearchContinuingPose(UE::PoseSearch::FSearchContext& SearchContext, UE::PoseSearch::FSearchResults& SearchResults) once we settle on FSearchResults API signatures
	UE_API UE::PoseSearch::FSearchResult SearchContinuingPose(UE::PoseSearch::FSearchContext& SearchContext) const;
	
	// Experimental, this feature might be removed without warning, not for production use
	UE_API void Search(UE::PoseSearch::FSearchContext& SearchContext, UE::PoseSearch::FSearchResults& SearchResults) const;
	// Experimental, this feature might be removed without warning, not for production use
	UE_API void SearchContinuingPose(UE::PoseSearch::FSearchContext& SearchContext, UE::PoseSearch::FSearchResults& SearchResults) const;

	UE_API bool Contains(const UObject* Object) const;

	UFUNCTION(BlueprintPure, Category = "Animation|Pose Search|Experimental", meta = (BlueprintThreadSafe))
	UE_API int32 GetNumAnimationAssets() const;

	UFUNCTION(BlueprintPure, Category = "Animation|Pose Search|Experimental", meta = (BlueprintThreadSafe))
	UE_API UObject* GetAnimationAsset(int32 Index) const;

#if WITH_EDITOR
	UE_API int32 GetNumberOfPrincipalComponents() const;

	UE_API virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	UE_API virtual bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;

	typedef FOnDerivedDataRebuildMulticaster::FDelegate FOnDerivedDataRebuild;
	void RegisterOnDerivedDataRebuild(const FOnDerivedDataRebuild& Delegate) { OnDerivedDataRebuild.Add(Delegate); }
	void UnregisterOnDerivedDataRebuild(FDelegateUserObject Unregister) { OnDerivedDataRebuild.RemoveAll(Unregister); }
	void NotifyDerivedDataRebuild() const { OnDerivedDataRebuild.Broadcast(); }

	typedef FOnSynchronizeWithExternalDependenciesMulticaster::FDelegate FOnSynchronizeWithExternalDependencies;
	void RegisterOnSynchronizeWithExternalDependencies(const FOnSynchronizeWithExternalDependencies& Delegate) { OnSynchronizeWithExternalDependencies.Add(Delegate); }
	void UnregisterOnSynchronizeWithExternalDependencies(FDelegateUserObject Unregister) { OnSynchronizeWithExternalDependencies.RemoveAll(Unregister); }
	void NotifySynchronizeWithExternalDependencies() const { OnSynchronizeWithExternalDependencies.Broadcast(); }

	UE_API void SynchronizeWithExternalDependencies();
	UE_API void SynchronizeWithExternalDependencies(TConstArrayView<UAnimSequenceBase*> SequencesBase);
#endif // WITH_EDITOR

#if WITH_EDITOR && ENABLE_ANIM_DEBUG
	UE_API void TestSynchronizeWithExternalDependencies();
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG

	UE_API TConstArrayView<int32> GetAssetIndexesForSourceAsset(const UObject* SourceAsset) const;
	// Experimental, this feature might be removed without warning, not for production use
	UE_API TConstArrayView<float> CalculateDynamicWeightsSqrt(TArrayView<float> DynamicWeightsSqrtBuffer) const;

#if WITH_EDITORONLY_DATA
	static UE_API void AppendToClassSchema(FAppendToClassSchemaContext& Context);
#endif // WITH_EDITORONLY_DATA

private:
	void SearchPCAKDTree(UE::PoseSearch::FSearchContext& SearchContext, UE::PoseSearch::FSearchResults& SearchResults) const;
	void SearchVPTree(UE::PoseSearch::FSearchContext& SearchContext, UE::PoseSearch::FSearchResults& SearchResults) const;
	void SearchBruteForce(UE::PoseSearch::FSearchContext& SearchContext, UE::PoseSearch::FSearchResults& SearchResults) const;
	void SearchEvent(UE::PoseSearch::FSearchContext& SearchContext, UE::PoseSearch::FSearchResults& SearchResults) const;

	typedef TArray<int32, TInlineAllocator<256, TMemStackAllocator<>>> FSelectableAssetIdx;
	UE_DEPRECATED(5.7, "Use PopulateSelectableAssetIdx taking FStackAssetSet as input instead")
	void PopulateSelectableAssetIdx(FSelectableAssetIdx& SelectableAssetIdx, TConstArrayView<const UObject*> AssetsToConsider) const;
	void PopulateSelectableAssetIdx(FSelectableAssetIdx& SelectableAssetIdx, const UE::PoseSearch::FStackAssetSet* AssetsToConsider) const;
	void PopulateSelectableAssetIdx(FSelectableAssetIdx& SelectableAssetIdx, const UE::PoseSearch::FStackDatabaseToAssetIndexes* AssetIndexesToConsiderPerDatabase) const;

	typedef TArray<int32, TInlineAllocator<256, TMemStackAllocator<>>> FNonSelectableIdx;
	void PopulateNonSelectableIdx(FNonSelectableIdx& NonSelectableIdx, UE::PoseSearch::FSearchContext& SearchContext
#if UE_POSE_SEARCH_TRACE_ENABLED
		, float ContinuingPoseCostAddend, float ContinuingInteractionCostAddend, float ContinuingContextInteractionCostAddend, TConstArrayView<float> DynamicWeightsSqrt
#endif //UE_POSE_SEARCH_TRACE_ENABLED
	) const;

	void UpdateCachedProperties();
	void ExtractCostAddends(const UE::PoseSearch::FSearchContext& SearchContext, bool bIsContinuingPoseSearch, float& ContinuingPoseCostAddend, float& ContinuingInteractionCostAddend, float& ContinuingContextInteractionCostAddend) const;
};

template<typename TDatabaseAnimationAsset>
inline const TDatabaseAnimationAsset* UPoseSearchDatabase::GetDatabaseAnimationAsset(int32 AnimationAssetIndex) const
{
	if (std::is_same<TDatabaseAnimationAsset, FPoseSearchDatabaseAnimationAsset>::value ||
		std::is_same<TDatabaseAnimationAsset, FPoseSearchDatabaseAnimationAssetBase>::value)
	{
		// using reinterpret_cast to be able to compile this method even with other types (that will return null)
		return reinterpret_cast<const TDatabaseAnimationAsset*>(GetDatabaseAnimationAsset(AnimationAssetIndex));
	}
	
	return nullptr;
}

template<typename TDatabaseAnimationAsset>
inline const TDatabaseAnimationAsset* UPoseSearchDatabase::GetDatabaseAnimationAsset(const UE::PoseSearch::FSearchIndexAsset& SearchIndexAsset) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GetDatabaseAnimationAsset<TDatabaseAnimationAsset>(SearchIndexAsset.GetSourceAssetIdx());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

template<typename TDatabaseAnimationAsset>
inline TDatabaseAnimationAsset* UPoseSearchDatabase::GetMutableDatabaseAnimationAsset(int32 AnimationAssetIndex)
{
	if (std::is_same<TDatabaseAnimationAsset, FPoseSearchDatabaseAnimationAsset>::value ||
		std::is_same<TDatabaseAnimationAsset, FPoseSearchDatabaseAnimationAssetBase>::value)
	{
		// using reinterpret_cast to be able to compile this method even with other types (that will return null)
		return reinterpret_cast<TDatabaseAnimationAsset*>(GetMutableDatabaseAnimationAsset(AnimationAssetIndex));
	}

	return nullptr;
}

template<typename TDatabaseAnimationAsset>
inline TDatabaseAnimationAsset* UPoseSearchDatabase::GetMutableDatabaseAnimationAsset(const UE::PoseSearch::FSearchIndexAsset& SearchIndexAsset)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GetMutableDatabaseAnimationAsset<TDatabaseAnimationAsset>(SearchIndexAsset.GetSourceAssetIdx());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#undef UE_API
