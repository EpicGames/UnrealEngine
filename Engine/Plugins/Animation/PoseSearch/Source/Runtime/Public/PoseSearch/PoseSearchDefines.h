// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/ParallelFor.h"
#include "BoneIndices.h"
#include "Logging/LogMacros.h"
#include "Animation/AnimTrace.h"

// Enable this if object tracing is enabled, mimics animation tracing
#ifndef UE_POSE_SEARCH_TRACE_ENABLED
#define UE_POSE_SEARCH_TRACE_ENABLED ANIM_TRACE_ENABLED
#endif

#ifndef UE_POSE_SEARCH_FORCE_SINGLE_THREAD
#define UE_POSE_SEARCH_FORCE_SINGLE_THREAD 0
#endif

#if UE_POSE_SEARCH_FORCE_SINGLE_THREAD
constexpr EParallelForFlags ParallelForFlags = EParallelForFlags::ForceSingleThread;
#else
constexpr EParallelForFlags ParallelForFlags = EParallelForFlags::None;
#endif // UE_POSE_SEARCH_FORCE_SINGLE_THREAD

POSESEARCH_API DECLARE_LOG_CATEGORY_EXTERN(LogPoseSearch, Log, All);

namespace UE::PoseSearch
{
static constexpr int8 RootSchemaBoneIdx = 0;
static constexpr int8 TrajectorySchemaBoneIdx = -1;
static constexpr int8 InvalidSchemaBoneIdx = -2;

static constexpr int8 InvalidSchemaCurveIdx = -1;

static constexpr FBoneIndexType RootBoneIndexType = 0;
static constexpr FBoneIndexType ComponentSpaceIndexType = FBoneIndexType(-1);
static constexpr FBoneIndexType WorldSpaceIndexType = FBoneIndexType(-2);

static constexpr int32 PreallocatedRolesNum = 4;
static constexpr int32 PreallocatedCachedQueriesNum = 8;
static constexpr int32 PreallocatedCachedChannelDataNum = 64;
static constexpr int32 PreallocatedSearchesNum = 4;
static constexpr float FiniteDelta = 1 / 60.0f; // Time delta used for computing pose derivatives
static constexpr int32 MaxNumberOfCollectedPoseCandidatesPerDatabase = 256;
static constexpr int32 InvalidSearchId = INDEX_NONE; // @todo: it should be 0, since the search id is an hash!
static constexpr int32 DatabaseIndexDerivedDataCacheKeyVersion = 2737; // bump this value to force reindex ALL the UPoseSearchDatabases, without having to change the global FDevSystemGuids::POSESEARCHDB_DERIVEDDATA_VER
static constexpr int32 DefaultTickPriority = 100;

template<uint32 Alignment = DEFAULT_ALIGNMENT> class TMemStackSetAllocator : public TSetAllocator<TSparseArrayAllocator<TMemStackAllocator<Alignment>, TMemStackAllocator<Alignment>>, TMemStackAllocator<Alignment>> {};

constexpr int32 TMax(int32 A, int32 B) { return (A > B ? A : B); }
template<typename ElementType> constexpr int32 TAlignOf() { return TMax(alignof(ElementType), 16); }
template<typename ElementType> using TAlignedArray = TArray<ElementType, TAlignedHeapAllocator<TAlignOf<ElementType>()>>;
template<typename ElementType> using TStackAlignedArray = TArray<ElementType, TMemStackAllocator<TAlignOf<ElementType>()>>;

}



