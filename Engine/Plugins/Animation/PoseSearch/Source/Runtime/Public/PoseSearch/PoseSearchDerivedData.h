// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "PoseSearch/PoseSearchDerivedDataKey.h"
#include "TickableEditorObject.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectSaveContext.h"

#define UE_API POSESEARCH_API

class UAnimSequenceBase;
class UPoseSearchDatabase;

namespace UE::PoseSearch
{
	struct FPoseSearchDatabaseAsyncCacheTask;
	class FPoseSearchDatabaseAsyncCacheTasks;

	enum class ERequestAsyncBuildFlag
	{
		NewRequest = 1 << 0,			// generates new key and kick off a task to get updated data (it'll cancel for eventual previous Database request, unless WaitPreviousRequest)
		ContinueRequest = 1 << 1,		// make sure there's associated data to the Database (doesn't have to be up to date)
		WaitForCompletion = 1 << 2,		// wait the termination of the NewRequest or ContinueRequest
	};
	ENUM_CLASS_FLAGS(ERequestAsyncBuildFlag);

	enum class EAsyncBuildIndexResult
	{
		InProgress,						// indexing in progress
		Success,						// the index has been built and the Database updated correctly
		Failed							// indexing failed
	};

	class FAsyncPoseSearchDatabasesManagement : public FTickableGameObject, public FTickableCookObject, public FGCObject
	{
	public:
		static UE_API EAsyncBuildIndexResult RequestAsyncBuildIndex(const UPoseSearchDatabase* Database, ERequestAsyncBuildFlag Flag);
		
		static void Clear();

	private:
		FAsyncPoseSearchDatabasesManagement();
		~FAsyncPoseSearchDatabasesManagement();

		static FAsyncPoseSearchDatabasesManagement& Get();
		static EAsyncBuildIndexResult RequestAsyncBuildIndexInternal(const UPoseSearchDatabase* Database, ERequestAsyncBuildFlag Flag);

		void OnObjectModified(UObject* Object);
		void OnObjectTransacted(UObject* Object, const FTransactionObjectEvent& TransactionObjectEvent);
		void OnPackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent);
		void OnPreObjectPropertyChanged(UObject* InObject, const FEditPropertyChain& InPropertyChain);
		void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);

		void Shutdown();
		void StartQueuedTasks(int32 MaxActiveTasks);
		void PreModified(UObject* Object, EObjectFlags ObjectFlagsToCheck);
		void PostModified(UObject* Object, EObjectFlags ObjectFlagsToCheck);

		// Begin FTickableGameObject
		virtual void Tick(float DeltaTime) override;
		virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
		virtual TStatId GetStatId() const override;
		virtual bool IsTickableWhenPaused() const override { return true; }
		virtual bool IsTickableInEditor() const override { return true; }
		// End FTickableGameObject

		// Begin FTickableCookObject
		virtual void TickCook(float DeltaTime, bool bCookCompete) override;
		// End FTickableCookObject

		// Begin FGCObject
		void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override { return TEXT("FAsyncPoseSearchDatabaseManagement"); }
		// End FGCObject
		
		void CollectDatabasesToSynchronize(UObject* Object);
		void SynchronizeDatabases();
		
		// map of all those databases UAnimSequenceBase(s) that were containing or are containing UAnimNotifyState_PoseSearchBranchIn that requires synchronization
		typedef TMap<TWeakObjectPtr<UPoseSearchDatabase>, TArray<TWeakObjectPtr<UAnimSequenceBase>>> TDatabasesToSynchronize;
		typedef TPair<TWeakObjectPtr<UPoseSearchDatabase>, TArray<TWeakObjectPtr<UAnimSequenceBase>>> TDatabasesToSynchronizePair;
		TDatabasesToSynchronize DatabasesToSynchronize;

		FPoseSearchDatabaseAsyncCacheTasks& Tasks;
		FDelegateHandle OnObjectModifiedHandle;
		FDelegateHandle OnObjectTransactedHandle;
		FDelegateHandle OnPackageReloadedHandle;
		FDelegateHandle OnPreObjectPropertyChangedHandle;
		FDelegateHandle OnObjectPropertyChangedHandle;

		// Experimental, this feature might be removed without warning, not for production use
		FPartialKeyHashes PartialKeyHashes;

		static FCriticalSection Mutex;
	};

	// Experimental, this feature might be removed without warning, not for production use
	struct FAsyncPoseSearchDatabasesManagementCache
	{
		UE_API EAsyncBuildIndexResult operator()(const UPoseSearchDatabase* Database, ERequestAsyncBuildFlag RequestAsyncBuildFlag = ERequestAsyncBuildFlag::ContinueRequest);
	private:
		TMap<const UPoseSearchDatabase*, EAsyncBuildIndexResult, TInlineSetAllocator<32>> CachedRequests;
	};
} // namespace UE::PoseSearch

#undef UE_API

#endif // WITH_EDITOR
