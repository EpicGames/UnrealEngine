// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Hash/Blake3.h"
#include "IO/IoHash.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/ObjectKey.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"

namespace UE::PoseSearch
{

using HashDigestType = FBlake3Hash;
using HashBuilderType = FBlake3;

static UObject* GetNonPackageOuter(UObject* Object)
{
	if (UObject* Outer = Object->GetOuter())
	{
		if (!Outer->IsA<UPackage>())
		{
			return Outer;
		}
	}
	return nullptr;
}

// Experimental, this feature might be removed without warning, not for production use
// FPartialKeyHashes contains the combined hash of all the properties, excluding the references to other UObject(s),
// for each UObject encountered during FKeyBuilder creation (serialization via FArchiveUObject)
struct FPartialKeyHashes
{
	struct FEntry
	{
		HashDigestType Hash;
		TArray<FObjectKey> Dependencies;

		bool CheckDependencies(TArrayView<UObject*> OtherDependencies) const
		{
			#if DO_CHECK
			if (Dependencies.Num() != OtherDependencies.Num())
			{
				return false;
			}
			
			for (int32 DependencyIndex = 0; DependencyIndex < Dependencies.Num(); ++DependencyIndex)
			{
				if (Dependencies[DependencyIndex] != OtherDependencies[DependencyIndex])
				{
					return false;
				}
			}
			#endif // DO_CHECK

			return true;
		}

		friend FArchive& operator<<(FArchive& Ar, FEntry& Entry);
	};

	typedef TPair<FObjectKey, FEntry> FPair;

	void Reset()
	{
		Entries.Reset();
	}

	void InvalidateKeysFor(UObject* Object)
	{
		if (UObject* NonPackageOuter = GetNonPackageOuter(Object))
		{
			Entries.Remove(NonPackageOuter);
		}
		Entries.Remove(Object);
	}

	const FEntry* Add(FObjectKey& ObjectKey, const HashDigestType& Hash, TArrayView<UObject*> Dependencies)
	{
		check(!Hash.IsZero());

		if (FEntry* OldEntry = Entries.Find(ObjectKey))
		{
			check(OldEntry->Hash == Hash);
			check(OldEntry->CheckDependencies(Dependencies));
			return OldEntry;
		}
		
		FEntry& NewEntry = Entries.Add(ObjectKey);
		NewEntry.Hash = Hash;
		NewEntry.Dependencies = Dependencies;
		return &NewEntry;
	}

	FEntry* FindAndRemoveResolved(FObjectKey& ObjectKey)
	{
		if (FPair* Pair = Entries.FindPair(ObjectKey))
		{
			// making sure all the FObjectKey(s) are still valid
			if (!Pair->Key.ResolveObjectPtr())
			{
				Entries.Remove(ObjectKey);
				return nullptr;
			}
			
			for (FObjectKey& Dependency : Pair->Value.Dependencies)
			{
				if (!Dependency.ResolveObjectPtr())
				{
					Entries.Remove(ObjectKey);
					return nullptr;
				}
			}

			return &Pair->Value;
		}

		return nullptr;
	}

	const FEntry* Find(const FObjectKey& ObjectKey) const
	{
		return Entries.Find(ObjectKey);
	}
	
	struct FEntries : public TMap<FObjectKey, FEntry>
	{
		FPair* FindPair(const FObjectKey& ObjectKey)
		{
			return Pairs.Find(ObjectKey);
		}
	};

	const FEntries& GetEntries() const
	{
		return Entries;
	}

private:
	FEntries Entries;
};

class FKeyBuilder : public FArchiveUObject
{
public:
	inline static const FName ExcludeFromHashName = FName(TEXT("ExcludeFromHash"));
	inline static const FName NeverInHashName = FName(TEXT("NeverInHash"));
	inline static const FName IgnoreForMemberInitializationTestName = FName(TEXT("IgnoreForMemberInitializationTest"));
	
	UE_NONCOPYABLE(FKeyBuilder);

	POSESEARCH_API FKeyBuilder();
	
	// Experimental, this feature might be removed without warning, not for production use
	POSESEARCH_API FKeyBuilder(const UObject* Object, bool bUseDataVer, bool bPerformConditionalPostLoadIfRequired, FPartialKeyHashes* PartialKeyHashes = nullptr, bool bInPerformSequenceResidencyRequests = true);

	// Experimental, this feature might be removed without warning, not for production use
	POSESEARCH_API bool ValidateAgainst(const FKeyBuilder& Other) const;

	using FArchiveUObject::IsSaving;
	using FArchiveUObject::operator<<;

	// Begin FArchive Interface
	POSESEARCH_API virtual void Seek(int64 InPos) override;
	POSESEARCH_API virtual bool ShouldSkipProperty(const FProperty* InProperty) const override;
	POSESEARCH_API virtual void Serialize(void* Data, int64 Length) override;
	POSESEARCH_API virtual FArchive& operator<<(FName& Name) override;
	POSESEARCH_API virtual FArchive& operator<<(FText& Value) override;
	POSESEARCH_API virtual FArchive& operator<<(class UObject*& Object) override;
	POSESEARCH_API virtual FString GetArchiveName() const override;
	// End FArchive Interface
	
	POSESEARCH_API bool AnyAssetNotFullyLoaded() const;
	POSESEARCH_API bool AnyAssetNotReady() const;
	POSESEARCH_API FIoHash Finalize();
	POSESEARCH_API const TSet<const UObject*>& GetDependencies() const;

	// Experimental, this feature might be removed without warning, not for production use
	static constexpr EObjectFlags ShouldIncludeFlagsToCheckOnReload = RF_Transient | RF_MirroredGarbage | RF_BeginDestroyed | RF_FinishDestroyed | RF_DuplicateTransient | RF_NonPIEDuplicateTransient;
	// Experimental, this feature might be removed without warning, not for production use
	static constexpr EObjectFlags ShouldIncludeFlagsToCheck = ShouldIncludeFlagsToCheckOnReload | RF_NewerVersionExists;
	// Experimental, this feature might be removed without warning, not for production use
	static POSESEARCH_API bool ShouldInclude(const UObject* Object, EObjectFlags ObjectFlagsToCheck);

protected:
	// to keep the key generation lightweight, we don't hash these types
	static POSESEARCH_API bool IsExcludedType(const UObject* Object);

	// to keep the key generation lightweight, we hash only the full names for these types. Object(s) will be added to Dependencies
	static POSESEARCH_API bool IsAddNameOnlyType(const UObject* Object);

	HashBuilderType Hasher;

	// UPoseSearchDatabase instance "owner" of the key generation
	UObject* KeyOwner = nullptr;

	// Set of objects that have already been serialized
	TSet<const UObject*> Dependencies;

	// true if some dependent assets are not fully loaded
	bool bAnyAssetNotFullyLoaded = false;

	// if true ConditionalPostLoad will be performed on the dependant assets requiring it
	bool bPerformConditionalPostLoad = false;

private:
	bool bPerformSequenceResidencyRequests = false;
	
	void SerializeObjectInternal(UObject* Object);
	FArchive& TryAddDependency(UObject* Object);

	FPartialKeyHashes& GetPartialKeyHashes() { return ExternalPartialKeyHashes ? *ExternalPartialKeyHashes : InternalPartialKeyHashes; }
	FArchive& operator<<(FStringBuilderBase& StringBuilderBase);

	TArray<UObject*> ObjectsToSerialize;
	TArray<UObject*> ObjectBeingSerializedDependencies;

	FPartialKeyHashes* ExternalPartialKeyHashes = nullptr;
	FPartialKeyHashes InternalPartialKeyHashes;

	TStringBuilder<256> StringBuilder;
};

} // namespace UE::PoseSearch

#endif // WITH_EDITOR
