// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTagContainer.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "PoseSearchEvent.generated.h"

#define UE_API POSESEARCH_API

USTRUCT(Experimental, BlueprintType, Category="Animation|Pose Search|Experimental")
struct FPoseSearchEvent
{
	GENERATED_BODY()

	bool IsValid() const;
	void Reset();
	UE_INTERNAL UE_API FPoseSearchEvent GetPlayRateOverriddenEvent(const FFloatInterval& PlayRateRangeBase) const;

	// Tag identifying this event
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	FGameplayTag EventTag;

	// Time in seconds until this event occurs
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	float TimeToEvent = 0.f;

	// if true pose candidates will be filtered by MotionMatching node 'PoseJumpThresholdTime' (DiscardedBy_PoseJumpThresholdTime) and 'PoseReselectHistory' (DiscardedBy_PoseReselectHistory),
	// as well as database assets FPoseSearchDatabaseAnimationAssetBase::bDisableReselection (DiscardedBy_AssetReselection)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	bool bEnablePoseFilters = false;
	
	// @todo: add PlayRate / PlayRateRangeOverride concept to UPoseSearchLibrary::MotionMatch
	// if true PlayRateRangeOverride will be used instead of FAnimNode_MotionMatching::PlayRate
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	bool bUsePlayRateRangeOverride = true;

	// @todo: support the concept of time to event channel weight, as wey of scoring better poses that are closer to the TimeToEvent
	// Effective range of play rate that can be applied to the selected animation to play, to account for better pose and trajectory matches.
	// that means the event search will evaluate poses in the range [TimeToEvent * PlayRate.Min, TimeToEvent * PlayRate.Max] seconds from any poses tagged with EventTag
	// PlayRateRangeOverride will be used if bUsePlayRateRangeOverride is true, otherwise FAnimNode_MotionMatching::PlayRate will be used instead
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State, meta = (ClampMin = "0.2", ClampMax = "3.0", UIMin = "0.2", UIMax = "3.0"))
	FFloatInterval PlayRateRangeOverride = FFloatInterval(1.f, 1.f);
};

UCLASS(MinimalAPI)
class UPoseSearchEventLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	UFUNCTION(BlueprintCallable, meta = (BlueprintThreadSafe), Category="Animation|Pose Search|Experimental")
	static void UpdatePoseSearchEvent(const FPoseSearchEvent& InNewEvent, bool bIsNewEventValid, float DeltaSeconds, UPARAM(ref) FPoseSearchEvent& InOutCurrentEvent);
};

#undef UE_API
