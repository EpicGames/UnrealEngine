// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DrawDebugHelpers.h"
#include "Interfaces/Interface_BoneReferenceSkeletonProvider.h"
#include "PoseSearch/PoseSearchRole.h"
#include "PoseSearchFeatureVectorHelper.h"
#include "PoseSearchFilter.h"
#include "StructUtils/InstancedStruct.h"
#include "PoseSearchFeatureChannel.generated.h"

#define UE_API POSESEARCH_API

class UPoseSearchSchema;

UENUM()
enum class EInputQueryPose : uint8
{
	// Use character pose to compose the query.
	UseCharacterPose,

	// If available reuse continuing pose from the database to compose the query, or else UseCharacterPose.
	UseContinuingPose,
};

// this enumeration controls the channel sampling time:
// for example if a channel specifies a bone and an origin bone (used to generate the reference system of the features associated to the bone),
// bone and origin bone will be evaluated at potentially different times:
UENUM()
enum class EPermutationTimeType : uint8
{
	// Bone and origin bone are sampled at the same sample time (plus eventual SampleTimeOffset for the bone):
	// it's defined as the current animation evaluation time.
	UseSampleTime,

	// Bone and origin bone are sampled at the same permutation time (plus eventual SampleTimeOffset for the bone):
	// it's defined as SamplingTime (as UseSampleTime) + Schema->PermutationsTimeOffset + PermutationIndex / Schema->PermutationsSampleRate
	// where PermutationIndex is in range [0, Schema->NumberOfPermutations).
	UsePermutationTime,

	// Bone is evaluated at sample time (and plus eventual SampleTimeOffset) and origin bone is evaluated at permutation time.
	UseSampleToPermutationTime,
};

namespace UE::PoseSearch
{

struct FDebugDrawParams;
struct FSearchContext;
struct FPoseMetadata;

#if WITH_EDITOR
class FAssetIndexer;

enum class ELabelFormat : uint8
{
	// output label example: "Traj_Vel_xy 1.20"
	Full_Horizontal,

	// output label example: "Traj\nVel_xy\n1.20"
	Full_Vertical,

	// output label example: "Vel_xy 1.20"
	Compact_Horizontal
};

typedef TStringBuilder<256> TLabelBuilder;

#endif // WITH_EDITOR

} // namespace UE::PoseSearch

//////////////////////////////////////////////////////////////////////////
// Feature channels interface
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCLASS(MinimalAPI, Abstract, BlueprintType, EditInlineNew)
class UPoseSearchFeatureChannel : public UObject, public IBoneReferenceSkeletonProvider, public IPoseSearchFilter
{
	GENERATED_BODY()
PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:
	int32 GetChannelCardinality() const { checkSlow(ChannelCardinality >= 0); return ChannelCardinality; }
	int32 GetChannelDataOffset() const { checkSlow(ChannelDataOffset >= 0); return ChannelDataOffset; }

	// Called during UPoseSearchSchema::Finalize to prepare the schema for this channel
	virtual bool Finalize(UPoseSearchSchema* Schema) PURE_VIRTUAL(UPoseSearchFeatureChannel::Finalize, return false;);
	
	// Called at runtime to add this channel's data to the query pose vector
	virtual void BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const PURE_VIRTUAL(UPoseSearchFeatureChannel::BuildQuery, );

	// UPoseSearchFeatureChannels can hold sub channels
	virtual TArrayView<TObjectPtr<UPoseSearchFeatureChannel>> GetSubChannels() { return TArrayView<TObjectPtr<UPoseSearchFeatureChannel>>(); }
	virtual TConstArrayView<TObjectPtr<UPoseSearchFeatureChannel>> GetSubChannels() const { return TConstArrayView<TObjectPtr<UPoseSearchFeatureChannel>>(); }

	virtual void AddDependentChannels(UPoseSearchSchema* Schema) const {}

	int32 GetNumFilters() const { return Filters.Num(); }
	const FPoseSearchFilter* GetFilter(int32 FilterIndex) const { return Filters[FilterIndex].GetPtr<FPoseSearchFilter>(); }

	virtual EPermutationTimeType GetPermutationTimeType() const { return EPermutationTimeType::UseSampleTime; }
	static UE_API void GetPermutationTimeOffsets(EPermutationTimeType PermutationTimeType, float DesiredPermutationTimeOffset, float& OutPermutationSampleTimeOffset, float& OutPermutationOriginTimeOffset);

#if ENABLE_DRAW_DEBUG
	// Draw this channel's data for the given pose vector
	UE_API virtual void DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseVector) const;
#endif //ENABLE_DRAW_DEBUG

#if WITH_EDITOR
	// Called at database build time to collect feature weights.
	// Weights is sized to the cardinality of the schema and the feature channel should write
	// its weights at the channel's data offset. Channels should provide a weight for each dimension.
	virtual void FillWeights(TArrayView<float> Weights) const PURE_VIRTUAL(UPoseSearchFeatureChannel::FillWeights, );

	// Called at database build time to populate pose vectors with this channel's data
	virtual bool IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const PURE_VIRTUAL(UPoseSearchFeatureChannel::IndexAsset, return false;);

	// returns the TLabelBuilder used editor side to identify this UPoseSearchFeatureChannel (for instance in the pose search debugger)
	UE_API virtual UE::PoseSearch::TLabelBuilder& GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat = UE::PoseSearch::ELabelFormat::Full_Horizontal) const;

	// returns true if the data associated to this channel can be normalized toghether with the associated with Other
	UE_API virtual bool CanBeNormalizedWith(const UPoseSearchFeatureChannel* Other) const;

	// if this channel GetNormalizationGroup returns a valid FName, all the channels of the same class with the same cardinality, and the same NormalizationGroup, 
	// will make CanBeNormalizedWith return true and will be normalized together.
	// for example in a locomotion database of a character holding a weapon, containing non mirrorable animations, you'd still want to normalize together 
	// left foot and right foot position and velocity, so you'd want those channels returning the same GetNormalizationGroup value
	virtual FName GetNormalizationGroup() const { return FName(); }

	UE_API const UPoseSearchSchema* GetSchema() const;

	UE_API virtual const UE::PoseSearch::FRole GetDefaultRole() const;
#endif

#if WITH_EDITORONLY_DATA
	virtual FLinearColor GetDebugColor() const { return FLinearColor::White; }
#endif // WITH_EDITORONLY_DATA

	// IBoneReferenceSkeletonProvider interface
	// Note this function is exclusively for FBoneReference details customization
	UE_API USkeleton* GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle) override;

protected:
#if WITH_EDITOR
	UE_API void GetOuterLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat) const;
	static UE_API void AppendLabelSeparator(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat, bool bTryUsingSpace = false);
#endif //WITH_EDITOR
	friend class ::UPoseSearchSchema;

	UPROPERTY(Transient)
	int32 ChannelDataOffset = INDEX_NONE;

	UPROPERTY(Transient)
	int32 ChannelCardinality = INDEX_NONE;

	UPROPERTY(EditAnywhere, Category = "Filters", meta = (ExcludeBaseStruct, BaseStruct = "/Script/PoseSearch.PoseSearchFilter"))
	TArray<FInstancedStruct> Filters;
};
#undef UE_API
