// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchFeatureChannel.h"
#include "PoseSearchFeatureChannel_Curve.generated.h"

#define UE_API POSESEARCH_API

UCLASS(MinimalAPI, Experimental, BlueprintType, EditInlineNew, Blueprintable, meta = (DisplayName = "Curve Channel"), CollapseCategories)
class UPoseSearchFeatureChannel_Curve : public UPoseSearchFeatureChannel
{
	GENERATED_BODY()

public:

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Settings")
	FName CurveName = NAME_None;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName SampleRole = UE::PoseSearch::DefaultRole;

	UPROPERTY(Transient)
	int8 CurveIdx = UE::PoseSearch::InvalidSchemaCurveIdx;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Settings")
	float Weight = 1.f;
#endif // WITH_EDITORONLY_DATA
	
	// the data relative to the sampling time associated to this channel will be offset by SampleTimeOffset seconds.
	// For example, if Curve is DistanceToWall, and SampleTimeOffset is 0.5, this channel will try to match the future DistanceToWall curve of the character 0.5 seconds ahead
	UPROPERTY(EditAnywhere, Category = "Settings")
	float SampleTimeOffset = 0.f;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ExcludeFromHash, DisplayPriority = 0))
	FLinearColor DebugColor = FLinearColor::Green;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category = "Settings")
	EInputQueryPose InputQueryPose = EInputQueryPose::UseContinuingPose;

#if WITH_EDITORONLY_DATA
	// if set, all the channels of the same class with the same cardinality, and the same NormalizationGroup, will be normalized together.
	// for example in a locomotion database of a character holding a weapon, containing non mirrorable animations, you'd still want to normalize together 
	// left foot and right foot position and Curve
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName NormalizationGroup;
#endif //WITH_EDITORONLY_DATA

	UE_DEPRECATED(5.8, "Use BP_GetCurveValueFromContext instead")
	UFUNCTION(BlueprintPure, BlueprintImplementableEvent, meta=(BlueprintThreadSafe, DisplayName = "Get World Curve"), Category = "Settings")
	UE_API float BP_GetCurveValue(const UAnimInstance* AnimInstance) const;

	UFUNCTION(BlueprintPure, BlueprintNativeEvent, meta=(BlueprintThreadSafe, DisplayName = "Get World Curve From Context"), Category = "Settings")
	UE_API float BP_GetCurveValueFromContext(UPARAM(Ref) FChooserEvaluationContext& Context) const;

	bool bUseBlueprintQueryOverride = false;

	UE_API UPoseSearchFeatureChannel_Curve();

	// UPoseSearchFeatureChannel interface
	UE_API virtual bool Finalize(UPoseSearchSchema* Schema) override;
	UE_API virtual void BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const override;

#if WITH_EDITOR
	UE_API virtual void FillWeights(TArrayView<float> Weights) const override;
	UE_API virtual bool IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const override;
	UE_API virtual UE::PoseSearch::TLabelBuilder& GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat = UE::PoseSearch::ELabelFormat::Full_Horizontal) const override;
	virtual FName GetNormalizationGroup() const override { return NormalizationGroup; }
#endif

#if WITH_EDITORONLY_DATA
	virtual FLinearColor GetDebugColor() const override { return DebugColor; }
#endif // WITH_EDITORONLY_DATA
};

#undef UE_API
