// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchFeatureChannel.h"
#include "BoneContainer.h"
#include "IObjectChooser.h"
#include "PoseSearchFeatureChannel_Distance.generated.h"

#define UE_API POSESEARCH_API

UCLASS(MinimalAPI, Experimental, EditInlineNew, Blueprintable, meta = (DisplayName = "Distance Channel"), CollapseCategories)
class UPoseSearchFeatureChannel_Distance : public UPoseSearchFeatureChannel
{
	GENERATED_BODY()

public:
	// Experimental, this feature might be removed without warning, not for production use
	// if bDefaultWithRootBone is true Bone.BoneName and OriginBone.BoneName get initialized to the associated
	// skeleton root bone if equals to NAME_None, otherwise transforms from the trajectory will be used
	UPROPERTY(EditAnywhere, Category = "Experimental")
	bool bDefaultWithRootBone = true;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference Bone;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName SampleRole = UE::PoseSearch::DefaultRole;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference OriginBone;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName OriginRole = UE::PoseSearch::DefaultRole;

	// Experimental, this feature might be removed without warning, not for production use
   	// During pose selection if the distance between query versus candidate poses for this 
   	// distance channel is greater than MaxDistance the candidate will be discarded.
   	// the filtering will be enabled only for MaxDistance > 0
   	UPROPERTY(EditAnywhere, Category = "Experimental|Filter")
   	float MaxDistance = 0.f;
	
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Settings")
	float Weight = 1.f;
#endif // WITH_EDITORONLY_DATA

	// if SamplingAttributeId >= 0, ALL the animations contained in the pose search database referencing the schema containing this channel are expected to have 
	// UAnimNotifyState_PoseSearchSamplingAttribute notify state with a matching SamplingAttributeId, and the UAnimNotifyState_PoseSearchSamplingAttribute properties
	// will be used as source of data instead of this channel "Bone". UAnimNotifyState_PoseSearchSamplingAttribute properties will be then converted into OriginBone space
	UPROPERTY(EditAnywhere, Category = "Settings")
	int32 SamplingAttributeId = INDEX_NONE;
	
	// the data relative to the sampling time associated to this channel will be offsetted by SampleTimeOffset seconds.
	// For example, if Bone is the head bone, and SampleTimeOffset is 0.5, this channel will try to match the future distance of the character head bone 0.5 seconds ahead
	UPROPERTY(EditAnywhere, Category = "Settings")
	float SampleTimeOffset = 0.f;

	// the data relative to the sampling time associated to this channel origin (root / trajectory bone) will be offsetted by OriginTimeOffset seconds.
	// For example, if Bone is the head bone, SampleTimeOffset is 0.5, and OriginTimeOffset is 0.5, this channel will try to match 
	// the future distance of the character head bone 0.5 seconds ahead, relative to the future root bone 0.5 seconds ahead
	UPROPERTY(EditAnywhere, Category = "Settings")
	float OriginTimeOffset = 0.f;

	// index referencing the associated bone in UPoseSearchSchema::BoneReferences
	UPROPERTY(Transient)
	int8 SchemaBoneIdx = 0;

	UPROPERTY(Transient)
	int8 SchemaOriginBoneIdx = 0;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ExcludeFromHash, DisplayPriority = 0))
	FLinearColor DebugColor = FLinearColor::Blue;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category = "Settings")
	EInputQueryPose InputQueryPose = EInputQueryPose::UseContinuingPose;

	UPROPERTY(EditAnywhere, Category = "Settings")
	EPermutationTimeType PermutationTimeType = EPermutationTimeType::UseSampleTime;

#if WITH_EDITORONLY_DATA
	// if set, all the channels of the same class with the same cardinality, and the same NormalizationGroup, will be normalized together.
	// for example in a locomotion database of a character holding a weapon, containing non mirrorable animations, you'd still want to normalize together 
	// all the distances
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName NormalizationGroup;
#endif //WITH_EDITORONLY_DATA

	UFUNCTION(BlueprintPure, BlueprintImplementableEvent, meta=(BlueprintThreadSafe, DisplayName = "Get Distance"), Category = "Settings")
	UE_API float BP_GetDistance(UPARAM(Ref) FChooserEvaluationContext& Context) const;

	bool bUseBlueprintQueryOverride = false;

	UE_API UPoseSearchFeatureChannel_Distance();

	// UPoseSearchFeatureChannel interface
	UE_API virtual bool Finalize(UPoseSearchSchema* Schema) override;
	UE_API virtual void BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const override;

	virtual EPermutationTimeType GetPermutationTimeType() const override { return PermutationTimeType; }
	UE_API virtual void AddDependentChannels(UPoseSearchSchema* Schema) const override;
	
	// IPoseSearchFilter interface
	// Experimental, this feature might be removed without warning, not for production use
	virtual bool IsFilterActive() const override { return MaxDistance > 0.f; }
	// Experimental, this feature might be removed without warning, not for production use
	UE_API virtual bool IsFilterValid(TConstArrayView<float> PoseValues, TConstArrayView<float> QueryValues, int32 PoseIdx, const UE::PoseSearch::FPoseMetadata& Metadata) const override;

#if ENABLE_DRAW_DEBUG
	UE_API virtual void DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseVector) const override;
#endif // ENABLE_DRAW_DEBUG

#if WITH_EDITOR
	UE_API virtual void FillWeights(TArrayView<float> Weights) const override;
	UE_API virtual bool IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const override;
	UE_API virtual UE::PoseSearch::TLabelBuilder& GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat = UE::PoseSearch::ELabelFormat::Full_Horizontal) const override;
	virtual FName GetNormalizationGroup() const override { return NormalizationGroup; }

	// IBoneReferenceSkeletonProvider interface
	UE_API USkeleton* GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle) override;
#endif

#if WITH_EDITORONLY_DATA
	virtual FLinearColor GetDebugColor() const override { return DebugColor; }
#endif // WITH_EDITORONLY_DATA
};

#undef UE_API
