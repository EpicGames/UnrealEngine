// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchFeatureChannel.h"
#include "BoneContainer.h"
#include "PoseSearchFeatureChannel_FilterCrashingLegs.generated.h"

#define UE_API POSESEARCH_API

// the idea is to calculate the angle between the direction from LeftThigh position to RightThigh position and the direction from LeftFoot position to RightFoot position, and divide it by PI to have values in range [-1,1]
// the number (called 'CrashingLegsValue' calculated in ComputeCrashingLegsValue) is gonna be
// 0 if the feet are aligned with the thighs (for example in an idle standing position)
// 0.5 if the right foot is exactly in front of the left foot (for example when a character is running  following a line)
// -0.5 if the left foot is exactly in front of the right foot
// close to 1 or -1 if the feet (and so the legs) are completely crossed
// at runtime we'll match the CrashingLegsValue and also filter by discarding pose candidates that don't respect the 'AllowedTolerance' between query and database values (happening in IsFilterValid)
UCLASS(MinimalAPI, Experimental, BlueprintType, EditInlineNew, meta = (DisplayName = "CrashingLegs Channel"), CollapseCategories)
class UPoseSearchFeatureChannel_FilterCrashingLegs : public UPoseSearchFeatureChannel
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference LeftThigh;
	
	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference RightThigh;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference LeftFoot;
	
	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference RightFoot;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName SampleRole = UE::PoseSearch::DefaultRole;

	UPROPERTY(EditAnywhere, Category = "Settings")
	float Weight = 0.2f;

	UPROPERTY()
	int8 LeftThighIdx;

	UPROPERTY()
	int8 RightThighIdx;

	UPROPERTY()
	int8 LeftFootIdx;

	UPROPERTY()
	int8 RightFootIdx;

	UPROPERTY(EditAnywhere, Category = "Settings")
	EInputQueryPose InputQueryPose = EInputQueryPose::UseContinuingPose;

	// if AllowedTolerance is zero the filter is disabled
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "0", UIMin = "0"))
	float AllowedTolerance = 0.3f;

	// UPoseSearchFeatureChannel interface
	UE_API virtual bool Finalize(UPoseSearchSchema* Schema) override;
	UE_API virtual void BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const override;

	UE_API virtual void AddDependentChannels(UPoseSearchSchema* Schema) const override;

#if ENABLE_DRAW_DEBUG
	UE_API virtual void DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseVector) const override;
#endif // ENABLE_DRAW_DEBUG

#if WITH_EDITOR
	UE_API virtual void FillWeights(TArrayView<float> Weights) const override;
	UE_API virtual bool IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const override;
	virtual const UE::PoseSearch::FRole GetDefaultRole() const override { return SampleRole; }
#endif

	// @todo: reimplement this using the new FPoseSearchFilter
	// IPoseSearchFilter interface
	UE_API virtual bool IsFilterActive() const override;
	UE_API virtual bool IsFilterValid(TConstArrayView<float> PoseValues, TConstArrayView<float> QueryValues, int32 PoseIdx, const UE::PoseSearch::FPoseMetadata& Metadata) const override;
};

#undef UE_API
