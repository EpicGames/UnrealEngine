// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchFeatureChannel.h"
#include "BoneContainer.h"
#include "PoseSearchFeatureChannel_Group.generated.h"

#define UE_API POSESEARCH_API

// Feature channels interface
UCLASS(MinimalAPI, Abstract, BlueprintType, EditInlineNew)
class UPoseSearchFeatureChannel_GroupBase : public UPoseSearchFeatureChannel
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName SampleRole = UE::PoseSearch::DefaultRole;

	// Experimental, this feature might be removed without warning, not for production use
	// identifier / metadata to categorize this group base channel weights to be able to zero them via 
	// UPoseSearchDatabase::CalculateDynamicWeightsSqrt
	UPROPERTY(EditAnywhere, Category = "Settings")
	int32 DebugWeightGroupID = INDEX_NONE;

	// UPoseSearchFeatureChannel interface
	UE_API virtual bool Finalize(UPoseSearchSchema* Schema) override;
	UE_API virtual void BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const override;
	UE_API virtual void AddDependentChannels(UPoseSearchSchema* Schema) const override; 

#if ENABLE_DRAW_DEBUG
	UE_API virtual void DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseVector) const override;
#endif // ENABLE_DRAW_DEBUG

#if WITH_EDITOR
	UE_API virtual void FillWeights(TArrayView<float> Weights) const override;
	UE_API virtual bool IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const override;
	virtual const UE::PoseSearch::FRole GetDefaultRole() const override { return SampleRole; }
#endif //WITH_EDITOR

	// IPoseSearchFilter interface
	UE_API virtual bool IsFilterActive() const override;
	UE_API virtual bool IsFilterValid(TConstArrayView<float> PoseValues, TConstArrayView<float> QueryValues, int32 PoseIdx, const UE::PoseSearch::FPoseMetadata& Metadata) const override;
};

UCLASS(MinimalAPI, BlueprintType, EditInlineNew, meta = (DisplayName = "Group Channel"), CollapseCategories)
class UPoseSearchFeatureChannel_Group : public UPoseSearchFeatureChannel_GroupBase
{
	GENERATED_BODY()

	virtual TArrayView<TObjectPtr<UPoseSearchFeatureChannel>> GetSubChannels() override { return SubChannels; }
	virtual TConstArrayView<TObjectPtr<UPoseSearchFeatureChannel>> GetSubChannels() const override { return SubChannels; }

#if WITH_EDITOR
	UE_API virtual UE::PoseSearch::TLabelBuilder& GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat = UE::PoseSearch::ELabelFormat::Full_Horizontal) const override;
#endif

public:
	UPROPERTY(EditAnywhere, Instanced, BlueprintReadWrite, Category = "SubChannels")
	TArray<TObjectPtr<UPoseSearchFeatureChannel>> SubChannels;
};

#undef UE_API
