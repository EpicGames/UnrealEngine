// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchFeatureChannel.h"
#include "PoseSearchFeatureChannel_Padding.generated.h"

#define UE_API POSESEARCH_API

UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Padding Channel"), CollapseCategories)
class UPoseSearchFeatureChannel_Padding : public UPoseSearchFeatureChannel
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Settings")
	int32 PaddingSize = 1;

	// UPoseSearchFeatureChannel interface
	UE_API virtual bool Finalize(UPoseSearchSchema* Schema) override;
	UE_API virtual void BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const override;

#if WITH_EDITOR
	UE_API virtual void FillWeights(TArrayView<float> Weights) const override;
	UE_API virtual bool IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const override;
	UE_API virtual UE::PoseSearch::TLabelBuilder& GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat = UE::PoseSearch::ELabelFormat::Full_Horizontal) const override;
#endif

	static UE_API void AddToSchema(UPoseSearchSchema* Schema, int32 PaddingSize);
};

#undef UE_API
