// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchFeatureChannel.h"
#include "PoseSearchFeatureChannel_PermutationTime.generated.h"

#define UE_API POSESEARCH_API

UCLASS(MinimalAPI, Experimental, EditInlineNew, meta = (DisplayName = "Permutation Time Channel"), CollapseCategories)
class UPoseSearchFeatureChannel_PermutationTime : public UPoseSearchFeatureChannel
{
	GENERATED_BODY()

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Settings")
	float Weight = 1.f;
#endif // WITH_EDITORONLY_DATA

	// UPoseSearchFeatureChannel interface
	UE_API virtual bool Finalize(UPoseSearchSchema* Schema) override;
	UE_API virtual void BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const override;

#if WITH_EDITOR
	UE_API virtual void FillWeights(TArrayView<float> Weights) const override;
	UE_API virtual bool IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const override;
	UE_API virtual UE::PoseSearch::TLabelBuilder& GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat = UE::PoseSearch::ELabelFormat::Full_Horizontal) const override;
#endif

	UE_API float GetPermutationTime(TConstArrayView<float> PoseVector) const;

	static UE_API void FindOrAddToSchema(UPoseSearchSchema* Schema);
};

#undef UE_API
