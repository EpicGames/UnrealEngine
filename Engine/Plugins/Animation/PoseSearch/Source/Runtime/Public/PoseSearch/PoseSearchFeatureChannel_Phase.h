// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchFeatureChannel.h"
#include "BoneContainer.h"
#include "PoseSearchFeatureChannel_Phase.generated.h"

#define UE_API POSESEARCH_API

UCLASS(MinimalAPI, Experimental, BlueprintType, EditInlineNew, meta = (DisplayName = "Phase Channel"), CollapseCategories)
class UPoseSearchFeatureChannel_Phase : public UPoseSearchFeatureChannel
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference Bone;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName SampleRole = UE::PoseSearch::DefaultRole;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Settings")
	float Weight = 1.f;
#endif // WITH_EDITORONLY_DATA

	// index referencing the associated bone in UPoseSearchSchema::BoneReferences
	UPROPERTY(Transient)
	int8 SchemaBoneIdx = 0;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ExcludeFromHash, DisplayPriority = 0))
	FLinearColor DebugColor = FLinearColor::Yellow;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category = "Settings")
	EInputQueryPose InputQueryPose = EInputQueryPose::UseContinuingPose;

#if WITH_EDITORONLY_DATA
	// if set, all the channels of the same class with the same cardinality, and the same NormalizationGroup, will be normalized together.
	// for example in a locomotion database of a character holding a weapon, containing non mirrorable animations, you'd still want to normalize together 
	// left foot and right foot position and velocity
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName NormalizationGroup;
#endif //WITH_EDITORONLY_DATA

	// UPoseSearchFeatureChannel interface
	UE_API virtual bool Finalize(UPoseSearchSchema* Schema) override;
	UE_API virtual void BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const override;

	UE_API virtual void AddDependentChannels(UPoseSearchSchema* Schema) const override;

#if ENABLE_DRAW_DEBUG
	UE_API virtual void DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseVector) const override;
#endif // ENABLE_DRAW_DEBUG

#if WITH_EDITOR
	UE_API virtual void FillWeights(TArrayView<float> Weights) const override;
	UE_API virtual bool IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const override;
	UE_API virtual UE::PoseSearch::TLabelBuilder& GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat = UE::PoseSearch::ELabelFormat::Full_Horizontal) const override;
	virtual FName GetNormalizationGroup() const override { return NormalizationGroup; }
	virtual const UE::PoseSearch::FRole GetDefaultRole() const override { return SampleRole; }
#endif

#if WITH_EDITORONLY_DATA
	virtual FLinearColor GetDebugColor() const override { return DebugColor; }
#endif // WITH_EDITORONLY_DATA
};

#undef UE_API
