// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearchFeatureChannel_Group.h"
#include "PoseSearchFeatureChannel_Pose.generated.h"

#define UE_API POSESEARCH_API

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EPoseSearchBoneFlags : uint32
{
	Velocity = 1 << 0,
	Position = 1 << 1,
	Rotation = 1 << 2,
	Phase = 1 << 3,
};
ENUM_CLASS_FLAGS(EPoseSearchBoneFlags);
constexpr bool EnumHasAnyFlags(int32 Flags, EPoseSearchBoneFlags Contains) { return (Flags & int32(Contains)) != 0; }
inline int32& operator|=(int32& Lhs, EPoseSearchBoneFlags Rhs) { return Lhs |= int32(Rhs); }

USTRUCT()
struct FPoseSearchBone
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Config)
	FBoneReference Reference;

	// This allows the user to define what information from the channel you want to compare to.
	UPROPERTY(EditAnywhere, meta = (Bitmask, BitmaskEnum = "/Script/PoseSearch.EPoseSearchBoneFlags"), Category = Config)
	int32 Flags = int32(EPoseSearchBoneFlags::Position);

#if WITH_EDITORONLY_DATA
	// if set, all the channels of the same class with the same cardinality, and the same NormalizationGroup, will be normalized together.
	// for example in a locomotion database of a character holding a weapon, containing non mirrorable animations, you'd still want to normalize together 
	// left foot and right foot position and velocity
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName NormalizationGroup;

	UPROPERTY(EditAnywhere, Category = Config)
	float Weight = 1.f;

	UPROPERTY(EditAnywhere, Category = Config, meta=(ExcludeFromHash, DisplayPriority = 0))
	FLinearColor DebugColor = FLinearColor::Green;
#endif // WITH_EDITORONLY_DATA
};

// UPoseSearchFeatureChannel_Pose
UCLASS(MinimalAPI, BlueprintType, EditInlineNew, meta = (DisplayName = "Pose Channel"), CollapseCategories)
class UPoseSearchFeatureChannel_Pose : public UPoseSearchFeatureChannel_GroupBase
{
	GENERATED_BODY()

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Settings")
	float Weight = 1.f;
#endif // WITH_EDITORONLY_DATA

	// List of skeletal joints and associated Flags (Velocity, Position, etc) to sample.
	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FPoseSearchBone> SampledBones;

	UPROPERTY()
	TArray<int8> SchemaBoneIdx;

	UPROPERTY(EditAnywhere, Category = "Settings")
	EInputQueryPose InputQueryPose = EInputQueryPose::UseContinuingPose;

	// if bUseCharacterSpaceVelocities is true, velocities will be calculated from the positions in character space, otherwise they will be calculated using global space positions
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bUseCharacterSpaceVelocities = true;

	UPROPERTY(EditAnywhere, Category = "Settings")
	EPermutationTimeType PermutationTimeType = EPermutationTimeType::UseSampleTime;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UPoseSearchFeatureChannel>> SubChannels;

	UE_API UPoseSearchFeatureChannel_Pose();

	// UPoseSearchFeatureChannel_GroupBase interface
	virtual TArrayView<TObjectPtr<UPoseSearchFeatureChannel>> GetSubChannels() override { return SubChannels; }
	virtual TConstArrayView<TObjectPtr<UPoseSearchFeatureChannel>> GetSubChannels() const override { return SubChannels; }

	// UPoseSearchFeatureChannel interface
	UE_API virtual bool Finalize(UPoseSearchSchema* Schema) override;

#if WITH_EDITOR
	UE_API virtual UE::PoseSearch::TLabelBuilder& GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat = UE::PoseSearch::ELabelFormat::Full_Horizontal) const override;
#endif
};


#undef UE_API
