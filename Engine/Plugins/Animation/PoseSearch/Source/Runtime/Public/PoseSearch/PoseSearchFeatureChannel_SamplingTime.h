// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchFeatureChannel.h"
#include "PoseSearchFeatureChannel_SamplingTime.generated.h"

#define UE_API POSESEARCH_API

// this channel is mainly for debug purposes to augment the features data with the sampling time (default weight is set to zero to be irrelevant during searches)
UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Sampling Time Channel"), CollapseCategories)
class UPoseSearchFeatureChannel_SamplingTime : public UPoseSearchFeatureChannel
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Settings")
	float Weight = 0.f;

	UPROPERTY(EditAnywhere, Category = "Settings")
	float TimeToMatch = 0.f;

	// UPoseSearchFeatureChannel interface
	UE_API virtual bool Finalize(UPoseSearchSchema* Schema) override;
	UE_API virtual void BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const override;

#if WITH_EDITOR
	UE_API virtual void FillWeights(TArrayView<float> Weights) const override;
	UE_API virtual bool IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const override;
	UE_API virtual UE::PoseSearch::TLabelBuilder& GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat = UE::PoseSearch::ELabelFormat::Full_Horizontal) const override;
#endif
};

#undef UE_API
