// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchFeatureChannel.h"
#include "PoseSearchFeatureChannel_TimeToEvent.generated.h"

#define UE_API POSESEARCH_API

UCLASS(MinimalAPI, EditInlineNew, Blueprintable, meta = (DisplayName = "Time To Event Channel"), CollapseCategories)
class UPoseSearchFeatureChannel_TimeToEvent : public UPoseSearchFeatureChannel
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Settings")
	float Weight = 0.f;

	UPROPERTY(EditAnywhere, Category = "Settings")
	int32 SamplingAttributeId = 0;

	UE_DEPRECATED(5.8, "Use BP_GetTimeToEventFromContext instead")
	UFUNCTION(BlueprintPure, BlueprintImplementableEvent, meta=(BlueprintThreadSafe, DisplayName = "Get Time To Event"), Category = "Settings")
	UE_API float BP_GetTimeToEvent(const UAnimInstance* AnimInstance) const;

	UFUNCTION(BlueprintPure, BlueprintNativeEvent, meta=(BlueprintThreadSafe, DisplayName = "Get Time To Event From Context"), Category = "Settings")
	UE_API float BP_GetTimeToEventFromContext(UPARAM(Ref) FChooserEvaluationContext& Context) const;

	bool bUseBlueprintQueryOverride = false;

	UE_API UPoseSearchFeatureChannel_TimeToEvent();

	// UPoseSearchFeatureChannel interface
	UE_API virtual bool Finalize(UPoseSearchSchema* Schema) override;
	UE_API virtual void BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const override;

#if WITH_EDITOR
	UE_API virtual void FillWeights(TArrayView<float> Weights) const override;
	UE_API virtual bool IndexAsset(UE::PoseSearch::FAssetIndexer& Indexer) const override;
	UE_API virtual UE::PoseSearch::TLabelBuilder& GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat = UE::PoseSearch::ELabelFormat::Full_Horizontal) const override;
#endif
};

#undef UE_API
