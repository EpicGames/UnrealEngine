// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearchFeatureChannel_Group.h"
#include "UObject/ObjectSaveContext.h"
#include "PoseSearchFeatureChannel_Trajectory.generated.h"

#define UE_API POSESEARCH_API

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EPoseSearchTrajectoryFlags : uint32
{
	Velocity = 1 << 0,
	Position = 1 << 1,
	VelocityDirection = 1 << 2,
	FacingDirection = 1 << 3,
	VelocityXY = 1 << 4,
	PositionXY = 1 << 5,
	VelocityDirectionXY = 1 << 6,
	FacingDirectionXY = 1 << 7,
};
ENUM_CLASS_FLAGS(EPoseSearchTrajectoryFlags);
constexpr bool EnumHasAnyFlags(int32 Flags, EPoseSearchTrajectoryFlags Contains) { return (Flags & int32(Contains)) != 0; }
inline int32& operator|=(int32& Lhs, EPoseSearchTrajectoryFlags Rhs) { return Lhs |= int32(Rhs); }

USTRUCT()
struct FPoseSearchTrajectorySample
{
	GENERATED_BODY()

	// the data relative to the sampling time associated to this FPoseSearchTrajectorySample will be offsetted by Offset seconds.
	// For example, Flags is Position, and Offset is 0.5, this channel will try to match the future position of the trajectory 0.5 seconds ahead
	UPROPERTY(EditAnywhere, Category = Config)
	float Offset = 0.f;

	// This allows the user to define what information from the channel you want to compare to.
	UPROPERTY(EditAnywhere, meta = (Bitmask, BitmaskEnum = "/Script/PoseSearch.EPoseSearchTrajectoryFlags"), Category = Config)
	int32 Flags = int32(EPoseSearchTrajectoryFlags::Position);

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = Config)
	float Weight = 1.f;

	// if set, all the channels of the same class with the same cardinality, and the same NormalizationGroup, will be normalized together.
	// for example in a locomotion database of a character holding a weapon, containing non mirrorable animations, you'd still want to normalize together 
	// left foot and right foot position and velocity
	UPROPERTY(EditAnywhere, Category = Config)
	FName NormalizationGroup;

	UPROPERTY(EditAnywhere, Category = Config, meta = (ExcludeFromHash, DisplayPriority = 0))
	FLinearColor DebugColor = FLinearColor::Blue;
#endif // WITH_EDITORONLY_DATA
};

UCLASS(MinimalAPI, BlueprintType, EditInlineNew, meta = (DisplayName = "Trajectory Channel"), CollapseCategories)
class UPoseSearchFeatureChannel_Trajectory : public UPoseSearchFeatureChannel_GroupBase
{
	GENERATED_BODY()

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Settings")
	float Weight = 1.f;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FPoseSearchTrajectorySample> Samples;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UPoseSearchFeatureChannel>> SubChannels;

	UE_API UPoseSearchFeatureChannel_Trajectory();

	// UPoseSearchFeatureChannel_GroupBase interface
	virtual TArrayView<TObjectPtr<UPoseSearchFeatureChannel>> GetSubChannels() override { return SubChannels; }
	virtual TConstArrayView<TObjectPtr<UPoseSearchFeatureChannel>> GetSubChannels() const override { return SubChannels; }

	// UPoseSearchFeatureChannel interface
	UE_API virtual bool Finalize(UPoseSearchSchema* Schema) override;

#if ENABLE_DRAW_DEBUG
	UE_API virtual void DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseVector) const override;
#endif // ENABLE_DRAW_DEBUG

#if WITH_EDITOR
	UE_API virtual UE::PoseSearch::TLabelBuilder& GetLabel(UE::PoseSearch::TLabelBuilder& LabelBuilder, UE::PoseSearch::ELabelFormat LabelFormat = UE::PoseSearch::ELabelFormat::Full_Horizontal) const override;
#endif

	UE_API float GetEstimatedSpeedRatio(TConstArrayView<float> QueryVector, TConstArrayView<float> PoseVector) const;
	UE_API FVector GetEstimatedFutureRootMotionVelocity(TConstArrayView<float> PoseVector) const;
};

#undef UE_API
