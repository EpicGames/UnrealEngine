// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearchFeatureVectorHelper.generated.h"

#define UE_API POSESEARCH_API

UENUM()
enum class EComponentStrippingVector : uint8
{
	// No component stripping.
	None,

	// Stripping X and Y components (matching only on the horizontal plane).
	StripXY,

	// Stripping Z (matching only vertically - caring only about the height of the feature).
	StripZ,
};

namespace UE::PoseSearch
{

/** Helper class for extracting and encoding features into a float buffer */
class FFeatureVectorHelper
{
public:
	static UE_API int32 GetVectorCardinality(EComponentStrippingVector ComponentStrippingVector);
	static UE_API void EncodeVector(TArrayView<float> Values, int32 DataOffset, const FVector& Vector, EComponentStrippingVector ComponentStrippingVector, bool bNormalize);
	static UE_API FVector DecodeVector(TConstArrayView<float> Values, int32 DataOffset, EComponentStrippingVector ComponentStrippingVector);

	static UE_API void EncodeVector2D(TArrayView<float> Values, int32 DataOffset, const FVector2D& Vector2D);
	static UE_API FVector2D DecodeVector2D(TConstArrayView<float> Values, int32 DataOffset);

	static UE_API void EncodeFloat(TArrayView<float> Values, int32 DataOffset, const float Value);
	static UE_API float DecodeFloat(TConstArrayView<float> Values, int32 DataOffset);

	static UE_API void Copy(TArrayView<float> Values, int32 DataOffset, int32 DataCardinality, TConstArrayView<float> OriginValues);
};

} // namespace UE::PoseSearch

#undef UE_API
