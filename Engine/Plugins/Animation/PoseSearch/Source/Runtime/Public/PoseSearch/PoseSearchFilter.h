// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DrawDebugHelpers.h"
#include "PoseSearch/PoseSearchIndex.h"
#include "PoseSearchFilter.generated.h"

#define UE_API POSESEARCH_API

class UPoseSearchDatabase;
class UPoseSearchFeatureChannel;
class UPoseSearchSchema;

namespace UE::PoseSearch
{
	struct FDebugDrawParams;
	struct FPoseMetadata;
	struct FSearchContext;
} // namespace UE::PoseSearch

class POSESEARCH_API UE_DEPRECATED(5.8, "Use FPoseSearchFilter instead") IPoseSearchFilter
{
public:
	virtual ~IPoseSearchFilter() {}

	// if true this filter will be evaluated
	UE_DEPRECATED(5.8, "Use FPoseSearchFilter::GetRequiredDataSize instead")
	virtual bool IsFilterActive() const { return false; }
	
	// if it returns false the pose candidate will be discarded
	UE_DEPRECATED(5.8, "Use FPoseSearchFilter::IsFilterValid instead")
	virtual bool IsFilterValid(TConstArrayView<float> PoseValues, TConstArrayView<float> QueryValues, int32 PoseIdx, const UE::PoseSearch::FPoseMetadata& Metadata) const { return true; }
};

USTRUCT(Experimental)
struct FPoseSearchFilter
{
	GENERATED_BODY()

	enum { InvalidFilterDataSize = -1 };
	enum { DataAlignment = 16 };

	virtual ~FPoseSearchFilter() {}

	// returns require data size in bytes for this filter. InvalidFilterDataSize if the filter is NOT active
	virtual int32 GetRequiredDataSize(const UPoseSearchFeatureChannel* Channel) const { return InvalidFilterDataSize; }

	// called to initialize Data with this FPoseSearchFilter data  only if GetRequiredDataSize returns >= 0. Data.Num() == GetRequiredDataSize()
	// returns successful it returns true, or else DeinitializeData will be called right away and IsFilterValid will NOT be called
	// it is safe to use Data to perform a new inplace in InitializeData and call the destructor in DeinitializeData.. but not recommended
	virtual bool InitializeData(TArrayView<uint8> Data, const UPoseSearchFeatureChannel* Channel, TConstArrayView<float> QueryValues, UE::PoseSearch::FSearchContext& SearchContext) const { return true; }

	// called to deinitialize Data with this FPoseSearchFilter data  only if GetRequiredDataSize returns >= 0. Data.Num() == GetRequiredDataSize()
	// it is safe to use Data to perform a new inplace in InitializeData and call the destructor in DeinitializeData.. but not recommended
	virtual void DeinitializeData(TArrayView<uint8> Data, const UPoseSearchFeatureChannel* Channel, UE::PoseSearch::FSearchContext& SearchContext) const {}
	
	// called for EVERY evaluated pose (be sure it's as fast as possible). if it returns false the pose candidate will be discarded
	virtual bool IsFilterValid(TConstArrayView<float> PoseValues, TConstArrayView<float> QueryValues, int32 PoseIdx, const UE::PoseSearch::FPoseMetadata& Metadata, TConstArrayView<uint8> Data, const UPoseSearchFeatureChannel* Channel) const { return true; }

#if ENABLE_DRAW_DEBUG
	virtual void DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseVector, const UPoseSearchFeatureChannel* Channel) const {}
#endif // ENABLE_DRAW_DEBUG
};

// debug FPoseSearchFilter to test InitializeData / DeinitializeData by padding the data by 1 byte
USTRUCT(MinimalAPI, Experimental)
struct FPoseSearchFilterPadding : public FPoseSearchFilter
{
	GENERATED_BODY()

	UE_API virtual int32 GetRequiredDataSize(const UPoseSearchFeatureChannel* Channel) const override;
};

// FPoseSearchFilterMaxPosition is invalid if the query for the associated UPoseSearchFeatureChannel_Position is too far from the candidate pose
USTRUCT(MinimalAPI, Experimental)
struct FPoseSearchFilterMaxPosition : public FPoseSearchFilter
{
	GENERATED_BODY()

	UE_API virtual int32 GetRequiredDataSize(const UPoseSearchFeatureChannel* Channel) const override;
	UE_API virtual bool InitializeData(TArrayView<uint8> Data, const UPoseSearchFeatureChannel* Channel, TConstArrayView<float> QueryValues, UE::PoseSearch::FSearchContext& SearchContext) const override;
	UE_API virtual void DeinitializeData(TArrayView<uint8> Data, const UPoseSearchFeatureChannel* Channel, UE::PoseSearch::FSearchContext& SearchContext) const override;
	UE_API virtual bool IsFilterValid(TConstArrayView<float> PoseValues, TConstArrayView<float> QueryValues, int32 PoseIdx, const UE::PoseSearch::FPoseMetadata& Metadata, TConstArrayView<uint8> Data, const UPoseSearchFeatureChannel* Channel) const override;
	
#if ENABLE_DRAW_DEBUG
	UE_API virtual void DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseValues, const UPoseSearchFeatureChannel* Channel) const override;
#endif // ENABLE_DRAW_DEBUG

	// During pose selection if the squared distance between query versus candidate poses for the associated 
	// UPoseSearchFeatureChannel_Position is greater than MaxPositionDistanceSquared the candidate will be discarded.
	// The filtering will be enabled only for MaxPositionDistanceSquared != 0
	UPROPERTY(EditAnywhere, Category = "Experimental|Filter", meta = (ClampMin = "0", UIMin = "0"))
	float MaxPositionDistanceSquared = 0.f;

private:
	struct FData
	{
		FVector Query;
	};
	static_assert(alignof(FData) <= DataAlignment);
};

// FPoseSearchFilterWedge is invalid if the query for the associated UPoseSearchFeatureChannel_Position doesn't land in the defined wedge relative to the trajectory
USTRUCT(MinimalAPI, Experimental)
struct FPoseSearchFilterWedge : public FPoseSearchFilter
{
	GENERATED_BODY()

	UE_API virtual int32 GetRequiredDataSize(const UPoseSearchFeatureChannel* Channel) const override;
	UE_API virtual bool InitializeData(TArrayView<uint8> Data, const UPoseSearchFeatureChannel* Channel, TConstArrayView<float> QueryValues, UE::PoseSearch::FSearchContext& SearchContext) const override;
	UE_API virtual void DeinitializeData(TArrayView<uint8> Data, const UPoseSearchFeatureChannel* Channel, UE::PoseSearch::FSearchContext& SearchContext) const override;
	UE_API virtual bool IsFilterValid(TConstArrayView<float> PoseValues, TConstArrayView<float> QueryValues, int32 PoseIdx, const UE::PoseSearch::FPoseMetadata& Metadata, TConstArrayView<uint8> Data, const UPoseSearchFeatureChannel* Channel) const override;

#if ENABLE_DRAW_DEBUG
	UE_API virtual void DebugDraw(const UE::PoseSearch::FDebugDrawParams& DrawParams, TConstArrayView<float> PoseValues, const UPoseSearchFeatureChannel* Channel) const override;
#endif // ENABLE_DRAW_DEBUG

	UPROPERTY(EditAnywhere, Category = "Experimental|Filter", meta = (ClampToMinMaxLimits))
	FFloatInterval RadiusOffsets = FFloatInterval(-100.f, 100.f);
	
	// wedge filter width in degrees
	UPROPERTY(EditAnywhere, Category = "Experimental|Filter", meta = (ClampMin = "0"))
	float WedgeWidthInDegrees = 70.f;

private:
	struct FData
	{
		FVector NormalizedQuery;
		float InnerRadius;
		float OuterRadius;
		float DotWedgeWidth;
	};
	static_assert(alignof(FData) <= DataAlignment);
};

namespace UE::PoseSearch
{
// Experimental, this feature might be removed without warning, not for production use
struct FSelectableAssetIdxFilter : public FPoseSearchFilter
{
	const FSelectableAssetIdxFilter& Init(TConstArrayView<int32> InSelectableAssetIdxFilter)
	{
		check(Algo::IsSorted(InSelectableAssetIdxFilter));
		SelectableAssetIdxFilter = InSelectableAssetIdxFilter;
		return *this;
	}

	virtual int32 GetRequiredDataSize(const UPoseSearchFeatureChannel* Channel) const override
	{
		return SelectableAssetIdxFilter.IsEmpty() ? InvalidFilterDataSize : 0;
	}

	virtual bool IsFilterValid(TConstArrayView<float> PoseValues, TConstArrayView<float> QueryValues, int32 PoseIdx, const UE::PoseSearch::FPoseMetadata& Metadata, TConstArrayView<uint8> Data, const UPoseSearchFeatureChannel* Channel) const override
	{
		return Algo::BinarySearch(SelectableAssetIdxFilter, int32(Metadata.GetAssetIndex())) != INDEX_NONE;
	}

	TConstArrayView<int32> SelectableAssetIdxFilter;
};

// Experimental, this feature might be removed without warning, not for production use
struct FNonSelectableIdxFilter : public FPoseSearchFilter
{
	const FNonSelectableIdxFilter& Init(TConstArrayView<int32> InNonSelectableIdx)
	{
		check(Algo::IsSorted(InNonSelectableIdx));
		NonSelectableIdx = InNonSelectableIdx;
		return *this;
	}

	virtual int32 GetRequiredDataSize(const UPoseSearchFeatureChannel* Channel) const override
	{
		return NonSelectableIdx.IsEmpty() ? InvalidFilterDataSize : 0;
	}
	virtual bool IsFilterValid(TConstArrayView<float> PoseValues, TConstArrayView<float> QueryValues, int32 PoseIdx, const UE::PoseSearch::FPoseMetadata& Metadata, TConstArrayView<uint8> Data, const UPoseSearchFeatureChannel* Channel) const override
	{
		return Algo::BinarySearch(NonSelectableIdx, PoseIdx) == INDEX_NONE;
	}

	TConstArrayView<int32> NonSelectableIdx;
};

// Experimental, this feature might be removed without warning, not for production use
struct FBlockTransitionFilter : public FPoseSearchFilter
{
	virtual int32 GetRequiredDataSize(const UPoseSearchFeatureChannel* Channel) const override
	{
		return 0;
	}

	virtual bool IsFilterValid(TConstArrayView<float> PoseValues, TConstArrayView<float> QueryValues, int32 PoseIdx, const UE::PoseSearch::FPoseMetadata& Metadata, TConstArrayView<uint8> Data, const UPoseSearchFeatureChannel* Channel) const override
	{
		return !Metadata.IsBlockTransition();
	}
};

// Experimental, this feature might be removed without warning, not for production use
struct FSearchFilters
{
	FSearchFilters(const UPoseSearchSchema* Schema, TConstArrayView<int32> NonSelectableIdx, TConstArrayView<int32> SelectableAssetIdx,
		bool bAddBlockTransitionFilter, TConstArrayView<float> QueryValues, FSearchContext& InSearchContext);
	~FSearchFilters();

	// @todo: template this with bAlignedAndPadded to be able to use faster ComparePoses
	bool AreFiltersValid(const FSearchIndex& SearchIndex, TConstArrayView<float> PoseValues, TConstArrayView<float> QueryValues, TConstArrayView<float> DynamicWeightsSqrt, int32 PoseIdx
#if UE_POSE_SEARCH_TRACE_ENABLED
		, float ContinuingPoseCostAddend, float ContinuingInteractionCostAddend, float ContinuingContextInteractionCostAddend, const UPoseSearchDatabase* Database
#endif // UE_POSE_SEARCH_TRACE_ENABLED
		) const;

	FSearchContext& GetSearchContext() const { return SearchContext; }

private:
		
	FSearchContext& SearchContext;
	FNonSelectableIdxFilter NonSelectableIdxFilter;
	FSelectableAssetIdxFilter SelectableAssetIdxFilter;
	FBlockTransitionFilter BlockTransitionFilter;
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TArray<const IPoseSearchFilter*, TInlineAllocator<64, TMemStackAllocator<>>> Filters_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	struct FFilterContext
	{
		FFilterContext(const FPoseSearchFilter* InFilter, const UPoseSearchFeatureChannel* InChannel, TArrayView<uint8> InData)
			: Filter(InFilter)
			, Channel(InChannel)
			, Data(InData)
		{
		}

		const FPoseSearchFilter* Filter;
		const UPoseSearchFeatureChannel* Channel;
		TArrayView<uint8> Data;
	};
	TArray<FFilterContext, TInlineAllocator<64, TMemStackAllocator<>>> FilterContexts;
	TArray<uint8, TMemStackAllocator<FPoseSearchFilter::DataAlignment>> FiltersData;
};

} // namespace UE::PoseSearch

#undef UE_API
