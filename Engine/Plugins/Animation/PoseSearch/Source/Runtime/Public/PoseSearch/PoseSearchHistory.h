// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneIndices.h"
#include "BonePose.h"
#include "Containers/RingBuffer.h"
#include "DrawDebugHelpers.h"
#include "Animation/AnimCurveTypes.h"
#include "Misc/MTTransactionallySafeAccessDetector.h"
#include "PoseSearch/PoseHistoryProvider.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "PoseSearch/PoseSearchTrajectoryLibrary.h"
#include "Animation/TrajectoryTypes.h"
#include "UObject/ObjectKey.h"

#include "PoseSearchHistory.generated.h"

#define UE_API POSESEARCH_API

class UAnimationAsset;
struct FAnimInstanceProxy;
class USkeleton;
class UWorld;

namespace UE::PoseSearch
{

struct FSearchResult;
typedef uint16 FComponentSpaceTransformIndex;
typedef TPair<FBoneIndexType, FComponentSpaceTransformIndex> FBoneToTransformPair;
typedef TMap<FBoneIndexType, FComponentSpaceTransformIndex> FBoneToTransformMap;

struct FHistoricalPoseIndex
{
	bool operator==(const FHistoricalPoseIndex& Index) const
	{
		return PoseIndex == Index.PoseIndex && DatabaseKey == Index.DatabaseKey;
	}

	friend inline uint32 GetTypeHash(const FHistoricalPoseIndex& Index)
	{
		return HashCombineFast(::GetTypeHash(Index.PoseIndex), GetTypeHash(Index.DatabaseKey));
	}

	int32 PoseIndex = INDEX_NONE;
	FObjectKey DatabaseKey;
};

struct FPoseIndicesHistory
{
	UE_INTERNAL UE_API void Update(const FSearchResult& SearchResult, float DeltaTime, float MaxTime);
	void Reset() { IndexToTime.Reset(); }
	bool operator==(const FPoseIndicesHistory& Other) const;
		
	TMap<FHistoricalPoseIndex, float> IndexToTime;
};

struct IComponentSpacePoseProvider
{
	virtual ~IComponentSpacePoseProvider() = default;
	virtual FTransform CalculateComponentSpaceTransform(const FSkeletonPoseBoneIndex SkeletonBoneIdx) = 0;
	virtual const USkeleton* GetSkeletonAsset() const = 0;
};

struct FComponentSpacePoseProvider : public IComponentSpacePoseProvider
{
	POSESEARCH_API FComponentSpacePoseProvider(FCSPose<FCompactPose>& InComponentSpacePose);

	virtual FTransform CalculateComponentSpaceTransform(const FSkeletonPoseBoneIndex SkeletonBoneIdx) override;
	virtual const USkeleton* GetSkeletonAsset() const override;

private:
	FCSPose<FCompactPose>& ComponentSpacePose;
};

struct FAIPComponentSpacePoseProvider : public IComponentSpacePoseProvider
{
	POSESEARCH_API FAIPComponentSpacePoseProvider(const FAnimInstanceProxy* InAnimInstanceProxy);

	virtual FTransform CalculateComponentSpaceTransform(const FSkeletonPoseBoneIndex SkeletonBoneIdx) override;
	virtual const USkeleton* GetSkeletonAsset() const override;

private:
	FCSPose<FCompactPose> ComponentSpacePose;
};

// Experimental, this feature might be removed without warning, not for production use
struct UE_EXPERIMENTAL(5.8, "This API is experimental and might be removed without warning, not for production use.") FSKMCComponentSpacePoseProvider : public IComponentSpacePoseProvider
{
	POSESEARCH_API FSKMCComponentSpacePoseProvider(const USkeletalMeshComponent* InSkeletalMeshComponent);

	virtual FTransform CalculateComponentSpaceTransform(const FSkeletonPoseBoneIndex SkeletonBoneIdx) override;
	virtual const USkeleton* GetSkeletonAsset() const override;

private:
	USkeleton* Skeleton = nullptr;
	const TArray<int32>* SkeletonToMeshTable = nullptr;
	const USkeletalMeshComponent* SkeletalMeshComponent = nullptr;
};

struct FPoseHistoryEntry
{
	// collected bones transforms in component space
	TArray<FQuat4f> ComponentSpaceRotations;
	TArray<FVector> ComponentSpacePositions;
	TArray<FVector3f> ComponentSpaceScales;
	TArray<float> CurveValues;
	float AccumulatedSeconds = 0.f;

#if WITH_EDITORONLY_DATA
	FTransform DebugMeshTransform;
#endif // WITH_EDITORONLY_DATA

	void Update(float Time, IComponentSpacePoseProvider& ComponentSpacePoseProvider, const FBoneToTransformMap& BoneToTransformMap, bool bStoreScales, const FBlendedCurve& Curves, const TConstArrayView<FName>& CollectedCurves
#if WITH_EDITORONLY_DATA
		, const FTransform& InDebugMeshTransform = FTransform::Identity
#endif // WITH_EDITORONLY_DATA
	);

	POSESEARCH_API void SetNum(int32 Num, bool bStoreScales);
	int32 Num() const;

	POSESEARCH_API void SetComponentSpaceTransform(int32 Index, const FTransform& Transform);
	FTransform GetComponentSpaceTransform(int32 Index) const;
	float GetCurveValue(int32 Index) const;
};
FArchive& operator<<(FArchive& Ar, FPoseHistoryEntry& Entry);

} // namespace UE::PoseSearch

// Experimental, this feature might be removed without warning, not for production use
USTRUCT(Experimental, BlueprintType)
struct FPoseSearchHistory
{
	GENERATED_BODY()

	UE_API void Initialize(int32 InNumPoses, float InSamplingInterval);
	UE_API void EvaluateComponentSpace(float DeltaTime, UE::PoseSearch::IComponentSpacePoseProvider& ComponentSpacePoseProvider, bool bStoreScales,
		float RootBoneRecoveryTime, float RootBoneTranslationRecoveryRatio, float RootBoneRotationRecoveryRatio,
		bool bNeedsReset, bool bCacheBones, const TArray<FBoneIndexType>& RequiredBones, 
		const FBlendedCurve& Curves = FBlendedCurve(), const TConstArrayView<FName>& InCollectedCurves = TConstArrayView<FName>()
#if WITH_EDITORONLY_DATA
		, const FTransform& InDebugMeshTransform = FTransform::Identity
#endif //WITH_EDITORONLY_DATA
	);

	UE_API bool GetTransformAtTime(float Time, FTransform& OutBoneTransform, const USkeleton* BoneIndexSkeleton = nullptr, FBoneIndexType BoneIndexType = UE::PoseSearch::RootBoneIndexType, FBoneIndexType ReferenceBoneIndexType = UE::PoseSearch::ComponentSpaceIndexType, bool bExtrapolate = false) const;
	UE_API bool GetCurveValueAtTime(float Time, const FName& CurveName, float& outcurvevalue, bool bExtrapolate = false) const;
	UE_API void SetTrajectory(const FTransformTrajectory& InTrajectory, float InTrajectorySpeedMultiplier = 1.f);
	UE_API void GenerateTrajectory(const UObject* AnimContext, float DeltaTime);
	UE_API float GetTrajectorySpeedMultiplier() const { return TrajectorySpeedMultiplier; }
	UE_API bool IsEmpty() const { return Entries.IsEmpty(); }
	const UE::PoseSearch::FBoneToTransformMap& GetBoneToTransformMap() const { return BoneToTransformMap; }
	const TConstArrayView<FName> GetCollectedCurves() const { return MakeConstArrayView(CollectedCurves); }

	int32 GetNumEntries() const { return Entries.Num(); }
	const UE::PoseSearch::FPoseHistoryEntry& GetEntry(int32 EntryIndex) const { return Entries[EntryIndex]; }
	const UE::PoseSearch::FPoseIndicesHistory* GetPoseIndicesHistory() const { return &PoseIndicesHistory; }

	int32 GetMaxNumPoses() const { return MaxNumPoses; }
	float GetSamplingInterval() const { return SamplingInterval; }
	// Experimental, this feature might be removed without warning, not for production use
	UE_EXPERIMENTAL(5.8, "This API is experimental and might be removed without warning, not for production use.")
	void SetSamplingInterval(float InSamplingInterval) { SamplingInterval = InSamplingInterval; }

	const FTransformTrajectory& GetTrajectory() const { return Trajectory; }
	FTransformTrajectory& EditTrajectory() { return Trajectory; }

private:
	UPROPERTY(EditAnywhere, Category=Settings, meta=(AllowPrivateAccess=true))
	FTransformTrajectory Trajectory;

	// max number of stored poses in this pose history
	UPROPERTY(EditAnywhere, Category=Settings, meta=(AllowPrivateAccess=true))
	int32 MaxNumPoses = 2;

	// frequency of sampling of poses in this pose history. negative numbers will store a new pose on every update
	UPROPERTY(EditAnywhere, Category=Settings, meta=(AllowPrivateAccess=true))
	float SamplingInterval = -1.f;
	
	UPROPERTY(EditAnywhere, Category=Settings, meta=(AllowPrivateAccess=true))
	float TrajectorySpeedMultiplier = 1.f;

	// skeleton from the last Update, to keep tracking skeleton changes, and support compatible skeletons
	TWeakObjectPtr<const USkeleton> LastUpdateSkeleton;

	// Map from skeleton bone indexes (encoded as FBoneIndexType) to internal pose history transform index (FComponentSpaceTransformIndex).
	// If Empty all the bones get collected
	
	// @todo: make it a UPROPERTY
	// typedef TMap<FBoneIndexType, FComponentSpaceTransformIndex> FBoneToTransformMap;
	UE::PoseSearch::FBoneToTransformMap BoneToTransformMap;
		
	// list of curves that we want to collect into our history.
	TArray<FName> CollectedCurves;

	// GetTypeHash for BoneToTransformMap
	uint32 BoneToTransformMapTypeHash = 0;

	// @todo: make it a UPROPERTY
	// ring buffer of collected bones
	TRingBuffer<UE::PoseSearch::FPoseHistoryEntry> Entries;

	// @todo: make it a UPROPERTY
	// data structure containing the previously played pose indexes with their respective datababases
	UE::PoseSearch::FPoseIndicesHistory PoseIndicesHistory;
};

namespace UE::PoseSearch
{

struct IPoseHistory : public TSharedFromThis<IPoseHistory, ESPMode::ThreadSafe>
{
	virtual ~IPoseHistory() = default;
	
	// returns transform of the the skeleton bone index BoneIndexType (encoded as FBoneIndexType) relative to the transform of ReferenceBoneIndexType:
	// if ReferenceBoneIndexType is 0 (RootBoneIndexType), OutBoneTransform is in root bone space
	// if ReferenceBoneIndexType is FBoneIndexType(-1) (ComponentSpaceIndexType), OutBoneTransform is in component space
	// if ReferenceBoneIndexType is FBoneIndexType(-2) (WorldSpaceIndexType), OutBoneTransform is in world space
	virtual bool GetTransformAtTime(float Time, FTransform& OutBoneTransform, const USkeleton* BoneIndexSkeleton = nullptr, FBoneIndexType BoneIndexType = RootBoneIndexType, FBoneIndexType ReferenceBoneIndexType = ComponentSpaceIndexType, bool bExtrapolate = false) const = 0;
	// @todo: consider consolidating into a (templated?) get X value at time once we add custom attributes to pose history.
	virtual bool GetCurveValueAtTime(float Time, const FName& CurveName, float& OutCurveValue, bool bExtrapolate = false) const = 0;
	
	// @note for review: Here we are just changing the return type to force users to update immediately. Will cause project to not compile for users who override the pure virtual func getter, not sure how we can gracefully deprecate this.
	// Another option is to just have a new function signature GetTransformTrajectory() and add a final tag to the GetTrajectory() to also force users to upgrade.
	// Lastly, to me this seems like a struct not many licensees would inherent from tbh.
	virtual const FTransformTrajectory& GetTrajectory() const = 0;
	
	// Experimental, this feature might be removed without warning, not for production use
	virtual void SetTrajectory(const FTransformTrajectory& InTrajectory, float InTrajectorySpeedMultiplier = 1.f) = 0;
	// Experimental, this feature might be removed without warning, not for production use
	virtual void GenerateTrajectory(const UObject* AnimContext, float DeltaTime) = 0;
	// @todo: deprecate this API. TrajectorySpeedMultiplier should be a global query scaling value passed as input parameter of FSearchContext during config BuildQuery
	virtual float GetTrajectorySpeedMultiplier() const = 0;
	virtual bool IsEmpty() const = 0;

	virtual const FBoneToTransformMap& GetBoneToTransformMap() const = 0;
	virtual const TConstArrayView<FName> GetCollectedCurves() const = 0;
	virtual int32 GetNumEntries() const = 0;
	virtual const FPoseHistoryEntry& GetEntry(int32 EntryIndex) const = 0;
	virtual const FPoseIndicesHistory* GetPoseIndicesHistory() const { return nullptr; }

#if ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
	virtual void DebugDraw(const UWorld* World, FColor Color) const = 0;
	virtual void DebugDraw(FAnimInstanceProxy& AnimInstanceProxy, FColor Color) const = 0;
	UE_API virtual void DebugDraw(FAnimInstanceProxy& AnimInstanceProxy, FColor Color, float Time, float PointSize = 6.f, bool bExtrapolate = false) const;
#endif
};

struct FArchivedPoseHistory : public IPoseHistory
{
	UE_API void InitFrom(const IPoseHistory* PoseHistory);

	// IPoseHistory interface
	UE_API virtual bool GetTransformAtTime(float Time, FTransform& OutBoneTransform, const USkeleton* BoneIndexSkeleton = nullptr, FBoneIndexType BoneIndexType = RootBoneIndexType, FBoneIndexType ReferenceBoneIndexType = ComponentSpaceIndexType, bool bExtrapolate = false) const override;
	UE_API virtual bool GetCurveValueAtTime(float Time, const FName& CurveName, float& outcurvevalue, bool bExtrapolate = false) const override;
	virtual const FTransformTrajectory& GetTrajectory() const override { return Trajectory; }
	// Experimental, this feature might be removed without warning, not for production use
	virtual void SetTrajectory(const FTransformTrajectory& InTrajectory, float InTrajectorySpeedMultiplier = 1.f) override { Trajectory = InTrajectory; }
	// Experimental, this feature might be removed without warning, not for production use
	virtual void GenerateTrajectory(const UObject* AnimContext, float DeltaTime) override { checkNoEntry(); }
	virtual float GetTrajectorySpeedMultiplier() const override { return 1.f; }
	virtual bool IsEmpty() const override { return Entries.IsEmpty(); }
	virtual const FBoneToTransformMap& GetBoneToTransformMap() const override { return BoneToTransformMap; }
	virtual const TConstArrayView<FName> GetCollectedCurves() const override { return CollectedCurves; }
	virtual int32 GetNumEntries() const override { return Entries.Num(); }
	virtual const FPoseHistoryEntry& GetEntry(int32 EntryIndex) const override { return Entries[EntryIndex]; }

#if ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
	UE_API virtual void DebugDraw(const UWorld* World, FColor Color) const override;
	virtual void DebugDraw(FAnimInstanceProxy& AnimInstanceProxy, FColor Color) const override { unimplemented(); }
#endif
	// End of IPoseHistory interface

	FBoneToTransformMap BoneToTransformMap;
	// @todo: Make this a map if this is expected to be big.
	TArray<FName> CollectedCurves;
	TArray<FPoseHistoryEntry> Entries;
	
	FTransformTrajectory Trajectory;
};
FArchive& operator<<(FArchive& Ar, FArchivedPoseHistory& Entry);

struct FPoseHistory : public IPoseHistory
{
	FPoseHistory() = default;
	virtual ~FPoseHistory() override = default;

	FPoseHistory(const FPoseHistory& Other);
	UE_API FPoseHistory(FPoseHistory&& Other);
	UE_API FPoseHistory& operator=(const FPoseHistory& Other);
	UE_API FPoseHistory& operator=(FPoseHistory&& Other);

	// @todo: deprecate and delete this method in favor of providing the trajectory via SetTrajectory
	UE_API void GenerateTrajectory(const UObject* AnimContext, float DeltaTime, const FPoseSearchTrajectoryData& TrajectoryData, const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling);
	
	UE_API void Initialize_AnyThread(int32 InNumPoses, float InSamplingInterval);
	UE_API void EvaluateComponentSpace_AnyThread(float DeltaTime, IComponentSpacePoseProvider& ComponentSpacePoseProvider, bool bStoreScales,
		float RootBoneRecoveryTime, float RootBoneTranslationRecoveryRatio, float RootBoneRotationRecoveryRatio,
		bool bNeedsReset, bool bCacheBones, const TArray<FBoneIndexType>& RequiredBones, 
		const FBlendedCurve& Curves = FBlendedCurve(), const TConstArrayView<FName>& InCollectedCurves = TConstArrayView<FName>()
#if WITH_EDITORONLY_DATA
		, const FTransform& InDebugMeshTransform = FTransform::Identity
#endif //WITH_EDITORONLY_DATA
	);

	// IPoseHistory interface
	UE_API virtual bool GetTransformAtTime(float Time, FTransform& OutBoneTransform, const USkeleton* BoneIndexSkeleton = nullptr, FBoneIndexType BoneIndexType = RootBoneIndexType, FBoneIndexType ReferenceBoneIndexType = ComponentSpaceIndexType, bool bExtrapolate = false) const override;
	UE_API virtual bool GetCurveValueAtTime(float Time, const FName& CurveName, float& outcurvevalue, bool bExtrapolate = false) const override;
	UE_API virtual const FTransformTrajectory& GetTrajectory() const override;
	// Experimental, this feature might be removed without warning, not for production use
	UE_API virtual void SetTrajectory(const FTransformTrajectory& InTrajectory, float InTrajectorySpeedMultiplier = 1.f) override;
	// Experimental, this feature might be removed without warning, not for production use
	UE_API virtual void GenerateTrajectory(const UObject* AnimContext, float DeltaTime) override;
	UE_API virtual float GetTrajectorySpeedMultiplier() const override;
	UE_API virtual bool IsEmpty() const override;
	UE_API virtual const FBoneToTransformMap& GetBoneToTransformMap() const override;
	UE_API virtual const TConstArrayView<FName> GetCollectedCurves() const override;

	UE_API virtual int32 GetNumEntries() const override;
	UE_API virtual const FPoseHistoryEntry& GetEntry(int32 EntryIndex) const override;
	UE_API virtual const FPoseIndicesHistory* GetPoseIndicesHistory() const override;

#if ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
	virtual void DebugDraw(const UWorld* World, FColor Color) const override { unimplemented(); }
	UE_API virtual void DebugDraw(FAnimInstanceProxy& AnimInstanceProxy, FColor Color) const override;
#endif
	// End of IPoseHistory interface

	UE_API int32 GetMaxNumPoses() const;
	UE_API float GetSamplingInterval() const;
	// Experimental, this feature might be removed without warning, not for production use
	UE_EXPERIMENTAL(5.8, "This API is experimental and might be removed without warning, not for production use.")
	UE_API void SetSamplingInterval(float InSamplingInterval);

	// Experimental, this feature might be removed without warning, not for production use
	UE_API const FPoseSearchHistory& GetPoseSearchHistory() const;
	// Experimental, this feature might be removed without warning, not for production use
	UE_API FPoseSearchHistory& EditPoseSearchHistory();

private:
	FPoseSearchHistory PoseSearchHistory;
	FPoseSearchTrajectoryData::FState TrajectoryDataState;
	UE_MT_DECLARE_TS_RW_ACCESS_DETECTOR(PoseDataThreadSafeCounter);
};

// pose history struct that can generate a trajectory properly by calling it's IPoseHistory::GenerateTrajectory API (something other IPoseHistory don't do fully)
struct FGenerateTrajectoryPoseHistory : public FPoseHistory
{
	bool bGenerateTrajectory = false;
	bool bIsTrajectoryGeneratedBeforePreUpdate = false;
	FPoseSearchTrajectoryData::FSampling TrajectoryDataSampling;
	FPoseSearchTrajectoryData TrajectoryData;

	UE_API virtual void GenerateTrajectory(const UObject* AnimContext, float DeltaTime) override;
};

struct FMemStackPoseHistory : public IPoseHistory
{
	UE_API void Init(const IPoseHistory* InPoseHistory);

	// IPoseHistory interface
	UE_API virtual bool GetTransformAtTime(float Time, FTransform& OutBoneTransform, const USkeleton* BoneIndexSkeleton = nullptr, FBoneIndexType BoneIndexType = RootBoneIndexType, FBoneIndexType ReferenceBoneIndexType = ComponentSpaceIndexType, bool bExtrapolate = false) const override;
	UE_API virtual bool GetCurveValueAtTime(float Time, const FName& CurveName, float& outcurvevalue, bool bExtrapolate = false) const override;
	virtual const FTransformTrajectory& GetTrajectory() const override { check(PoseHistory); return PoseHistory->GetTrajectory(); }
	// Experimental, this feature might be removed without warning, not for production use
	UE_API virtual void SetTrajectory(const FTransformTrajectory& InTrajectory, float InTrajectorySpeedMultiplier = 1.f) override;
	// Experimental, this feature might be removed without warning, not for production use
	UE_API virtual void GenerateTrajectory(const UObject* AnimContext, float DeltaTime) override;
	virtual float GetTrajectorySpeedMultiplier() const override { check(PoseHistory); return PoseHistory->GetTrajectorySpeedMultiplier(); }
	virtual bool IsEmpty() const override { check(PoseHistory); return PoseHistory->IsEmpty() && FutureEntries.IsEmpty(); }
	virtual const FBoneToTransformMap& GetBoneToTransformMap() const override { check(PoseHistory); return PoseHistory->GetBoneToTransformMap(); }
	virtual const TConstArrayView<FName> GetCollectedCurves() const override { check(PoseHistory); return PoseHistory->GetCollectedCurves(); }
	
	UE_API virtual int32 GetNumEntries() const override;
	UE_API virtual const FPoseHistoryEntry& GetEntry(int32 EntryIndex) const override;
	UE_API virtual const FPoseIndicesHistory* GetPoseIndicesHistory() const override;
	
#if ENABLE_DRAW_DEBUG && ENABLE_ANIM_DEBUG
	virtual void DebugDraw(const UWorld* World, FColor Color) const override { unimplemented(); }
	UE_API virtual void DebugDraw(FAnimInstanceProxy& AnimInstanceProxy, FColor Color) const override;
#endif
	// End of IPoseHistory interface

	UE_API void AddFutureRootBone(float Time, const FTransform& FutureRootBoneTransform, bool bStoreScales);
	UE_API void AddFuturePose(float Time, IComponentSpacePoseProvider& ComponentSpacePoseProvider, const FBlendedCurve& Curves = FBlendedCurve()
#if WITH_EDITORONLY_DATA
		, const FTransform& InDebugMeshTransform = FTransform::Identity
#endif //WITH_EDITORONLY_DATA
	);
	
	// Experimental, this feature might be removed without warning, not for production use
	// extracting a pose from AnimationAsset at AnimationTime and  (using BlendParameters in case it's a blend space) and placing it in the FMemStackPoseHistory at IntervalTime seconds
	// if FiniteDeltaTime is > FiniteDeltaTime a second pose will be extracted at AnimationTime - FiniteDeltaTime and placed at IntervalTime - FiniteDeltaTime (for MM to be able to calculate velocities)
	UE_API void ExtractAndAddFuturePoses(const UAnimationAsset* AnimationAsset, float AnimationTime, float FiniteDeltaTime, const FVector& BlendParameters, float IntervalTime, const USkeleton* OverrideSkeleton = nullptr, bool bUseRefPoseRootBone = false
#if WITH_EDITORONLY_DATA
		, const FTransform& InDebugMeshTransform = FTransform::Identity
#endif //WITH_EDITORONLY_DATA
	);

	const IPoseHistory* GetThisOrPoseHistory() const { return FutureEntries.IsEmpty() ? PoseHistory : this; }

private:
	const IPoseHistory* PoseHistory = nullptr;
	TArray<FPoseHistoryEntry, TInlineAllocator<4, TMemStackAllocator<>>> FutureEntries;
};

} // namespace UE::PoseSearch

USTRUCT(BlueprintType)
struct FPoseHistoryReference
{
	GENERATED_BODY()
	TSharedPtr<UE::PoseSearch::IPoseHistory> PoseHistory;
};



#undef UE_API
