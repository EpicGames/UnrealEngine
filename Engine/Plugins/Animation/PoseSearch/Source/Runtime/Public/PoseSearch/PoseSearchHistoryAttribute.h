// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearchHistory.h"
#include "Animation/AttributesContainer.h"
#include "Animation/AttributeTraits.h"
#include "PoseSearchHistoryAttribute.generated.h"

namespace UE::PoseSearch
{
	POSESEARCH_API extern const UE::Anim::FAttributeId PoseHistoryAttributeId;
}

/** Attribute type containing a const pointer to pose history for use by downstream systems. Uses a TWeakObjectPtr for scope checks.*/
USTRUCT(Experimental)
struct FPoseHistoryAnimationAttribute
{
	GENERATED_BODY()

	const UE::PoseSearch::IPoseHistory* PoseHistory = nullptr;
	TWeakObjectPtr<const UObject> ScopeObject;

	bool IsValid() const
	{
		return ScopeObject.IsValid() && PoseHistory != nullptr;
	}
};

inline POSESEARCH_API uint32 GetTypeHash(const FPoseHistoryAnimationAttribute& Key)
{
	return HashCombine(GetTypeHash(Key.ScopeObject), GetTypeHash(Key.PoseHistory));
}

namespace UE::Anim
{
	/** Pose history attribute is not blend-able by default */
	template<>
	struct TAttributeTypeTraits<FPoseHistoryAnimationAttribute> : public TAttributeTypeTraitsBase<FPoseHistoryAnimationAttribute>
	{
		enum
		{
			IsBlendable = false,
		};
	};
}
