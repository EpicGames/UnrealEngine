// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearchHistory.h"
#include "Animation/AnimExecutionContext.h"
#include "Animation/AnimNodeReference.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "PoseSearchHistoryCollectorAnimNodeLibrary.generated.h"

#define UE_API POSESEARCH_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS
struct FPoseSearchQueryTrajectory;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
struct FTransformTrajectory;
struct FAnimNode_PoseSearchHistoryCollector;
class USkeleton;

USTRUCT(BlueprintType)
struct FPoseSearchHistoryCollectorAnimNodeReference : public FAnimNodeReference
{
	GENERATED_BODY()

	typedef FAnimNode_PoseSearchHistoryCollector FInternalNodeType;
};

// Exposes operations that can be run on a Pose History node via Anim Node Functions such as "On Become Relevant" and "On Update".
UCLASS(MinimalAPI)
class UPoseSearchHistoryCollectorAnimNodeLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Get a Pose History node context from an anim node context */
	UFUNCTION(BlueprintCallable, Category = "Animation|PoseHistory", meta = (BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static UE_API FPoseSearchHistoryCollectorAnimNodeReference ConvertToPoseHistoryNode(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result);

	/** Get a Pose History node context from an anim node context (pure) */
	UFUNCTION(BlueprintPure, Category = "Animation|PoseHistory", meta = (BlueprintThreadSafe, DisplayName = "Convert to Pose History Node"))
	static UE_API void ConvertToPoseHistoryNodePure(const FAnimNodeReference& Node, FPoseSearchHistoryCollectorAnimNodeReference& PoseSearchHistoryCollectorNode, bool& Result);

	UFUNCTION(BlueprintPure, Category = "Animation|PoseHistory", meta = (BlueprintThreadSafe, DisplayName = "Get Pose History Node Trajectory"))
	static UE_API void GetPoseHistoryNodeTransformTrajectory(const FPoseSearchHistoryCollectorAnimNodeReference& PoseSearchHistoryCollectorNode, FTransformTrajectory& Trajectory);

	UFUNCTION(BlueprintCallable, Category = "Animation|PoseHistory", meta = (BlueprintThreadSafe, DisplayName = "Set Pose History Node Trajectory"))
	static UE_API void SetPoseHistoryNodeTransformTrajectory(const FPoseSearchHistoryCollectorAnimNodeReference& PoseSearchHistoryCollectorNode, const FTransformTrajectory& Trajectory);

	UFUNCTION(BlueprintPure, Category = "Animation|PoseHistory", meta = (BlueprintThreadSafe))
	static UE_API FPoseHistoryReference GetPoseHistoryReference(const FPoseSearchHistoryCollectorAnimNodeReference& Node);

	UFUNCTION(BlueprintPure, Category = "Animation|PoseHistory", meta = (BlueprintThreadSafe))
	static UE_API bool GetPoseHistoryBoneWorldTransform(FTransform& OutTransform, const FPoseSearchHistoryCollectorAnimNodeReference& Node, const USkeleton* Skeleton, const FName BoneName, const float Time);

	UFUNCTION(BlueprintPure, Category = "Animation|PoseHistory", meta = (BlueprintThreadSafe))
	static UE_API bool GetPoseHistoryBoneWorldLinearVelocity(FVector& OutVelocity, const FPoseSearchHistoryCollectorAnimNodeReference& Node, const USkeleton* Skeleton, const FName BoneName, const float Time, const float DeltaTime);
};

#undef UE_API
