// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTagContainer.h"
#include "PoseSearch/KDTree.h"
#include "PoseSearch/PoseSearchCost.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "PoseSearch/VPTree.h"

namespace UE::PoseSearch
{

POSESEARCH_API void CompareFeatureVectors(TConstArrayView<float> A, TConstArrayView<float> B, TConstArrayView<float> WeightsSqrt, TArrayView<float> Result);
POSESEARCH_API float CompareFeatureVectors(TConstArrayView<float> A, TConstArrayView<float> B);

/**
 * This is kept for each pose in the search index along side the feature vector values and is used to influence the search.
 */
struct FPoseMetadata
{
private:
	enum { ValueOffsetNumBits = 27 };
	enum { AssetIndexNumBits = 20 };
	enum { BlockTransitionNumBits = 1 };

	uint32 ValueOffset : ValueOffsetNumBits = 0;
	uint32 AssetIndex : AssetIndexNumBits = 0;
	bool bBlockTransition : BlockTransitionNumBits = 0;
	FFloat16 CostAddend = 0.f;

public:
	FPoseMetadata(uint32 InValueOffset = 0, uint32 InAssetIndex = 0, bool bInBlockTransition = false, float InCostAddend = 0.f)
	: ValueOffset(InValueOffset)
	, AssetIndex(InAssetIndex)
	, bBlockTransition(bInBlockTransition)
	, CostAddend(InCostAddend)
	{
		// checking for overflowing inputs
		check(InValueOffset < (1 << ValueOffsetNumBits));
		check(InAssetIndex < (1 << AssetIndexNumBits));
	}

	bool IsBlockTransition() const
	{
		return bBlockTransition;
	}

	uint32 GetAssetIndex() const
	{
		return AssetIndex;
	}

	float GetCostAddend() const
	{
		return CostAddend;
	}

	uint32 GetValueOffset() const
	{
		return ValueOffset;
	}

	void SetValueOffset(uint32 Value)
	{
		check(Value < (1 << ValueOffsetNumBits));
		ValueOffset = Value;
	}

	bool operator==(const FPoseMetadata& Other) const
	{
		return
			ValueOffset == Other.ValueOffset &&
			AssetIndex == Other.AssetIndex &&
			bBlockTransition == Other.bBlockTransition &&
			CostAddend == Other.CostAddend;
	}

	friend FArchive& operator<<(FArchive& Ar, FPoseMetadata& Metadata);
};

/**
* Information about a source animation asset used by a search index.
* Some source animation entries may generate multiple FSearchIndexAsset entries.
**/
struct FSearchIndexAsset
{
	FSearchIndexAsset() {}

	FSearchIndexAsset(
		int32 InSourceAssetIdx,
		bool bInMirrored,
		bool bInLooping,
		bool bInDisableReselection,
		int32 InPermutationIdx,
		const FVector& InBlendParameters,
		int32 InFirstPoseIdx,
		int32 InFirstSampleIdx,
		int32 InLastSampleIdx,
		float InToRealTimeFactor = 1.f)
		: SourceAssetIdx(InSourceAssetIdx)
		, bMirrored(bInMirrored)
		, bLooping(bInLooping)
		, bDisableReselection(bInDisableReselection)
		, PermutationIdx(InPermutationIdx)
		, BlendParameterX(InBlendParameters.X)
		, BlendParameterY(InBlendParameters.Y)
		, FirstPoseIdx(InFirstPoseIdx)
		, FirstSampleIdx(InFirstSampleIdx)
		, LastSampleIdx(InLastSampleIdx)
		, ToRealTimeFactor(InToRealTimeFactor)
	{
		check(FMath::IsNearlyZero(InBlendParameters.Z));
		check(ToRealTimeFactor > UE_KINDA_SMALL_NUMBER);		
	}

	FSearchIndexAsset(
		int32 InSourceAssetIdx,
		int32 InFirstPoseIdx,
		bool bInMirrored,
		bool bInLooping,
		bool bInDisableReselection,
		const FFloatInterval& InSamplingInterval,
		int32 SchemaSampleRate,
		int32 InPermutationIdx,
		FVector InBlendParameters = FVector::Zero(),
		float InToRealTimeFactor = 1.f)
		: FSearchIndexAsset(
			InSourceAssetIdx,
			bInMirrored,
			bInLooping,
			bInDisableReselection,
			InPermutationIdx,
			InBlendParameters,
			InFirstPoseIdx,
			FMath::CeilToInt(InSamplingInterval.Min * SchemaSampleRate),
			FMath::FloorToInt(InSamplingInterval.Max * SchemaSampleRate),
			InToRealTimeFactor)
	{
		check(SchemaSampleRate > 0);
	}
	
	FFloatInterval GetExtrapolationTimeInterval(int32 SchemaSampleRate, const FFloatInterval& AdditionalExtrapolationTime) const;
	int32 GetSourceAssetIdx() const { return SourceAssetIdx; }
	bool IsMirrored() const { return bMirrored; }
	bool IsLooping() const { return bLooping; }
	bool IsDisableReselection() const { return bDisableReselection; }
	int32 GetPermutationIdx() const { return PermutationIdx; }
	FVector GetBlendParameters() const { return FVector(BlendParameterX, BlendParameterY, 0.f); }
	float GetToRealTimeFactor() const { return ToRealTimeFactor; }
	int32 GetFirstPoseIdx() const { return FirstPoseIdx; }

	bool IsPoseInRange(int32 PoseIdx) const { return (PoseIdx >= FirstPoseIdx) && (PoseIdx < FirstPoseIdx + GetNumPoses()); }
	bool operator==(const FSearchIndexAsset& Other) const;
	
#if DO_CHECK
	bool IsInitialized() const 
	{
		return 
			SourceAssetIdx != INDEX_NONE &&
			PermutationIdx != INDEX_NONE &&
			FirstPoseIdx != INDEX_NONE &&
			FirstSampleIdx != INDEX_NONE &&
			LastSampleIdx != INDEX_NONE;
	}
#endif // DO_CHECK

	int32 GetBeginSampleIdx() const { return FirstSampleIdx; }
	int32 GetEndSampleIdx() const {	return LastSampleIdx + 1; }
	int32 GetNumPoses() const { return GetEndSampleIdx() - GetBeginSampleIdx(); }

	float GetFirstSampleTime(int32 SchemaSampleRate) const { check(SchemaSampleRate > 0); return FirstSampleIdx / float(SchemaSampleRate); }
	float GetLastSampleTime(int32 SchemaSampleRate) const { check(SchemaSampleRate > 0); return LastSampleIdx / float(SchemaSampleRate); }

	int32 GetPoseIndexFromTime(float RealTimeInSeconds, int32 SchemaSampleRate) const
	{
#if DO_CHECK
		check(IsInitialized());
#endif
		const int32 NumPoses = GetNumPoses();
		int32 PoseOffset = FMath::RoundToInt(SchemaSampleRate * RealTimeInSeconds) - FirstSampleIdx;
		if (bLooping)
		{
			PoseOffset = FMath::WrapExclusive(PoseOffset, 0, NumPoses);
			return FirstPoseIdx + PoseOffset;
		}

		if (PoseOffset >= 0 && PoseOffset < NumPoses)
		{
			return FirstPoseIdx + PoseOffset;
		}

		return INDEX_NONE;
	}

	// iteractes over all the valid pose indexes contained in this FSearchIndexAsset from StartTime to EndTime, calling ProcessPoseIndex function on every pose accounting for looping
	void IteratePoseIndexesOverTime(float StartTime, float EndTime, int32 SchemaSampleRate, const TFunction<void(int32 PoseIdx)>& ProcessPoseIndex) const
	{
#if DO_CHECK
		check(IsInitialized());
		check(StartTime <= EndTime);
#endif
		const int32 NumPoses = GetNumPoses();
		const int32 StartPoseOffset = FMath::RoundToInt(SchemaSampleRate * StartTime) - FirstSampleIdx;
		const int32 EndPoseOffset = FMath::RoundToInt(SchemaSampleRate * EndTime) - FirstSampleIdx;
		if (bLooping)
		{
			const int32 EndWrappedPoseOffset = FMath::WrapExclusive(EndPoseOffset, 0, NumPoses);
			const int32 StartWrappedPoseOffset = FMath::WrapExclusive(StartPoseOffset, 0, NumPoses);

			for (int32 CurrentWrappedPoseOffset = StartWrappedPoseOffset;;)
			{
				ProcessPoseIndex(FirstPoseIdx + CurrentWrappedPoseOffset);

				if (CurrentWrappedPoseOffset == EndWrappedPoseOffset)
				{
					break;
				}

				CurrentWrappedPoseOffset = (CurrentWrappedPoseOffset + 1) % NumPoses;
			}
		}
		else
		{
			check(StartPoseOffset <= EndPoseOffset);
			const bool bOutOfStartRange = StartPoseOffset < 0 && EndPoseOffset < 0;
			const bool bOutOfEndRange = StartPoseOffset >= NumPoses && EndPoseOffset >= NumPoses;

			if (!bOutOfStartRange && !bOutOfEndRange)
			{
				const int32 EndClampedPoseOffset = FMath::Clamp(EndPoseOffset, 0, NumPoses - 1);
				const int32 StartClampedPoseOffset = FMath::Clamp(StartPoseOffset, 0, NumPoses - 1);

				for (int32 CurrentClampedPoseOffset = StartClampedPoseOffset; CurrentClampedPoseOffset <= EndClampedPoseOffset; ++CurrentClampedPoseOffset)
				{
					ProcessPoseIndex(FirstPoseIdx + CurrentClampedPoseOffset);
				}
			}
		}
	}

	float GetTimeFromPoseIndex(int32 PoseIdx, int32 SchemaSampleRate) const
	{
		check(SchemaSampleRate > 0);

		const int32 PoseOffset = PoseIdx - FirstPoseIdx;
		check(PoseOffset >= 0 && PoseOffset < GetNumPoses());

		const float Time = (FirstSampleIdx + PoseOffset) / float(SchemaSampleRate);
		return Time;
	}

	// calculates the delta time in seconds ToPoseIdx - FromPoseIdx
	float GetDeltaTimeBetweenPoseIndexes(int32 FromPoseIdx, int32 ToPoseIdx, int32 SchemaSampleRate) const
	{
		check(SchemaSampleRate > 0);

		check(IsPoseInRange(FromPoseIdx) && IsPoseInRange(ToPoseIdx));

		int32 PoseOffset = ToPoseIdx - FromPoseIdx;
		if (PoseOffset < 0 && bLooping)
		{
			PoseOffset += GetNumPoses();
		}
		
		const float DeltaTime = PoseOffset / float(SchemaSampleRate);
		return DeltaTime;
	}

	friend FArchive& operator<<(FArchive& Ar, FSearchIndexAsset& IndexAsset);

private:
	// Index of the source asset in search index's container (i.e. UPoseSearchDatabase)
	const int32 SourceAssetIdx = INDEX_NONE;

	const bool bMirrored : 1 = false; 
	const bool bLooping : 1 = false;
	const bool bDisableReselection : 1 = false;
	const int32 PermutationIdx = INDEX_NONE;
	
	// fixed blend space patameters associated to this FSearchIndexAsset if SourceAssetIdx refers to a blend space
	const float BlendParameterX = 0.f;
	const float BlendParameterY = 0.f;

	const int32 FirstPoseIdx = INDEX_NONE;
	const int32 FirstSampleIdx = INDEX_NONE;
	const int32 LastSampleIdx = INDEX_NONE;

	// if SourceAssetIdx refers to a blend space, the animation asset would be in a normalized time interval [0-1]. ToRealTimeFactor brings the normalized time to the real time for the fixed BlendParameterX/BlendParameterY
	const float ToRealTimeFactor = 1.f;
};

#if WITH_EDITORONLY_DATA
struct FSearchStats
{
	float AverageSpeed = 0.f;
	float MaxSpeed = 0.f;
	float AverageAcceleration = 0.f;
	float MaxAcceleration = 0.f;

	void Reset();
	bool operator==(const FSearchStats& Other) const;
	friend FArchive& operator<<(FArchive& Ar, FSearchStats& Stats);
};
#endif // WITH_EDITORONLY_DATA

// compact representation of an array of arrays
template <typename Type = int32>
struct FSparsePoseMultiMap
{
	FSparsePoseMultiMap() = default;
	~FSparsePoseMultiMap() = default;
	FSparsePoseMultiMap(const FSparsePoseMultiMap& Other) = default;
	FSparsePoseMultiMap(FSparsePoseMultiMap&& Other) = default;
	FSparsePoseMultiMap& operator=(const FSparsePoseMultiMap& Other) = default;
	FSparsePoseMultiMap& operator=(FSparsePoseMultiMap&& Other) = default;

	FSparsePoseMultiMap(Type InMaxKey, Type InMaxValue)
	{
		Init(InMaxKey, InMaxValue);
	}

	FSparsePoseMultiMap(TConstArrayView<TPair<Type, TArray<Type>>> Pairs, Type InMaxValue)
	{
		Init(Pairs, InMaxValue);
	}

	void Init(Type InMaxKey, Type InMaxValue)
	{
		MaxKey = InMaxKey;
		MaxValue = InMaxValue;
		DeltaKeyValue = InMaxValue >= InMaxKey ? InMaxValue - InMaxKey + 1 : 0;
	
		DataValues.Reset();
		DataValues.Reserve(InMaxKey * 2);
		
		for (Type Index = 0; Index < InMaxKey; ++Index)
		{
			DataValues.Add(Type(INDEX_NONE));
		}
	}

	void Init(TConstArrayView<TPair<Type, TArray<Type>>> Pairs, Type InMaxValue)
	{
		Init(Pairs.Num(), InMaxValue);
		
		for (const TPair<Type, TArray<Type>>& Pair : Pairs)
		{
			Insert(Pair.Key, Pair.Value);
		}
	}
	
	void Insert(Type Key, TConstArrayView<Type> Values)
	{
#if DO_CHECK
		// key must be valid..
		check(Key != Type(INDEX_NONE));

		// ..and within range of acceptance
		check(Key >= 0 && Key < MaxKey);

		// DataValues[Key] should be empty - inserting the same key multiple times is not allowed
		check(DataValues[Key] == Type(INDEX_NONE));

		// Values must contains at least one element..
		check(!Values.IsEmpty());

		// ..and none of the elements should be an invalid value (or it'll confuse the key/value decoding)
		for (Type Value : Values)
		{
			check(Value <= MaxValue && Value != Type(INDEX_NONE));
		}
#endif //DO_CHECK

		// if Values contains only one element we store it directly at the location referenced by key
		if (Values.Num() == 1)
		{
			DataValues[Key] = Values[0];
		}
		// else we store the offset of the beginning of the encoded array (where the first element is the array size, followed by all the Values[i] elements
		else
		{
			// checking for overflow
			check((DataValues.Num() + 1 + Values.Num()) < (1 << (sizeof(Type) * 4 - 1)));
			check(int32(MaxKey) <= DataValues.Num());

			// adding DeltaKeyValue to DataValues.Num() to making sure DataValues[Key] > MaxValue
			DataValues[Key] = DataValues.Num() + DeltaKeyValue;
			check(DataValues[Key] > MaxValue);

			// encoding Values at the end of DataValues, by storing its size.. 
			DataValues.Add(Values.Num());
			// ..and its data right after
			DataValues.Append(Values);
		}
	}

	TConstArrayView<Type> operator [](Type Key) const
	{
		check(Key != Type(INDEX_NONE) && Key < MaxKey);
		const Type Value = DataValues[Key];
		if (Value <= MaxValue)
		{
			return MakeArrayView(DataValues).Slice(Key, 1);
		}

		check(Value >= DeltaKeyValue);
		const Type DecodedArrayStartLocation = Value - DeltaKeyValue;

		// decoding the array at location DecodedArrayStartLocation: its size is stored at DecodedArrayStartLocation offset..
		const Type Size = DataValues[DecodedArrayStartLocation];
		// ..and it's data starts at the next location DecodedArrayStartLocation + 1
		const Type DataOffset = DecodedArrayStartLocation + 1;
		check(int32(DataOffset + Size) <= DataValues.Num());
		return MakeArrayView(DataValues).Slice(DataOffset, Size);
	}

	Type Num() const
	{
		return MaxKey;
	}

	SIZE_T GetAllocatedSize() const
	{
		return sizeof(MaxKey) + sizeof(MaxValue) + sizeof(DeltaKeyValue) + DataValues.GetAllocatedSize();
	}

	bool operator==(const FSparsePoseMultiMap& Other) const
	{
		return
			MaxKey == Other.MaxKey &&
			MaxValue == Other.MaxValue &&
			DeltaKeyValue == Other.DeltaKeyValue &&
			DataValues == Other.DataValues;
	}

	friend FArchive& operator<<(FArchive& Ar, FSparsePoseMultiMap& SparsePoseMultiMap)
	{
		Ar << SparsePoseMultiMap.MaxKey;
		Ar << SparsePoseMultiMap.MaxValue;
		Ar << SparsePoseMultiMap.DeltaKeyValue;
		Ar << SparsePoseMultiMap.DataValues;
		return Ar;
	}

	void Reset()
	{
		MaxKey = Type(0);
		MaxValue = Type(0);
		DeltaKeyValue = Type(0);
		DataValues.Reset();
	}

	Type MaxKey = Type(0);
	Type MaxValue = Type(0);
	Type DeltaKeyValue = Type(0);
	TArray<Type> DataValues;
};

#if WITH_EDITOR
// Experimental, this feature might be removed without warning, not for production use
struct FEventDataCollector
{
	typedef TSet<int32, DefaultKeyFuncs<int32>, TInlineSetAllocator<256>> FPoseIndexes;
	typedef TPair<FGameplayTag, FPoseIndexes> FTagToPoseIndexes;
	typedef TMap<FGameplayTag, FPoseIndexes, TInlineSetAllocator<16>> FData;

	void Emplace(const FGameplayTag& EventTag, int32 PoseIdx);
	void MergeWith(const FEventDataCollector& Other);
	const FData& GetData() const { return Data; }

private:
	TMap<FGameplayTag, FPoseIndexes, TInlineSetAllocator<16>> Data;
};
#endif // WITH_EDITOR

// Experimental, this feature might be removed without warning, not for production use
struct FEventData
{
	typedef TArray<int32> FPoseIndexes;
	typedef TPair<FGameplayTag, FPoseIndexes> FTagToPoseIndexes;
	typedef TArray<FTagToPoseIndexes> FData;

	friend FArchive& operator<<(FArchive& Ar, FEventData& EventData);
	bool operator==(const FEventData& Other) const;
	const TConstArrayView<int32> GetPosesWithEvent(const FGameplayTag& GameplayTag) const;
	bool IsPoseFromEventTag(int32 EventPoseIdx, const FGameplayTag& GameplayTag) const;
	const FData& GetData() const { return Data; }
	void Reset();

#if WITH_EDITOR
	void Initialize(const FEventDataCollector& EventDataCollector);
	POSESEARCH_API SIZE_T GetAllocatedSize(void) const;
	void ValidateEventData() const;
#endif // WITH_EDITOR

private:
	FData Data;
};

/**
* case class for FSearchIndex. building block used to gather data for data mining and calculate weights, pca, kdtree stuff
*/
struct FSearchIndexBase
{
	TAlignedArray<float> Values;
	FSparsePoseMultiMap<int32> ValuesVectorToPoseIndexes;
	TAlignedArray<FPoseMetadata> PoseMetadata;
	bool bAnyBlockTransition = false;
	TAlignedArray<FSearchIndexAsset> Assets;
	// Experimental, this feature might be removed without warning, not for production use
	FEventData EventData;

	// minimum of the database metadata CostAddend: it represents the minimum cost of any search for the associated database (we'll skip the search in case the search result total cost is already less than MinCostAddend)
	float MinCostAddend = -MAX_FLT;

#if WITH_EDITORONLY_DATA
	FSearchStats Stats;
#endif // WITH_EDITORONLY_DATA

	// overriding ctor, copy, move to allow deprecations.
	FSearchIndexBase();
	~FSearchIndexBase();
	FSearchIndexBase(const FSearchIndexBase& Other);
	FSearchIndexBase(FSearchIndexBase&& Other);
	FSearchIndexBase& operator=(const FSearchIndexBase& Other);
	FSearchIndexBase& operator=(FSearchIndexBase&& Other);

	int32 GetNumPoses() const { return PoseMetadata.Num(); }
	int32 GetNumValuesVectors(int32 DataCardinality) const
	{
		check(DataCardinality > 0);
		check(Values.Num() % DataCardinality == 0);
		return Values.Num() / DataCardinality;
	}

	bool IsValidPoseIndex(int32 PoseIdx) const { return PoseIdx < GetNumPoses(); }
	bool IsEmpty() const;
	bool IsValuesEmpty() const { return Values.IsEmpty(); }

	void ResetValues() { Values.Reset(); }
	void AllocateData(int32 DataCardinality, int32 NumPoses);
	
	const FSearchIndexAsset& GetAssetForPose(int32 PoseIdx) const;
	POSESEARCH_API const FSearchIndexAsset* GetAssetForPoseSafe(int32 PoseIdx) const;

	void Reset();
	
	void PruneDuplicateValues(float SimilarityThreshold, int32 DataCardinality, bool bDoNotGenerateValuesVectorToPoseIndexes);

	TConstArrayView<float> GetPoseValuesBase(int32 PoseIdx, int32 DataCardinality) const
	{
		check(!IsValuesEmpty() && PoseIdx >= 0 && PoseIdx < GetNumPoses());
		check(Values.Num() % DataCardinality == 0);
		const int32 ValueOffset = PoseMetadata[PoseIdx].GetValueOffset();
		return MakeArrayView(&Values[ValueOffset], DataCardinality);
	}

	TConstArrayView<float> GetValuesVector(int32 ValuesVectorIdx, int32 DataCardinality) const
	{
		check(!IsValuesEmpty() && ValuesVectorIdx >= 0 && ValuesVectorIdx < GetNumValuesVectors(DataCardinality));
		const int32 ValueOffset = ValuesVectorIdx * DataCardinality;
		return MakeArrayView(&Values[ValueOffset], DataCardinality);
	}

	bool operator==(const FSearchIndexBase& Other) const;

	friend FArchive& operator<<(FArchive& Ar, FSearchIndexBase& Index);
};

/**
* A search index for animation poses. The structure of the search index is determined by its UPoseSearchSchema.
* May represent a single animation (see UPoseSearchSequenceMetaData) or a collection (see UPoseSearchDatabase).
*/
struct FSearchIndex : public FSearchIndexBase
{
	// we store weights square roots to reduce numerical errors when CompareFeatureVectors 
	// ((VA - VB) * VW).square().sum()
	// instead of
	// ((VA - VB).square() * VW).sum()
	// since (VA - VB).square() could lead to big numbers, and VW being multiplied by the variance of the dataset
	TAlignedArray<float> WeightsSqrt;

	// array containing the data of FSearchIndexBase::Values encoded in PCA space. used to create the kdtree.
	// Not all the poses have a related PCAValue(s) array since, for instance, poses marked as block transition 
	// don't have to be searchable via kdtree, but only via continuing pose search, so method like 
	// GetPCAValueIndex or GetPoseToPCAValuesVectorIndexes could not find the respective PCAValueIndex of a requested pose
	TAlignedArray<float> PCAValues;
	FSparsePoseMultiMap<int32> PCAValuesVectorToPoseIndexes;
	TAlignedArray<float> PCAProjectionMatrix;
	TAlignedArray<float> Mean;

	FKDTree KDTree;
	FVPTree VPTree;

#if WITH_EDITORONLY_DATA
	TAlignedArray<float> DeviationEditorOnly;
	float PCAExplainedVarianceEditorOnly = 0.f;
#endif

	FSearchIndex() = default;
	~FSearchIndex() = default;
	FSearchIndex(const FSearchIndex& Other); // custom copy constructor to deal with the KDTree DataSrc
	FSearchIndex(FSearchIndex&& Other);
	FSearchIndex& operator=(const FSearchIndex& Other); // custom equal operator to deal with the KDTree DataSrc
	FSearchIndex& operator=(FSearchIndex&& Other);

	void Reset();
	POSESEARCH_API TConstArrayView<float> GetPoseValues(int32 PoseIdx) const;
	TConstArrayView<float> GetReconstructedPoseValues(int32 PoseIdx, TArrayView<float> BufferUsedForReconstruction) const;
	int32 GetNumDimensions() const;
	int32 GetNumberOfPrincipalComponents() const;
	POSESEARCH_API TConstArrayView<float> PCAProject(TConstArrayView<float> PoseValues, TArrayView<float> BufferUsedForProjection) const;

	POSESEARCH_API TConstArrayView<float> GetPoseValuesSafe(int32 PoseIdx, TArray<float>& BufferUsedForReconstruction) const;

	// since PCAValues (as well as Values can be pruned out from duplicate data, we lose the 1:1 mapping between PoseIdx and PCAValuesVectorIdx
	// that in the case of GetPoseValuesSafe it's stored in PoseMetadata[PoseIdx].GetValueOffset(), but missing for the PCAValues, so this API input is NOT a PoseIdx
	// mapping between PoseIdx to PCAValuesVectorIdx can be reconstructed by inverting the PCAValuesVectorToPoseIndexes via GetPoseToPCAValuesVectorIndexes
	POSESEARCH_API TConstArrayView<float> GetPCAPoseValues(int32 PCAValuesVectorIdx) const;

	int32 GetNumPCAValuesVectors(int32 DataCardinality) const
	{
		check(DataCardinality > 0);
		check(PCAValues.Num() % DataCardinality == 0);
		return PCAValues.Num() / DataCardinality;
	}

	void PruneDuplicatePCAValues(float SimilarityThreshold, int32 NumberOfPrincipalComponents);
	void PrunePCAValuesFromBlockTransitionPoses(int32 NumberOfPrincipalComponents);

	// returns the inverse mapping of PCAValuesVectorToPoseIndexes
	POSESEARCH_API void GetPoseToPCAValuesVectorIndexes(TArray<uint32>& PoseToPCAValuesVectorIndexes) const;
	
	// returns the PCAValueIndex from the pose index. the method is very slow. consider calling GetPoseToPCAValuesVectorIndexes to cache ALL the PCAValueIndex from the pose index mapping
	int32 GetPCAValueIndex(int32 PoseIdx) const;

	bool operator==(const FSearchIndex& Other) const;
	friend FArchive& operator<<(FArchive& Ar, FSearchIndex& Index);
};

struct FVPTreeDataSource
{
	explicit FVPTreeDataSource(const FSearchIndex& InSearchIndex)
		: SearchIndex(InSearchIndex)
	{
	}

    const TConstArrayView<float> operator[](int32 Index) const
    {
		const int32 DataCardinality = SearchIndex.GetNumDimensions();
        return SearchIndex.GetValuesVector(Index, DataCardinality);
    }

    int32 Num() const
    {
		const int32 DataCardinality = SearchIndex.GetNumDimensions();
        return SearchIndex.GetNumValuesVectors(DataCardinality);
    }

    static float GetDistance(const TConstArrayView<float> A, const TConstArrayView<float> B);

private:
	const FSearchIndex& SearchIndex;
};

} // namespace UE::PoseSearch
