// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "PoseSearch/MultiAnimAsset.h"
#include "PoseSearch/PoseSearchRole.h"
#include "PoseSearchInteractionAsset.generated.h"

#define UE_API POSESEARCH_API

USTRUCT(Experimental, BlueprintType)
struct FPoseSearchInteractionAssetItem
{
	GENERATED_BODY()

	// associated aniamtion for this FPoseSearchInteractionAssetItem
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	TObjectPtr<UAnimationAsset> Animation;

	// associated role for this FPoseSearchInteractionAssetItem
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FName Role;

	// relative weight to the other FPoseSearchInteractionAssetItem::WarpingWeightRotation(s) defining which character will be rotated while warping
	// 0 - the associated character to this item will move fully to compensate the warping errors
	// > 0 && all the other FPoseSearchInteractionAssetItem::WarpingWeightTranslation as zero, and the associated character will not move
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Warping", meta = (ClampMin = "0", ClampMax = "1"))
	float WarpingWeightRotation = 0.5f;

	// relative weight to the other FPoseSearchInteractionAssetItem::WarpingWeightTranslation(s) defining which character will be translated while warping
	// 0 - the associated character to this item will move fully to compensate the warping errors
	// > 0 && all the other FPoseSearchInteractionAssetItem::WarpingWeightTranslation as zero, and the associated character will not move
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Warping", meta = (ClampMin = "0", ClampMax = "1"))
	float WarpingWeightTranslation = 0.5f;

	// offset from the origin
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FTransform Origin = FTransform::Identity;

#if WITH_EDITORONLY_DATA
	// If null, the default preview mesh for the skeleton will be used. Otherwise, this will be used in preview scenes.
	UPROPERTY(EditAnywhere, Category = "Preview", meta = (ExcludeFromHash))
	TObjectPtr<USkeletalMesh> PreviewMesh = nullptr;
#endif // WITH_EDITORONLY_DATA
};

UCLASS(MinimalAPI, Experimental, BlueprintType, Category = "Animation|Pose Search")
class UPoseSearchInteractionAsset : public UMultiAnimAsset
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	TArray<FPoseSearchInteractionAssetItem> Items;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (ClampMin = "0", ClampMax = "1"))
	float WarpingBankingWeight = 0.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (ClampMin = "0", ClampMax = "1"))
	float MinimalTranslationWeight = 0.f;

#if WITH_EDITORONLY_DATA

	UPROPERTY(Transient, EditAnywhere, BlueprintReadWrite, Category = "Debug", meta = (EditCondition=bEnableDebugWarp, EditConditionHides))
	TArray<FTransform> DebugWarpOffsets;

	// used to test warping: 0 - no warping applied, 1 - full warping/alignment applied
	// test warping actors will be offsetted by Items::DebugWarpOffset transforms from the original
	// UMultiAnimAsset::GetOrigin() definition and warped accordingly with CalculateWarpTransforms
	// following the rotation and translation weights defined in Items::WarpingWeightRotation and
	// Items::WarpingWeightTranslation as relative weights between the Items (they'll be normalized at runtime)
	UPROPERTY(Transient, EditAnywhere, BlueprintReadWrite, Category = "Debug", meta = (ClampMin = "0", ClampMax = "1", EditCondition=bEnableDebugWarp, EditConditionHides))
	float DebugWarpAmount = 0.f;

	// useful to test TimeOffset
	UPROPERTY(Transient, EditAnywhere, BlueprintReadWrite, Category = "Debug", meta = (ClampMin = "0", EditCondition=bEnableDebugWarp, EditConditionHides))
	float DebugTimeOffset = 0.f;

	UPROPERTY(Transient, EditAnywhere, BlueprintReadWrite, Category = "Debug")
	bool bEnableDebugWarp = false;
#endif // WITH_EDITORONLY_DATA

	UE_API virtual bool IsLooping() const override;
	UE_API virtual bool HasRootMotion() const override;
	UE_API virtual float GetPlayLength(const FVector& BlendParameters) const override;

	virtual int32 GetNumRoles() const override { return Items.Num(); }
	virtual UE::PoseSearch::FRole GetRole(int32 RoleIndex) const override { return Items[RoleIndex].Role; }
	
	UE_API virtual UAnimationAsset* GetAnimationAsset(const UE::PoseSearch::FRole& Role) const override;

	UE_API virtual FTransform GetOrigin(const UE::PoseSearch::FRole& Role) const override;

#if WITH_EDITOR
	UE_API FTransform GetDebugWarpOrigin(const UE::PoseSearch::FRole& Role, bool bComposeWithDebugWarpOffset) const;
	UE_API virtual USkeletalMesh* GetPreviewMesh(const UE::PoseSearch::FRole& Role) const override;
#endif // WITH_EDITOR

	UE_API virtual void CalculateWarpTransforms(float Time, float TimeOffset, const TConstArrayView<const FTransform> ActorRootBoneTransforms, TArrayView<FTransform> FullAlignedActorRootBoneTransforms,
		const TConstArrayView<const UMirrorDataTable*> MirrorDataTables = TConstArrayView<const UMirrorDataTable*>(), const TConstArrayView<TObjectPtr<const UObject>> DebugAnimContexts = TConstArrayView<TObjectPtr<const UObject>>()) const override;
};

#undef UE_API
