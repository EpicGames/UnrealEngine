// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchHistory.h"
#include "PoseSearchInteractionAvailability.generated.h"

class UPoseSearchDatabase;

// input for MotionMatchInteraction_Pure: it declares that the associated character ("AnimContext" that could be an AnimInstance or an AnimNextCharacterComponent)
// is willing to partecipate in an interaction described by a UMultiAnimAsset (derived by UPoseSearchInteractionAsset) contained in the UPoseSearchDatabase Database
// with one of the roles in RolesFilter (if empty ANY of the Database roles can be taken) the MotionMatchInteraction_Pure will ultimately setup a motion matching query
// using looking for the pose history "PoseHistoryName" to gather bone and trajectory positions for this character for an interaction to be valid,
// the query needs to find all the other interacting characters within BroadPhaseRadius, and reach a maximum cost of MaxCost
// Experimental, this feature might be removed without warning, not for production use
USTRUCT(Experimental, BlueprintType, Category = "Animation|Pose Search")
struct FPoseSearchInteractionAvailability
{
	GENERATED_BODY()

	// Database describing the interaction. It'll contains multi character UMultiAnimAsset and a schema with multiple skeletons with associated roles
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings)
	TObjectPtr<UPoseSearchDatabase> Database;

	// in case this availability Database is valid (not null), Tag (if IsTagValid()) is used to flag the Database with a specific name. Different availabilities can share the same Tag.
	// in case this availability Database is NOT valid, we use the valid Tag to figure out all the possible databases that can be assigned to this availability from all the published availabilities.
	// The reason behind Tag is, for example, to be able to have NPCs been able to interact with a main character (MC), without the MC having a direct dependency to the database used 
	// for the interaction allowing those NPCs to be contextually loaded/unloaded, streamed in/out, with the obvious advantages for the the memory managment of the "payload" database.
	// Another reason for Tag, is to facilitate the setup of interactions, where the MC have to publish only one availability with its own assigned Role (in RolesFilter) 
	// automatically contextually resolved in multiple different types of possible databases: it could be MC-NPC, MC-Vehicle, MC-Whatever
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings)
	FName Tag;

	// roles the character is willing to take to partecipate in this interaction. If empty ANY of the Database roles can be taken
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings)
	TArray<FName> RolesFilter;

	// the associated character to this FPoseSearchInteractionAvailability will partecipate in an interaction only if all the necessary roles gest assigned to character within BroadPhaseRadius centimeters
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings)
	float BroadPhaseRadius = 500.f;

	// during interaction the BroadPhaseRadius will be incremented by BroadPhaseRadiusIncrementOnInteraction to create 
	// geometrical histeresys, where it's harder for actors to get into interaction rather than staying in interaction
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings)
	float BroadPhaseRadiusIncrementOnInteraction = 10.f;
	
	// if true, the system will disable collsions between interacting characters
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings)
	bool bDisableCollisions = false;

	// the Actor with the higher TickPriority of any Availability request will be elected as the MainActor of the interaction island (containing all the actors that could interact with each other)
	// the main Actor will tick first and all the other interacting actors will tick after in a concurrently from each other. TickPriority is useful if your setup is already enforcing tick dependency 
	// between actors, and the motion matching interaction system needs to play nicely with them.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings)
	int32 TickPriority = UE::PoseSearch::DefaultTickPriority;

	bool IsTagValid() const { return !Tag.IsNone(); }

	bool operator==(const FPoseSearchInteractionAvailability& Other) const = default;
};

namespace UE::PoseSearch
{
// Experimental, this feature might be removed without warning, not for production use
struct FInteractionAvailability : public FPoseSearchInteractionAvailability
{
	void Init(const FPoseSearchInteractionAvailability& InAvailability, FName InPoseHistoryName, const IPoseHistory* InPoseHistory)
	{
		check(!InPoseHistory || InPoseHistory->AsWeak().IsValid());

		static_cast<FPoseSearchInteractionAvailability&>(*this) = InAvailability;
		PoseHistoryName = InPoseHistoryName;
		PoseHistory = InPoseHistory;
	}

	FString GetPoseHistoryName() const;
	const IPoseHistory* GetPoseHistory(const UObject* AnimContext) const;

private:
	FName PoseHistoryName;
	const IPoseHistory* PoseHistory = nullptr;
};

// Experimental, this feature might be removed without warning, not for production use
struct FInteractionAnimContextAvailabilities
{
	TObjectPtr<const UObject> AnimContext;
	TArray<FInteractionAvailability> Availabilities;
};

} // namespace UE::PoseSearch
