// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"
#include "PoseSearch/PoseSearchAnimNotifies.h"
#include "PoseSearch/PoseSearchConstraints.h"
#include "PoseSearch/PoseSearchInteractionAvailability.h"
#include "PoseSearch/PoseSearchInteractionValidator.h"
#include "PoseSearch/PoseSearchLibrary.h"

class UMultiAnimAsset;
class UPoseSearchInteractionSubsystem;
struct FPoseSearchBlueprintResult;
struct FPoseSearchContinuingProperties;

namespace UE::PoseSearch
{
struct FInteractionIsland;
struct FInteractionSearchContext;
struct IInteractionIslandDependency;

// Experimental, this feature might be removed without warning, not for production use
typedef TPair<TObjectPtr<AActor>, TObjectPtr<AActor>> FDisabledCollision;
// Experimental, this feature might be removed without warning, not for production use
typedef TArray<FDisabledCollision> FDisabledCollisions;

// Experimental, this feature might be removed without warning, not for production use
struct FInteractionSearchContextBase
{
	int32 Num() const { return AnimContexts.Num(); }
	const UObject* GetAnimContext(int32 Index) const { return AnimContexts[Index]; }
	const IPoseHistory* GetPoseHistory(int32 Index) const;
	const FRole GetRole(int32 Index) const { return Roles[Index]; }
	const UPoseSearchDatabase* GetDatabase() const { return Database; }
	const TConstArrayView<FRole> GetRoles() const { return Roles; }
	const TConstArrayView<TObjectPtr<const UObject>> GetAnimContexts() const { return AnimContexts; }

	bool IsDisableCollisions() const { return bDisableCollisions; }

	uint32 GetHashForEquivalence() const;
	uint32 GetContextHashForEquivalence() const;
	void CacheHashesForEquivalence();
	
	void SetDatabase(const UPoseSearchDatabase* InDatabase) { Database = InDatabase; }
	void AddReferencedObjects(FReferenceCollector& Collector);
	void Add(const UObject* AnimContext, const IPoseHistory* PoseHistory, const FRole Role, bool bWantsDisableCollisions);

#if DO_CHECK
	bool CheckForConsistency() const;
#endif // DO_CHECK

private:
	void CalculateHashesForEquivalence(uint32& OutCachedHashForEquivalence, uint32& OutCachedContextHashForEquivalence) const;

	TArray<TObjectPtr<const UObject>, TInlineAllocator<PreallocatedRolesNum>> AnimContexts;

	// storing PoseHistories as TWeakPtr since they are from nodes or traits that can be invalidated within the same frame.
	// for performance reasons are not used to determine equivalence IsEquivalent / GetHashForEquivalence
	TArray<TWeakPtr<const IPoseHistory, ESPMode::ThreadSafe>, TInlineAllocator<PreallocatedRolesNum>> PoseHistories;
	
	TArray<FRole, TInlineAllocator<PreallocatedRolesNum>> Roles;

	TObjectPtr<const UPoseSearchDatabase> Database;
	bool bDisableCollisions = false;
	enum { InvalidCachedHashForEquivalence = 0 };
	uint32 CachedHashForEquivalence = InvalidCachedHashForEquivalence;
	uint32 CachedContextHashForEquivalence = InvalidCachedHashForEquivalence;
};

// Experimental, this feature might be removed without warning, not for production use
struct FInteractionSearchResult : public FSearchResult
{
	int32 SearchIndex = INDEX_NONE;

	// cached actors root transforms for all the roles in SelectedAnimation (as UMultiAnimAsset),
	// so we don't have to query the pose history to gather it when it's not thread safe to do so
	TArray<FTransform, TInlineAllocator<PreallocatedRolesNum>> ActorRootTransforms;

	// cached actors root bone transforms for all the roles in SelectedAnimation (as UMultiAnimAsset),
	// so we don't have to query the pose history to gather it when it's not thread safe to do so
	TArray<FTransform, TInlineAllocator<PreallocatedRolesNum>> ActorRootBoneTransforms;

	// @todo: preallocate some...
	// all the active constraints in this result
	TArray<FPoseSearchConstraint> Constraints;

	bool operator==(const FInteractionSearchResult& Other) const;
};

// Experimental, this feature might be removed without warning, not for production use
struct FValidInteractionSearch : public FInteractionSearchContextBase
{
	FValidInteractionSearch& Init(const FInteractionSearchContextBase& SearchContextBase);
	void Update(const FInteractionSearchResult& InSearchResult);
	void DoDisableCollisions();
	void UndoDisableCollisions();

#if ENABLE_VISUAL_LOG
	void VLogContext(const FColor& Color) const;
#endif // ENABLE_VISUAL_LOG

	void AddReferencedObjects(FReferenceCollector& Collector);
	const FInteractionSearchResult& GetSearchResult() const { return SearchResult; }

private:
	FInteractionSearchResult SearchResult;
	FDisabledCollisions DisabledCollisions;

#if ENABLE_ANIM_DEBUG
	UE_MT_DECLARE_TS_RW_ACCESS_DETECTOR(ThreadSafeCounter);
#endif // ENABLE_ANIM_DEBUG
};

struct FValidInteractionSearches
{
	void Update(const UPoseSearchInteractionSubsystem* InteractionSubsystem);
	const FValidInteractionSearch* Get(const FInteractionSearchContext& SearchContext) const;
	// returns true if the tuple of AnimContexts (e.g. pair of AnimInstance(s) ~ Characters) from the SearchContext is used in any FValidInteractionSearch with any assigned roles
	bool IsThereAnyValidSearchWithTheSameContexts(const FInteractionSearchContext& SearchContext) const;
	void AddReferencedObjects(FReferenceCollector& Collector);

private:
	static void OnInteractionStart(FValidInteractionSearch& ValidInteractionSearch, const UPoseSearchInteractionSubsystem* InteractionSubsystem);
	static void OnInteractionContinuing(FValidInteractionSearch& ValidInteractionSearch, const UPoseSearchInteractionSubsystem* InteractionSubsystem);
	static void OnInteractionEnd(FValidInteractionSearch& ValidInteractionSearch, const UPoseSearchInteractionSubsystem* InteractionSubsystem);

	// mapping from FValidInteractionSearch::GetHashForEquivalence to FValidInteractionSearch
	TMap<uint32, FValidInteractionSearch> Searches;

	// mapping from FValidInteractionSearch::GetContextHashForEquivalence to FValidInteractionSearch::GetHashForEquivalence(s)
	// the map Key represents what AnimContexts (e.g. AnimInstance(s), that can be confused with Actors...) can be interacting. 
	// each map Value represents a possible interaction (associated to a FValidInteractionSearch), and only one will be a valid search,
	// and all the others are searches that still has been performed (and are valid)
	TMap<uint32, TArray<uint32>> ContextsToSearches;

#if ENABLE_ANIM_DEBUG
	UE_MT_DECLARE_TS_RW_ACCESS_DETECTOR(ThreadSafeCounter);
#endif // ENABLE_ANIM_DEBUG
};


// Experimental, this feature might be removed without warning, not for production use
// stack allocated temporary struct used to generate interaction contexts, then the FInteractionSearchContextBase portion will be 
// stored including DisabledCollisions as FValidInteractionSearch
struct FInteractionSearchContext : public FInteractionSearchContextBase
{
	void Add(const UObject* AnimContext, const IPoseHistory* PoseHistory
		, const FRole Role, bool bWantsDisableCollisions, int32 TickPriority
#if ENABLE_ANIM_DEBUG
		, float InBroadPhaseRadius, float InBroadPhaseRadiusIncrementOnInteraction
#endif // ENABLE_ANIM_DEBUG
	);

	int32 GetTickPriority(int32 Index) const { return TickPriorities[Index]; }

	FPoseSearchContinuingProperties GetContinuingProperties() const;

	bool IsContinuingInteraction() const { return bIsContinuingInteraction; }
	bool IsContinuingContextInteraction() const { return bIsContinuingContextInteraction; }
#if ENABLE_ANIM_DEBUG
	float GetDebugBroadPhaseRadius(int32 Index) const;
#endif // ENABLE_ANIM_DEBUG
	void SetContinuingInteraction(bool bInIsContinuingInteraction) { bIsContinuingInteraction = bInIsContinuingInteraction; }
	void SetContinuingContextInteraction(bool bInIsContinuingContextInteraction) { bIsContinuingContextInteraction = bInIsContinuingContextInteraction; }
	void AddReferencedObjects(FReferenceCollector& Collector);
	void SetContinuingProperties(float InPlayingAssetAccumulatedTime, const UObject* InPlayingAsset,
		bool bInIsPlayingAssetMirrored, const FVector& InPlayingAssetBlendParameters, EPoseSearchInterruptMode InInterruptMode);

private:
	TObjectPtr<const UObject> PlayingAsset;
	float PlayingAssetAccumulatedTime = 0.f;
	bool bIsPlayingAssetMirrored = false;
	FVector PlayingAssetBlendParameters = FVector::ZeroVector;
	EPoseSearchInterruptMode InterruptMode = EPoseSearchInterruptMode::DoNotInterrupt;
	bool bIsContinuingInteraction = false;
	bool bIsContinuingContextInteraction = false;
	TArray<int32, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> TickPriorities;

#if ENABLE_ANIM_DEBUG
	TArray<float, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> DebugBroadPhaseRadius;
	TArray<float, TInlineAllocator<PreallocatedRolesNum, TMemStackAllocator<>>> BroadPhaseRadiusIncrementOnInteraction;
#endif // ENABLE_ANIM_DEBUG
};

// Experimental, this feature might be removed without warning, not for production use
typedef TArray<FInteractionSearchContext, TInlineAllocator<PreallocatedSearchesNum, TMemStackAllocator<>>> FInteractionSearchContexts;

// Experimental, this feature might be removed without warning, not for production use
// FInteractionIsland contains ticks functions injected between the interacting actors TickActorComponents (UCharacterMovementComponent, or UCharacterMoverComponent)
// and PostTickComponent (USkeletalMeshComponent, or UAnimNextComponent)
// to create a execution threading fence to be able to perform motion matching searches between the involved characters in a thread safe manner.
// Look at UPoseSearchInteractionSubsystem "Execution model and threading details" for additional information
struct FInteractionIsland
{
	UE_NONCOPYABLE(FInteractionIsland);
	
	FInteractionIsland(ULevel* Level, UPoseSearchInteractionSubsystem* Subsystem);
	~FInteractionIsland();

	bool DoSearch_AnyThread(const UObject* AnimContext, FPoseSearchBlueprintResult& Result);
	bool GetResult_AnyThread(const UObject* AnimContext, FPoseSearchBlueprintResult& Result, bool bCompareOwningActors = false);

	const TArray<UActorComponent*>& GetTickActorComponents() const { return TickActorComponents; }
	const TArray<const UObject*>& GetIslandAnimContexts() const { return IslandAnimContexts; }
	const TArray<FInteractionSearchContext>& GetSearchContexts() const { return SearchContexts; }
	const TArray<FInteractionSearchResult>& GetSearchResults() const { return SearchResults; }

	bool IsInitialized() const;
	void AddSearchContext(const FInteractionSearchContext& SearchContext);
	void Uninitialize(bool bValidateTickDependencies);

	bool HasTickDependencies() const;
	void InjectToActor(const UObject* AnimContext, bool bAddTickDependencies);
	
#if ENABLE_ANIM_DEBUG
	void LogTickDependencies() const;
#endif // ENABLE_ANIM_DEBUG

	void AddReferencedObjects(FReferenceCollector& Collector);
	void UpdateConstraints(float DeltaTime);
	bool GetConstraint(const UObject* AnimContext, FName SocketName, float& OutDesiredReach, FTransform& OutTransform, bool bCompareOwningActors = false) const;

private:
#if ENABLE_ANIM_DEBUG
	static void LogTickDependencies(const TConstArrayView<UActorComponent*> TickActorComponents, int32 InteractionIslandIndex);
	void ValidateTickDependenciesLeaks() const;
#endif // ENABLE_ANIM_DEBUG

	void AddTickDependencies(UActorComponent* TickActorComponent, bool bInIsMainActor);
	void RemoveTickDependencies(bool bValidateTickDependencies);
	static IInteractionIslandDependency* FindCustomDependency(UActorComponent* InTickComponent);

	const FInteractionSearchResult* GetInteractionSearchResult_AnyThread(const UObject* AnimContext, bool bCompareOwningActors, FRole& OutRole) const;

	const UObject* GetMainAnimContext() const;
	const AActor* GetMainActor() const;

	// running before main actor animation evaluation
	struct FPreTickFunction : public FTickFunction
	{
		virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
		virtual FString DiagnosticMessage() override { return TEXT("UE::PoseSearch::FInteractionIsland::FPreTickFunction"); }
		FInteractionIsland* Island = nullptr;
	};
	FPreTickFunction PreTickFunction;

	// running after main actor animation evaluation, and before any NON main actor animation evaluation
	struct FPostTickFunction : public FTickFunction
	{
		virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
		virtual FString DiagnosticMessage() override { return TEXT("UE::PoseSearch::FInteractionIsland::FPostTickFunction"); }
		FInteractionIsland* Island = nullptr;
	};
	FPostTickFunction PostTickFunction;

	// running ALL actors animation evaluation
	struct FClosingTickFunction : public FTickFunction
	{
		virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
		virtual FString DiagnosticMessage() override { return TEXT("UE::PoseSearch::FInteractionIsland::FClosingTickFunction"); }
		FInteractionIsland* Island = nullptr;
	};
	FClosingTickFunction ClosingTickFunction;

	bool bHasTickDependencies = false;

	TArray<TObjectPtr<UActorComponent>> TickActorComponents;
	// UActorComponent with an ijected tick dependency to the ClosingTickFunction
	TArray<TObjectPtr<UActorComponent>> PostClosingTickActorComponents;

	// all the AnimContext in this island. Each SearchContexts will contain a subset of IslandAnimContexts 
	TArray<TObjectPtr<const UObject>> IslandAnimContexts;

	// there's one FSearchContext for each search we need to perform (including all the possible roles combinations). Added by UPoseSearchInteractionSubsystem::Tick
	// islands don't get deallocated, so this array once warmed up will not hit the allocator and waste cycles
	TArray<FInteractionSearchContext> SearchContexts;

	// SearchResults contains only the best results, and it has not necessarly the same cardinality as SearchContexts. usually SearchResults.Num() < SearchContexts.Num()
	// islands don't get deallocated, so this array once warmed up will not hit the allocator and waste cycles
	TArray<FInteractionSearchResult> SearchResults;
	bool bSearchPerfomed = false;

	UPoseSearchInteractionSubsystem* InteractionSubsystem = nullptr;

#if ENABLE_ANIM_DEBUG
	// used to analyze thread safety
	friend struct UE::PoseSearch::FInteractionValidator;
	FThreadSafeCounter InteractionIslandThreadSafeCounter = 0;
	FThreadSafeCounter TickFunctionsThreadSafeCounter = 0;
	bool bPreTickFunctionExecuted = false;
	bool bPostTickFunctionExecuted = false;
	bool bClosingTickFunctionExecuted = false;
#endif // ENABLE_ANIM_DEBUG
};

// Experimental, this feature might be removed without warning, not for production use
// Allows systems other than regular actor components to hook into interaction island dependencies 
struct IInteractionIslandDependency : public IModularFeature
{
	static inline FName FeatureName = "IInteractionIslandDependency";

	virtual ~IInteractionIslandDependency() = default;

	virtual bool CanMakeDependency(const UObject* InIslandObject, const UObject* InObject) const = 0;
	virtual const FTickFunction* FindTickFunction(UObject* InObject) const = 0;

	virtual void AddPrerequisite(UObject* InIslandObject, FTickFunction& InIslandTickFunction, UObject* InObject) const = 0;
	virtual void AddSubsequent(UObject* InIslandObject, FTickFunction& InIslandTickFunction, UObject* InObject) const = 0;
	virtual void RemovePrerequisite(UObject* InIslandObject, FTickFunction& InIslandTickFunction, UObject* InObject) const = 0;
	virtual void RemoveSubsequent(UObject* InIslandObject, FTickFunction& InIslandTickFunction, UObject* InObject) const = 0;
};

} // namespace UE::PoseSearch
