// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "PoseSearch/PoseSearchInteractionIsland.h"
#include "PoseSearchInteractionLibrary.generated.h"

#define UE_API POSESEARCH_API

// roles an AnimContext (currently only AnimInstance(s) are supported) can take during a MotionMatchMulti search
USTRUCT(Experimental, BlueprintType, Category = "Animation|Pose Search")
struct FPoseSearchAnimContextRoles
{
	GENERATED_BODY()

	// @todo: add UAF support, since we currently allow AnimContext of type UAnimInstance
	// @todo: add a UPROPERTY filter to select only UAnimInstance or UUAFComponent
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	TObjectPtr<UObject> AnimContext = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	TArray<FName> Roles;
};

// single query for a MotionMatchMulti search.
// Database should contain UMultiAnimAsset(s) (for example UPoseSearchInteractionAsset(s))
// AnimContextsRoles lists all the AnimContext that could partecipate in interactions and the roles they're willing to take (possible roles are defined in Database->Schema->Skeletons[i].Role)
USTRUCT(Experimental, BlueprintType, Category = "Animation|Pose Search")
struct FPoseSearchMotionMatchMultiQuery
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	TObjectPtr<const UPoseSearchDatabase> Database = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	TArray<FPoseSearchAnimContextRoles> AnimContextsRoles;
};

// Experimental, this feature might be removed without warning, not for production use
UCLASS(MinimalAPI, Experimental)
class UPoseSearchInteractionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	// function publishing this character (via its AnimInstance) FPoseSearchInteractionAvailability to the UPoseSearchInteractionSubsystem,
	// FPoseSearchInteractionAvailability represents the character availability to partecipate in an interaction with other characters for the next frame.
	// that means there will always be one frame delay between publiching availabilities and getting a result back from MotionMatchInteraction_Pure!
	// 
	// if FPoseSearchBlueprintResult has a valid SelectedAnimation, this will be the animation assigned to this character to partecipate in this interaction.
	// additional interaction properties, like assigned role, SelectedAnimation time, SearchCost, etc can be found within the result
	// ContinuingProperties are used to figure out the continuing pose and bias it accordingly. ContinuingProperties can reference directly the UMultiAnimAsset
	// or any of the roled UMultiAnimAsset::GetAnimationAsset, and the UPoseSearchInteractionSubsystem will figure out the related UMultiAnimAsset
	// PoseHistoryName is the name of the pose history node used for the associated motion matching search
	// if bValidateResultAgainstAvailabilities is true, the result will be invalidated if doesn't respect the new availabilities
	UFUNCTION(Experimental, BlueprintPure, Category = "Animation|Pose Search", meta = (BlueprintThreadSafe))
	static UE_API FPoseSearchBlueprintResult MotionMatchInteraction_Pure(TArray<FPoseSearchInteractionAvailability> Availabilities, const UObject* AnimContext, FName PoseHistoryName, bool bValidateResultAgainstAvailabilities = true);

	// BlueprintCallable version of MotionMatchInteraction_Pure
	UFUNCTION(Experimental, BlueprintCallable, Category = "Animation|Pose Search", meta = (BlueprintThreadSafe))
	static UE_API FPoseSearchBlueprintResult MotionMatchInteraction(TArray<FPoseSearchInteractionAvailability> Availabilities, const UObject* AnimContext, FName PoseHistoryName, bool bValidateResultAgainstAvailabilities = true);

	// this function calculates the mesh space transforms (OutFullAlignedTransforms) the AnimContext(s) should reach within a time of TimeOffset
	// to be consistently aligned to the selected UMultiAnimAsset (CurrentResult.SelectedAnim) at the selected time (CurrentResult.SelectedTime), 
	// given the interacting AnimContext(s) transforms (CurrentResult.ActorRootBoneTransforms, CurrentResult.ActorRootTransforms)
	// if bWarpUsingRootBone is true the function will use the AnimContext root bone transforms, otherwise the mesh space transforms
	UFUNCTION(Experimental, BlueprintCallable, Category = "Animation|Pose Search", meta = (BlueprintThreadSafe))
	static UE_API void CalculateFullAlignedTransforms(const FPoseSearchBlueprintResult& CurrentResult, float TimeOffset, bool bWarpUsingRootBone, TArray<FTransform>& OutFullAlignedTransforms);

	// this function calculates the mesh space transform for the AnimContext assigned the role CurrentResult.Role as CalculateFullAlignedTransforms does for all the AnimContext(s)
	UFUNCTION(Experimental, BlueprintCallable, Category = "Animation|Pose Search", meta = (BlueprintThreadSafe))
	static UE_API void CalculateFullAlignedTransform(const FPoseSearchBlueprintResult& CurrentResult, float TimeOffset, bool bWarpUsingRootBone, FTransform& OutFullAlignedTransform);

	// this function calculates the mesh space transform (OutNoAlignedTransform), or root bone transform if bWarpUsingRootBone is true,
	// the AnimContext will reach within a time of TimeOffset, if it continues playing the CurrentResult.SelectedAnim from CurrentResult.SelectedTime forward
	UFUNCTION(Experimental, BlueprintCallable, Category = "Animation|Pose Search", meta = (BlueprintThreadSafe))
	static UE_API void CalculateNoAlignedTransform(const FPoseSearchBlueprintResult& CurrentResult, float TimeOffset, bool bWarpUsingRootBone, FTransform& OutNoAlignedTransform);

	UE_EXPERIMENTAL(5.8, "Experimental, this feature might be removed without warning, not for production use")
	static UE_API void MotionMatchInteraction(FPoseSearchBlueprintResult& Result, const TArrayView<const FPoseSearchInteractionAvailability> Availabilities, const UObject* AnimContext, FName PoseHistoryName, const UE::PoseSearch::IPoseHistory* PoseHistory, bool bValidateResultAgainstAvailabilities = true);
	UE_EXPERIMENTAL(5.8, "Experimental, this feature might be removed without warning, not for production use")
	static UE_API void MotionMatchInteraction(FPoseSearchBlueprintResult& InOutResult, const TArrayView<const FPoseSearchInteractionAvailability> Availabilities, const UObject* AnimContext, FName PoseHistoryName, const UE::PoseSearch::IPoseHistory* PoseHistory, bool bValidateResultAgainstAvailabilities, bool bKeepInteractionAlive, float BlendTime, float DeltaTime);

	UFUNCTION(Experimental, BlueprintPure, Category = "Animation|Pose Search", meta = (BlueprintThreadSafe))
	static UE_API FPoseSearchContinuingProperties GetMontageContinuingProperties(UAnimInstance* AnimInstance);

	UFUNCTION(Experimental, BlueprintPure, Category = "Animation|Pose Search", meta = (BlueprintThreadSafe))
	static UE_API void GetMotionMatchInteractionConstraint(const UObject* AnimContext, FName SocketName, float& OutDesiredReach, FTransform& OutTransform, bool& IsValid, bool bCompareOwningActors = true);

	// function performing all the possible MotionMatching searches between the AnimContexts (eg AnimInstance(s)) defined in MotionMatchMultiQueries and 
	// compiled from the valid combination of the AnimContexts allowed Roles.
	// Results will output the best mathing options for all those AnimContexts willing to partecipate in interactions.
	// Users can iterate over Results, and assign the Results[i].SelectedAnim playback to the corresponding GetActor(Results[i])
	UFUNCTION(Experimental, BlueprintCallable, Category = "Animation|Pose Search", meta = (AdvancedDisplay = "ContinuingProperties, Future, EventToSearch"))
	static UE_API void MotionMatchMulti(
		TArray<FPoseSearchMotionMatchMultiQuery> MotionMatchMultiQueries,
		const FName PoseHistoryName, 
		const FPoseSearchContinuingProperties ContinuingProperties,
		TArray<FPoseSearchBlueprintResult>& Results);

	UE_EXPERIMENTAL(5.8, "Experimental, this feature might be removed without warning, not for production use")
	static UE_API void UpdateConstraints(TArray<FPoseSearchConstraint>& InOutConstraints, const UMultiAnimAsset* MultiAnimAsset, float DeltaTime, float AssetTime, bool bIsMirrored, const FVector& BlendParameters, const TConstArrayView<TObjectPtr<const UObject>> AnimContexts, const TConstArrayView<UE::PoseSearch::FRole> Roles);

	UFUNCTION(Experimental, BlueprintCallable, Category = "Animation|Pose Search")
	static UE_API void UpdateConstraints(UPARAM(ref) TArray<FPoseSearchConstraint>& InOutConstraints, float DeltaTime, const UMultiAnimAsset* MultiAnimAsset, const TArray<UObject*> AnimContexts, float SelectedTime, bool bIsMirrored, FVector BlendParameters);

	UFUNCTION(Experimental, BlueprintPure, Category = "Animation|Pose Search", meta = (BlueprintThreadSafe))
	static UE_API void GetConstraint(const TArray<FPoseSearchConstraint>& Constraints, FName AnimContextRole, FName SocketName, float& OutDesiredReach, FTransform& OutTransform, bool& IsValid);
};

#undef UE_API
