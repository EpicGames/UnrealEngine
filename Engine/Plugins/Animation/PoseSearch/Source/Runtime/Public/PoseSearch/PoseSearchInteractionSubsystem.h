// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchInteractionLibrary.h"
#include "PoseSearch/PoseSearchInteractionValidator.h"
#include "Subsystems/WorldSubsystem.h"
#include "PoseSearchInteractionSubsystem.generated.h"

#define UE_API POSESEARCH_API

struct FPoseSearchContinuingProperties;
class UAnimInstance;
class UPoseSearchDatabase;

namespace UE::PoseSearch
{
struct FAnimContextInfos;
struct FTagToDatabases;
} // namespace UE::PoseSearch

DECLARE_MULTICAST_DELEGATE_OneParam(FOnPoseSearchInteractionEvent, const UE::PoseSearch::FValidInteractionSearch&);

// World subsystem accepting the publication of characters (via their AnimInstance(s)) FPoseSearchInteractionAvailability, representing the characters willingness to partecipate in an 
// interaction with other characters from the next frame forward via Query_AnyThread method.
//
// The same method will return the FPoseSearchBlueprintResult from the PREVIOUS Tick processing (categorization of FPoseSearchInteractionAvailability(s) in multiple FInteractionIsland(s)),
// to the requesting character, containing the animation to play at what time, and the assigned role to partecipate in the selected interaction within the assigned 
// FInteractionIsland

// Execution model and threading details:
/////////////////////////////////////////
// 
// - by calling UPoseSearchInteractionLibrary::MotionMatchInteraction_Pure(TArray<FPoseSearchInteractionAvailability> Availabilities, UObject* AnimInstance), characters publish their availabilities
//   to partecipate in interactions to the UPoseSearchInteractionSubsystem
// - UPoseSearchInteractionSubsystem::Tick processes those FPoseSearchInteractionAvailability(s) and creates/updates UE::PoseSearch::FInteractionIsland. For each FInteractionIsland it injects 
//   a tick prerequisite via FInteractionIsland::InjectToActor (that calls AddPrerequisite) to all the Actors in the same island.
//   NoTe: the next frame the execution will be:
//			for each island[k]
//			{
//				for each Actor[k][i]
//				{
//					Tick all the TickActorComponents prerequisites, such as CharacterMovementComponent[k][i] (or Mover) in parallel
//				}
// 
//				Tick Island[k].PreTickFunction (that eventually generates the trajectories with all the updated CMCs or Mover)
//
//				Tick Actor[k][0] to have CBP running before and ABP or UAF on the same actor 0
//				Tick Actor[k][0].SkeletalMeshComponent (or AnimNextComponent, that performs the MotionMatchInteraction queries for all the involved actors via DoSearch_AnyThread)
// 
//				Tick Island[k].PostTickFunction (currently just a threading fence for the execution of all the other SkeletalMeshComponent(s) or AnimNextComponent(s))
// 
//				for each Actor[k][i]
//				{
//					if (i != 0)
//					{
//						Tick Actor[k][i] to have CBP running before and ABP or UAF on the same actor i
// 						Tick SkeletalMeshComponent[k][i] (or AnimNextComponent(s) that DoSearch_AnyThread get the cached result calculated by Tick Actor[k][0].SkeletalMeshComponent) in parallel
//					}
//				}
// 
//				Tick Island[k].ClosingTickFunction (that calculates all the necessary constraints)
// 
//				for each PostClosingTickActorComponent (stored in FInteractionIsland::PostClosingTickActorComponents) with tick dependencies to SkeletalMeshComponent[k][i] (or AnimNextComponent(s))
//				{
//					Tick PostClosingTickActorComponent
//				}
//			}
// - next frame UPoseSearchInteractionLibrary::MotionMatchInteraction_Pure(TArray<FPoseSearchInteractionAvailability> Availabilities, UObject* AnimInstance), with the context of all the published 
//   availabilities and created islands, will find the associated FInteractionIsland to the AnimInstance and call FInteractionIsland::DoSearch_AnyThread 
//   (via UPoseSearchInteractionSubsystem::Query_AnyThread) that will perform ALL (YES, ALL, so the bigger the island the slower the execution) the motion matching searches
//   for all the possible Actors / databases / Roles combinations, and populate FInteractionIsland::SearchResults with ALL the results for the island.
//   Ultimately the MotionMatchInteraction_Pure will return the SearchResults associated to the requesting AnimInstance with information about what animation to play
//   at what time with wich Role.

UCLASS(MinimalAPI, Experimental, Category="Animation|Pose Search")
class UPoseSearchInteractionSubsystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()

public:
	static UE_API UPoseSearchInteractionSubsystem* GetSubsystem_AnyThread(const UObject* AnimInstance);
	static UE_API UPoseSearchInteractionSubsystem* GetSubsystem_AnyThread(const UWorld* World);

	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;

	// it processes FPoseSearchInteractionAvailability(s) and creates/updates FInteractionIsland
	UE_API virtual void Tick(float DeltaSeconds) override;
	
	UE_API virtual TStatId GetStatId() const override;

	// publishing FPoseSearchInteractionAvailability(s) for the requesting character (via AnimContext as UAnimInstance or UAnimNextComponent) 
	// and getting the FPoseSearchBlueprintResult from the PREVIOUS Tick update
	// containing the animation to play at what time, and the assigned role to partecipate in the selected interaction.
	// Either a PoseHistoryName or a PoseHistory are required to perform the associated motion matching searches
	// if bValidateResultAgainstAvailabilities is true, the result will be validated against the currently published availabilities,
	// that could differ from the previous frame one for this AnimContext, and discarded if not valid within the context of the new availabilities
	UE_API void Query_AnyThread(const TArrayView<const FPoseSearchInteractionAvailability> Availabilities, const UObject* AnimContext, 
		FPoseSearchBlueprintResult& Result,	FName PoseHistoryName, const UE::PoseSearch::IPoseHistory* PoseHistory, bool bValidateResultAgainstAvailabilities);

	// use bCompareOwningActors = true in case you're querying the UPoseSearchInteractionSubsystem from a different AnimContext, 
	// for example when using this API from a different ABP from a different skeletal mesh component
	UE_API void GetResult_AnyThread(const UObject* AnimContext, FPoseSearchBlueprintResult& Result, bool bCompareOwningActors = false);

	TConstArrayView<UE::PoseSearch::FInteractionIsland*> GetInteractionIslands() const { return Islands; }

	// garbage collection handling
	static UE_API void AddReferencedObjects(UObject* This, FReferenceCollector& Collector);

	const TConstArrayView<FPoseSearchConstraint> GetConstraints(const UE::PoseSearch::FInteractionSearchContext& SearchContext) const;

	// use bCompareOwningActors = true in case you're querying the UPoseSearchInteractionSubsystem from a different AnimContext, 
	// for example when using this API from a different ABP from a different skeletal mesh component
	bool GetConstraint(const UObject* AnimContext, FName SocketName, float& OutDesiredReach, FTransform& OutTransform, bool bCompareOwningActors = false);

	TConstArrayView<UE::PoseSearch::FInteractionIsland*> GetIslands() const { return Islands; }

	FOnPoseSearchInteractionEvent OnInteractionStartedDelegate;
	FOnPoseSearchInteractionEvent OnInteractionContinuingDelegate;
	FOnPoseSearchInteractionEvent OnInteractionEndedDelegate;

private:
#if ENABLE_ANIM_DEBUG
	friend struct UE::PoseSearch::FInteractionValidator;
#endif // ENABLE_ANIM_DEBUG

	UE::PoseSearch::FInteractionIsland& CreateIsland();
	UE::PoseSearch::FInteractionIsland& GetAvailableIsland();

	void DestroyIsland(int32 Index);
	void DestroyAllIslands();
	void RegenerateAllIslands(float DeltaSeconds);
	bool ShouldRegenerateAllIslands();

#if DO_CHECK
	bool ValidateAllIslands() const;
#endif // DO_CHECK

	UE::PoseSearch::FInteractionIsland* FindIsland(const UObject* AnimContext, bool bCompareOwningActors = false);
	
#if ENABLE_ANIM_DEBUG
	void DebugDrawIslands() const;
	void DebugLogTickDependencies() const;
#endif // ENABLE_ANIM_DEBUG

	void AddAvailabilities(const TArrayView<const FPoseSearchInteractionAvailability> Availabilities, const UObject* AnimContext, FName PoseHistoryName, const UE::PoseSearch::IPoseHistory* PoseHistory);
	void GenerateAnimContextInfosAndTagToDatabases(UE::PoseSearch::FAnimContextInfos& AnimContextInfos, UE::PoseSearch::FTagToDatabases& TagToDatabases) const;
	void GenerateSearchContexts(float DeltaSeconds, UE::PoseSearch::FInteractionSearchContexts& SearchContexts) const;

	// it could contain duplicated availabilities
	TArray<UE::PoseSearch::FInteractionAnimContextAvailabilities> AnimContextsAvailabilitiesBuffer;
	FThreadSafeCounter AnimContextsAvailabilitiesIndex = 0;
	int32 AnimContextsAvailabilitiesNum = 0;

	// array of groups of characters that needs to be anaylzed together for possible interactions
	TArray<UE::PoseSearch::FInteractionIsland*> Islands;

	UE::PoseSearch::FValidInteractionSearches ValidInteractionSearches;
};

#undef UE_API
