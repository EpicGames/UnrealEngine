// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchRole.h"

struct FPoseSearchBlueprintResult;
class UMultiAnimAsset;

namespace UE::PoseSearch
{

// Experimental, this feature might be removed without warning, not for production use
POSESEARCH_API int32 GetRoleIndex(const UMultiAnimAsset* MultiAnimAsset, const FRole& Role);

// Experimental, this feature might be removed without warning, not for production use
FRoleToIndex MakeRoleToIndex(const UMultiAnimAsset* MultiAnimAsset);
	
// Experimental, this feature might be removed without warning, not for production use
POSESEARCH_API void CalculateFullAlignedTransforms(const FPoseSearchBlueprintResult& Result, float Time, float TimeOffset, bool bWarpUsingRootBone, TArrayView<FTransform> OutFullAlignedTransforms);

// Experimental, this feature might be removed without warning, not for production use
POSESEARCH_API FTransform CalculateDeltaAlignment(const FTransform& MeshWithoutOffset, const FTransform& MeshWithOffset, const FTransform& FullAlignedTransform, float WarpingRotationRatio, float WarpingTranslationRatio);

// Experimental, this feature might be removed without warning, not for production use
template <typename EvaluateCombinationType>
static void GenerateCombinationsRecursive(int32 DataCardinality, int32 DataIndex, TArray<int32, TInlineAllocator<PreallocatedRolesNum>>& Combination, int32 CombinationIndex, EvaluateCombinationType EvaluateCombination)
{
	if (CombinationIndex == Combination.Num())
	{ 
		EvaluateCombination(Combination);
	}
	else if (DataIndex < DataCardinality)
	{
		Combination[CombinationIndex] = DataIndex;
		GenerateCombinationsRecursive(DataCardinality, DataIndex + 1, Combination, CombinationIndex + 1, EvaluateCombination);
		GenerateCombinationsRecursive(DataCardinality, DataIndex + 1, Combination, CombinationIndex, EvaluateCombination);
	}
} 

// Experimental, this feature might be removed without warning, not for production use
// generates all the possible (unique) combinations for the indexes of a set of DataCardinality number of elements grouped in tuples of CombinationCardinality size:
// the number of generated combinations P(DataCardinality, CombinationCardinality) = (DataCardinality! / (DataCardinality - CombinationCardinality)!) / CombinationCardinality!
// see: https://en.wikipedia.org/wiki/Combination
// 
// for example with a set with 3 (DataCardinality) elements, that we want to combine in pairs (2 CombinationCardinality),
// EvaluateCombination will be called (3! / (3-2)!) / 2! = 3 times with input parameters:
// EvaluateCombination([0, 1])
// EvaluateCombination([0, 2])
// EvaluateCombination([1, 2])

template <typename EvaluateCombinationType>
static void GenerateCombinations(int32 DataCardinality, int32 CombinationCardinality, EvaluateCombinationType EvaluateCombination)
{
	TArray<int32, TInlineAllocator<PreallocatedRolesNum>> Combination;
	Combination.SetNum(CombinationCardinality);
	GenerateCombinationsRecursive(DataCardinality, 0, Combination, 0, EvaluateCombination);
}

} // namespace UE::PoseSearch