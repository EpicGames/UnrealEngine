// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimTypes.h"

#define UE_API POSESEARCH_API

namespace UE::PoseSearch
{
#if ENABLE_ANIM_DEBUG
struct FInteractionIsland;

// Experimental, this feature might be removed without warning, not for production use
struct FInteractionValidator
{
	UE_API explicit FInteractionValidator(const UObject* AnimContext);
	UE_API explicit FInteractionValidator(FInteractionIsland* InValidatingIsland);
	UE_API ~FInteractionValidator();

	const UObject* ValidatingAnimContext = nullptr;
	FInteractionIsland* ValidatingIsland = nullptr;
};

#define CheckInteractionThreadSafety(ValidationContext) const FInteractionValidator InteractionValidator(ValidationContext)
#else
#define CheckInteractionThreadSafety(ValidationContext)
#endif // ENABLE_ANIM_DEBUG

} // namespace UE::PoseSearch

#undef UE_API
