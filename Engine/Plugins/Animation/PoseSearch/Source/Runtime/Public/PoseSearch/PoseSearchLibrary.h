// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IObjectChooser.h"
#include "Animation/AnimExecutionContext.h"
#include "Animation/AnimNodeReference.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "PoseSearch/PoseSearchEvent.h"
#include "PoseSearch/PoseSearchHistory.h"
#include "PoseSearch/PoseSearchResult.h"
#include "PoseSearch/PoseSearchRole.h"
#include "SequenceEvaluatorLibrary.h"
#include "SequencePlayerLibrary.h"
#include "PoseSearchLibrary.generated.h"

#define UE_API POSESEARCH_API

struct FAnimationUpdateContext;
struct FAnimNode_PoseSearchHistoryCollector_Base;

namespace UE::PoseSearch
{
	struct FSearchContext;

	// Experimental, this feature might be removed without warning, not for production use
	UE_API FRole GetCommonDefaultRole(const TConstArrayView<TObjectPtr<const UPoseSearchDatabase>> Databases);
	// Experimental, this feature might be removed without warning, not for production use
	UE_API float CalculateWantedPlayRate(const FSearchResult& SearchResult, const FSearchContext& SearchContext, const FFloatInterval& PlayRate, float TrajectorySpeedMultiplier, const FPoseSearchEvent& EventToSearch);
} // namespace UE::PoseSearch


UENUM(BlueprintType)
enum class EPoseSearchInterruptMode : uint8
{
	// continuing pose search will be performed if valid
	DoNotInterrupt,

	// continuing pose search will be interrupted if its database is not listed in the searchable databases
	InterruptOnDatabaseChange,

	// continuing pose search will be interrupted if its database is not listed in the searchable databases, 
	// and continuing pose will be invalidated (forcing the schema to use pose history to build the query)
	InterruptOnDatabaseChangeAndInvalidateContinuingPose,

	// continuing pose search will always be interrupted
	ForceInterrupt,

	/// continuing pose search will always be interrupted
	// and continuing pose will be invalidated (forcing the schema to use pose history to build the query)
	ForceInterruptAndInvalidateContinuingPose,
};

USTRUCT()
struct FMotionMatchingState
{
	GENERATED_BODY()

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FMotionMatchingState() = default;
	FMotionMatchingState(const FMotionMatchingState& Other) = default;
	FMotionMatchingState(FMotionMatchingState&& Other) = default;
	FMotionMatchingState& operator=(const FMotionMatchingState& Other) = default;
	FMotionMatchingState& operator=(FMotionMatchingState&& Other) = default;
	~FMotionMatchingState() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Reset the state to a default state using the current Database
	POSESEARCH_API void Reset();

	FVector GetEstimatedFutureRootMotionVelocity() const;

	UPROPERTY(Transient)
	FPoseSearchBlueprintResult SearchResult;

	// Time since the last pose jump (real time in seconds, not a normalized time)
	float ElapsedPoseSearchTime = 0.f;

	UE_DEPRECATED(5.7, "this property has been moved into the pose history. Use IPoseHistory::GetPoseIndicesHistory to access it")
	UE::PoseSearch::FPoseIndicesHistory PoseIndicesHistory;
};

USTRUCT(Experimental, BlueprintType, Category="Animation|Pose Search")
struct FPoseSearchFutureProperties
{
	GENERATED_BODY()

	// Animation to play (it'll start at AnimationTime seconds)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	TObjectPtr<const UObject> Animation;

	// Start time for Animation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	float AnimationTime = 0.f;

	// Interval time before playing Animation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=State)
	float IntervalTime = 0.f;
};

USTRUCT(Experimental, BlueprintType, Category="Animation|Pose Search")
struct FPoseSearchContinuingProperties
{
	GENERATED_BODY()

	// Currently playing animation
	UPROPERTY(Transient, EditAnywhere, BlueprintReadWrite, Category=State)
	TObjectPtr<const UObject> PlayingAsset = nullptr;

	// Currently playing animation accumulated time
	UPROPERTY(Transient, EditAnywhere, BlueprintReadWrite, Category=State)
	float PlayingAssetAccumulatedTime = 0.f;

	UPROPERTY(Transient, EditAnywhere, BlueprintReadWrite, Category=State)
	bool bIsPlayingAssetMirrored = false;

	// PlayingAsset associated BlendParameters (if PlayingAsset is a UBlendSpace)
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category=State)
	FVector PlayingAssetBlendParameters = FVector::ZeroVector;

	UPROPERTY(Transient, EditAnywhere, BlueprintReadWrite, Category=State)
	EPoseSearchInterruptMode InterruptMode = EPoseSearchInterruptMode::DoNotInterrupt;

	UPROPERTY(Transient, EditAnywhere, BlueprintReadWrite, Category=State)
	bool bIsContinuingInteraction = false;

	UPROPERTY(Transient, EditAnywhere, BlueprintReadWrite, Category=State)
	bool bIsContinuingContextInteraction = false;
	
	// database where the PlayingAsset was originally from
	// (optional property, used to provide an additional database to search for the continuing pose associated to PlayingAsset)
	UPROPERTY(Transient, EditAnywhere, BlueprintReadWrite, Category=State)
	TObjectPtr<const UPoseSearchDatabase> PlayingAssetDatabase = nullptr;

	// Experimental, this feature might be removed without warning, not for production use
	UE_API void InitFrom(const FPoseSearchBlueprintResult& SearchResult, EPoseSearchInterruptMode InInterruptMode);
};

UCLASS(MinimalAPI)
class UPoseSearchLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

#if UE_POSE_SEARCH_TRACE_ENABLED

	// deprecate in favor of TraceMotionMatching using FSearchResults once we settle on FSearchResults API signatures
	static void TraceMotionMatching(
		UE::PoseSearch::FSearchContext& SearchContext,
		const UE::PoseSearch::FSearchResult& SearchResult,
		float ElapsedPoseSearchTime,
		float DeltaTime,
		bool bSearch,
		float WantedPlayRate,
		EPoseSearchInterruptMode InterruptMode);

	// Experimental, this feature might be removed without warning, not for production use
	static void TraceMotionMatching(
		UE::PoseSearch::FSearchContext& SearchContext,
		const UE::PoseSearch::FSearchResults& SearchResults,
		float ElapsedPoseSearchTime,
		float WantedPlayRate,
		EPoseSearchInterruptMode InterruptMode);
#endif // UE_POSE_SEARCH_TRACE_ENABLED

public:
	/**
	* Implementation of the core motion matching algorithm
	*
	* @param AnimContext					Input animation context, either UAnimInstance or UAnimNextComponent
	* @param PoseHistory					Input IPoseHistory, used to gather historical information about trajectory and bone transforms
	* @param Databases						Input array of databases to search
	* @param DeltaTime						Input DeltaTime
	* @param PoseJumpThresholdTime			Input don't jump to poses of the same segment that are within the interval this many seconds away from the continuing pose.
	* @param PoseReselectHistory			Input prevent re-selection of poses that have been selected previously within this much time (in seconds) in the past. This is across all animation segments that have been selected within this time range.
	* @param SearchThrottleTime				Input minimum amount of time to wait between searching for a new pose segment. It allows users to define how often the system searches, default for locomotion is searching every update, but you may only want to search once for other situations, like jump.
	* @param PlayRate						Input effective range of play rate that can be applied to the animations to account for discrepancies in estimated velocity between the movement modeland the animation.
	* @param InOutMotionMatchingState		Input/Output encapsulated motion matching algorithm and state
	* @param InterruptMode					Input continuing pose search interrupt mode
	* @param bShouldUseCachedChannelData	Input if true, motion matching will try to reuse the continuing pose channels features across multiple schemas without querying the pose history
	* @param bDebugDrawQuery				Input draw the composed query if valid
	* @param bDebugDrawCurResult			Input draw the current result if valid
	*/
	UE_DEPRECATED(5.7, "Use MotionMatch API instead")
	static UE_API void UpdateMotionMatchingState(
		FChooserEvaluationContext* AnimContext,
		const UE::PoseSearch::IPoseHistory* PoseHistory,
		const TConstArrayView<TObjectPtr<const UPoseSearchDatabase>> Databases,
		float DeltaTime,
		const FFloatInterval& PoseJumpThresholdTime,
		float PoseReselectHistory,
		float SearchThrottleTime,
		const FFloatInterval& PlayRate,
		FMotionMatchingState& InOutMotionMatchingState,
		EPoseSearchInterruptMode InterruptMode,
		bool bShouldUseCachedChannelData,
		bool bDebugDrawQuery = false,
		bool bDebugDrawCurResult = false,
		const FPoseSearchEvent& EventToSearch = FPoseSearchEvent());
	
	/**
	* Implementation of the core motion matching algorithm
	*
	* @param AnimInstance					Input animation instance
	* @param AssetsToSearch					Input assets to search (UPoseSearchDatabase or any animation asset containing UAnimNotifyState_PoseSearchBranchIn)
	* @param PoseHistoryName				Input tag of the associated PoseSearchHistoryCollector node in the anim graph
	* @param ContinuingProperties			Input struct specyfying the currently playing animation to be able to bias the search in selecting the continuing pose
	* @param Future							Input future properties to match (animation / start time / time offset)
	* @param SelectedAnimation				Output selected animation from the Database asset
	* @param Result							Output FPoseSearchBlueprintResult with the search result
	*/
	UFUNCTION(Experimental, BlueprintPure, Category = "Animation|Pose Search", meta = (BlueprintThreadSafe, AdvancedDisplay = "ContinuingProperties, Future", Keywords = "PoseMatch"))
	static UE_API void MotionMatch(
		UAnimInstance* AnimInstance,
		TArray<UObject*> AssetsToSearch,
		const FName PoseHistoryName,
		const FPoseSearchContinuingProperties ContinuingProperties,
		const FPoseSearchFutureProperties Future,
		FPoseSearchBlueprintResult& Result);

	// deprecate in favor of MotionMatch with the FSearchResults parameter once we settle on FSearchResults API signatures
	static UE_API UE::PoseSearch::FSearchResult MotionMatch(
		const TArrayView<const UObject*> AnimContexts,
		const TArrayView<const UE::PoseSearch::FRole> Roles,
		const TArrayView<const UE::PoseSearch::IPoseHistory*> PoseHistories, 
		const TArrayView<const UObject*> AssetsToSearch,
		const FPoseSearchContinuingProperties& ContinuingProperties,
		const FPoseSearchFutureProperties& Future,
		const FPoseSearchEvent& EventToSearch);

	UE_EXPERIMENTAL(5.8, "This API is experimental and might be removed without warning, not for production use")
	static UE_API void MotionMatch(
		const TArrayView<const UObject*> AnimContexts,
		const TArrayView<const UE::PoseSearch::FRole> Roles,
		const TArrayView<const UE::PoseSearch::IPoseHistory*> PoseHistories, 
		const TArrayView<const UObject*> AssetsToSearch,
		const FPoseSearchContinuingProperties& ContinuingProperties,
		const FPoseSearchFutureProperties& Future,
		const FPoseSearchEvent& EventToSearch,
		UE::PoseSearch::FSearchResults& SearchResults);

	UE_EXPERIMENTAL(5.8, "This API is experimental and might be removed without warning, not for production use")
	static UE_API void MotionMatch(
		const TArrayView<FChooserEvaluationContext> Contexts,
		const TArrayView<const UE::PoseSearch::FRole> Roles,
		const TArrayView<const UE::PoseSearch::IPoseHistory*> PoseHistories, 
		const TArrayView<const UObject*> AssetsToSearch,
		const FPoseSearchContinuingProperties& ContinuingProperties,
		const FPoseSearchFutureProperties& Future,
		const FPoseSearchEvent& EventToSearch,
		UE::PoseSearch::FSearchResults& SearchResults);
			
	UE_EXPERIMENTAL(5.8, "This API is experimental and might be removed without warning, not for production use")
	static UE_API void MotionMatch(
		const TArrayView<FChooserEvaluationContext> Contexts,
		const TArrayView<const UE::PoseSearch::FRole> Roles,
		const TArrayView<const UE::PoseSearch::IPoseHistory*> PoseHistories,
		const TArrayView<const UObject*> AssetsToSearch,
		const FPoseSearchContinuingProperties& ContinuingProperties,
		const float DesiredPermutationTimeOffset,
		const FPoseSearchEvent& EventToSearch,
		UE::PoseSearch::FSearchResults& SearchResults);

	UE_EXPERIMENTAL(5.8, "This API is experimental and might be removed without warning, not for production use")
	static UE_API void MotionMatch(
		const TArrayView<const UObject*> AnimContexts,
		const TArrayView<const UE::PoseSearch::FRole> Roles,
		const TArrayView<const UE::PoseSearch::IPoseHistory*> PoseHistories,
		const TArrayView<const UObject*> AssetsToSearch,
		const FPoseSearchContinuingProperties& ContinuingProperties,
		const float DesiredPermutationTimeOffset,
		const FPoseSearchEvent& EventToSearch,
		UE::PoseSearch::FSearchResults& SearchResults);

	UE_EXPERIMENTAL(5.8, "This API is experimental and might be removed without warning, not for production use")
	static UE_API void MotionMatch(
		UE::PoseSearch::FSearchContext& SearchContext,
		const TArrayView<const UObject*> AssetsToSearch,
		const FPoseSearchContinuingProperties& ContinuingProperties,
		UE::PoseSearch::FSearchResults& SearchResults);

	static UE_API const FAnimNode_PoseSearchHistoryCollector_Base* FindPoseHistoryNode(
		const FName PoseHistoryName,
		const UAnimInstance* AnimInstance);

	UFUNCTION(Experimental, BlueprintCallable, Category = "Animation|Pose Search", meta = (BlueprintThreadSafe))
	static UE_API void OverridePoseHistoryFromOwningMesh(UAnimInstance* AnimInstance, const FName PoseHistoryName);

	UFUNCTION(Experimental, BlueprintPure, Category = "Animation|Pose Search", meta = (BlueprintThreadSafe))
	static UE_API void IsAnimationAssetLooping(const UObject* Asset, bool& bIsAssetLooping);

	UFUNCTION(Experimental, BlueprintPure, Category = "Animation|Pose Search", meta = (BlueprintThreadSafe))
	static UE_API void GetDatabaseTags(const UPoseSearchDatabase* Database, TArray<FName>& Tags);

	UFUNCTION(Experimental, BlueprintCallable, Category = "Animation|Pose Search", meta = (BlueprintThreadSafe))
	static UE_API void ClearPoseSearchDatabasesManagement();

	UFUNCTION(Experimental, BlueprintPure, Category = "Animation|Pose Search", meta = (BlueprintThreadSafe))
	static UE_API const AActor* GetActor(const FPoseSearchBlueprintResult& Result);
	
	UFUNCTION(Experimental, BlueprintPure, Category = "Animation|Pose Search", meta = (BlueprintThreadSafe))
	static UE_API const AActor* GetActorForRole(const FPoseSearchBlueprintResult& Result, const FName& Role);
};

#undef UE_API
