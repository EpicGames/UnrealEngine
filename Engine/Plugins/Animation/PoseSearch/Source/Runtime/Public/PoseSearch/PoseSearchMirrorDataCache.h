// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/MirrorDataTable.h"

#define UE_API POSESEARCH_API

class UMirrorDataTable;
struct FBoneContainer;
struct FCompactPose;

namespace UE::PoseSearch
{

// @todo: make a memstack allocator version of FMirrorDataCache
struct FMirrorDataCache
{
	UE_API FMirrorDataCache();
	// fast initialization to mirror only the root bone
	UE_API FMirrorDataCache(const UMirrorDataTable* InMirrorDataTable);
	UE_API FMirrorDataCache(const UMirrorDataTable* InMirrorDataTable, const FBoneContainer& BoneContainer);
	
	// fast initialization to mirror only the root bone
	UE_API void Init(const UMirrorDataTable* InMirrorDataTable);
	UE_API void Init(const UMirrorDataTable* InMirrorDataTable, const FBoneContainer& BoneContainer);
	UE_API void Reset();

	UE_API FTransform MirrorTransform(const FTransform& InTransform) const;
	UE_API void MirrorPose(FCompactPose& Pose) const;
	const UMirrorDataTable* GetMirrorDataTable() const { return MirrorDataTable.Get(); }

private:
	// Mirror data table pointer copied from Schema for convenience
	TWeakObjectPtr<const UMirrorDataTable> MirrorDataTable;

	// Compact pose format of Mirror Bone Map
	TCustomBoneIndexArray<FCompactPoseBoneIndex, FCompactPoseBoneIndex> CompactPoseMirrorBones;

	// Pre-calculated component space rotations of reference pose, which allows mirror to work with any joint orientation
	// Only initialized and used when a mirroring table is specified
	TCustomBoneIndexArray<FQuat, FCompactPoseBoneIndex> ComponentSpaceRefRotations;
};

} // namespace UE::PoseSearch

#undef UE_API
