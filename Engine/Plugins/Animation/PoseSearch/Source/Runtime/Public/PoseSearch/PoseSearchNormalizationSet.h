// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DataAsset.h"
#include "PoseSearchNormalizationSet.generated.h"

#define UE_API POSESEARCH_API

class UPoseSearchDatabase;

/**
* Data asset used to allow multiple Pose Search Databases to be normalized together.
*/
UCLASS(MinimalAPI, BlueprintType, Category = "Animation|Pose Search", meta = (DisplayName = "Pose Search Normalization Set"))
class UPoseSearchNormalizationSet : public UDataAsset
{
	GENERATED_BODY()
	
public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "NormalizationSet")
	TArray<TObjectPtr<const UPoseSearchDatabase>> Databases;

	UE_API void AddUniqueDatabases(TArray<const UPoseSearchDatabase*>& UniqueDatabases) const;
};

#undef UE_API
