// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimTypes.h"
#include "PoseSearch/PoseSearchCost.h"
#include "PoseSearch/PoseSearchRole.h"
#include "PoseSearchResult.generated.h"

#define UE_API POSESEARCH_API

class UAnimationAsset;
class UPoseSearchDatabase;
class UPoseSearchSchema;
struct FGameplayTag;
struct FPoseSearchBlueprintResult;

namespace UE::PoseSearch
{
struct FSearchIndexAsset;

struct FDatabasePoseIdx
{
	// pose index from Database associated to this search result
	int32 PoseIdx = INDEX_NONE;

	// database associated to this search result. Poses are derived data stored in Database->GetSearchIndex()
	TWeakObjectPtr<const UPoseSearchDatabase> Database;

	FDatabasePoseIdx() = default;
	bool IsValid() const { return PoseIdx != INDEX_NONE && Database != nullptr; }
	bool operator==(const FDatabasePoseIdx& Other) const { return PoseIdx == Other.PoseIdx && Database == Other.Database; }

	UE_API const FSearchIndexAsset* GetSearchIndexAsset(bool bMandatory = false) const;
	// Experimental, this feature might be removed without warning, not for production use
	UE_API const UAnimationAsset* GetCurrentResultAnimationAsset() const;
	// Experimental, this feature might be removed without warning, not for production use
	UE_API const UAnimationAsset* GetCurrentResultAnimationAsset(const FRole& Role) const;
};

struct FSearchResult : FDatabasePoseIdx
{
	// best cost of the currently selected PoseIdx (it could be equal to ContinuingPoseCost)
	FPoseSearchCost PoseCost;
	
	// Experimental, this feature might be removed without warning, not for production use
	// if this search result is from an event search, EventPoseIdx is the pose index at event time from Database associated to this search result
	// if it's not an event search result EventPoseIdx is INDEX_NONE
	int32 EventPoseIdx = INDEX_NONE;

	// time of the associated GetSearchIndexAsset from this search. During continuing pose searches,
	// AssetTime is resynchronized via UpdateWithNormalizedTime or UpdateWithRealTime, so it's not precisely the quantized time associated to PoseIdx
	UE_DEPRECATED(5.7, "AssetTime will be made private. Use GetAssetTime and SetAssetTime instead")
	float AssetTime = MAX_flt;

	// true if the search result is from a continuing pose search
	bool bIsContinuingPoseSearch = false;

#if WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED
	// debug information related to the brute force search, to be able to compare it versus a kdtree/vptree search
	FPoseSearchCost BruteForcePoseCost;
	int32 BestPosePos = 0;
#endif // WITH_EDITOR && ENABLE_ANIM_DEBUG && UE_POSE_SEARCH_TRACE_ENABLED

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FSearchResult() = default;
	FSearchResult(const FSearchResult& Other) = default;
	FSearchResult(FSearchResult&& Other) = default;
	FSearchResult& operator=(const FSearchResult& Other) = default;
	FSearchResult& operator=(FSearchResult&& Other) = default;

	static bool IsAssetTimeValid(float AssetTime) { return AssetTime != MAX_flt; }
	bool IsAssetTimeValid() const { return IsAssetTimeValid(AssetTime); }
	float GetAssetTime() const { check(IsAssetTimeValid(AssetTime)); return AssetTime; }
	void SetAssetTime(float InAssetTime) { check(IsAssetTimeValid(InAssetTime)); AssetTime = InAssetTime; }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Experimental, this feature might be removed without warning, not for production use
	// Attempts to set the internal state to match the provided asset time including updating the internal DbPoseIdx. 
	// If the provided asset time is out of bounds for the currently playing asset then this function will reset the 
	// state back to the default state.
	void UpdateWithNormalizedTime(float NormalizedTime);
	
	// Experimental, this feature might be removed without warning, not for production use
	void UpdateWithRealTime(float RealTime);

	// Experimental, this feature might be removed without warning, not for production use
	bool DebugValidate() const;

	// Experimental, this feature might be removed without warning, not for production use
	bool IsEventSearchResult() const
	{
		return EventPoseIdx != INDEX_NONE;
	}

	// Experimental, this feature might be removed without warning, not for production use
	// returns true if EventPoseIdx is from the input EventTag
	bool IsEventSearchFromTag(const FGameplayTag& EventTag) const;
	
	// Experimental, this feature might be removed without warning, not for production use
	float CalculateTimeToEvent() const;

	void Reset() { *this = FSearchResult(); }

	bool operator==(const FSearchResult& Other) const
	{
		// best cost of the currently selected PoseIdx (it could be equal to ContinuingPoseCost)
		return static_cast<const FDatabasePoseIdx&>(*this) == static_cast<const FDatabasePoseIdx&>(Other) &&
			PoseCost == Other.PoseCost &&
			EventPoseIdx == Other.EventPoseIdx &&
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			AssetTime == Other.AssetTime &&
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			bIsContinuingPoseSearch == Other.bIsContinuingPoseSearch;
	}

	// Experimental, this feature might be removed without warning, not for production use
	UE_API void InitFrom(const FPoseSearchBlueprintResult& BlueprintResult);
};

// Experimental, this feature might be removed without warning, not for production use
struct FSearchResults
{
	FSearchResults()
		: UpdateWith([](const FSearchResult& SearchResult) {})
		, FinalizeResults([]() {})
		, GetBestResult([]() { return FSearchResult(); })
		, ShouldPerformSearch([](float) { return true; })
		, IterateOverSearchResults([](const TFunctionRef<bool(FSearchResult& SearchResult)>) { return true; })
	{
	}

	virtual ~FSearchResults() = default;

	FSearchResults(const FSearchResults& Other) = delete;
	FSearchResults(FSearchResult&& Other) = delete;
	FSearchResults& operator=(const FSearchResults& Other) = delete;
	FSearchResults& operator=(FSearchResults&& Other) = delete;
	
	TFunction<void(const FSearchResult& SearchResult)> UpdateWith;
	TFunction<void()> FinalizeResults;
	TFunction<FSearchResult()> GetBestResult;
	
	// every search will ask FSearchResults if it should perform the search, knowing that its minimum cost will be SearchMinimumCost:
	// it's up to this method to perform early out optimizations if the holded results are already good enough fo skip the search
	TFunction<bool(float)> ShouldPerformSearch;

	// calls IterateOverSearchResultsFunction for each FSearchResult this FSearchResults is holding
	// is IterateOverSearchResultsFunction returns true, the iteration will stop and IterateOverSearchResults will return true otherwise will return false (useful to implement contains / find methods)
	TFunction<bool(const TFunctionRef<bool(FSearchResult& SearchResult)>)> IterateOverSearchResults;
};

// Experimental, this feature might be removed without warning, not for production use
struct FSearchResults_Single : public FSearchResults
{
	UE_API FSearchResults_Single();

private:
	FSearchResult SingleSearchResult;
};

// Experimental, this feature might be removed without warning, not for production use
struct FSearchResults_Multi : public FSearchResults
{
	explicit FSearchResults_Multi(int32 InMaxNumberOfWantedResults = 1);

private:
	// @todo: evaluate a max heap instead (using AlgoImpl::HeapSiftUp / AlgoImpl::HeapSiftDown)
	// SearchResults[0] is the best result (lowest FSearchResult::PoseCost), SearchResults.Last() is the worst result (higher FSearchResult::PoseCost)
	TArray<FSearchResult, TInlineAllocator<PreallocatedSearchesNum, TMemStackAllocator<>>> SearchResults;
	int32 MaxNumberOfWantedResults = 1;
};

// Experimental, this feature might be removed without warning, not for production use
struct FSearchResults_AssetBests : public FSearchResults
{
	FSearchResults_AssetBests();
	const FSearchResult* FindSearchResultFor(const UPoseSearchDatabase* Database, int32 SourceAssetIdx) const;
	
	// shrinks FSearchResults_AssetBests down to MaxNumResults preserving the best results (the ones with less cost)
	void Shrink(int32 MaxNumResults);

	typedef TMap<int32, FSearchResult, TInlineSetAllocator<16, TMemStackSetAllocator<>>> FSourceAssetIdxToBestSearchResult;
	typedef TMap<const UPoseSearchDatabase*, FSourceAssetIdxToBestSearchResult, TInlineSetAllocator<4, TMemStackSetAllocator<>>> FPerDatabaseSourceAssetIdxToBestSearchResult;

private:
	FPerDatabaseSourceAssetIdxToBestSearchResult PerDatabaseSourceAssetIdxToBestSearchResult;
};

} // namespace UE::PoseSearch

USTRUCT(BlueprintType, Category="Animation|Pose Search")
struct FPoseSearchBlueprintResult
{
	GENERATED_BODY()

	// selected animation selected by the motion matching alghoritm that matches the query (trajectory + historical pose) the best
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category=State)
	TObjectPtr<UObject> SelectedAnim = nullptr;
	
	// SelectedAnim associated time (to have the best result out of the motion matchign selection, play the SelectedAnim at the SelectedTime)
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category=State)
	float SelectedTime = 0.f;
	
	// SelectedAnim at SelectedTime is from the motion matching continuing pose search
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category=State)
	bool bIsContinuingPoseSearch = false;
	
	// SelectedAnim suggested play rate (it's usually calculated by estimating the ratio between the trajectory speed in the query against the speed in the result)
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category=State)
	float WantedPlayRate = 1.f;

	// SelectedAnim associated looping state
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category=State)
	bool bLoop = false;
	
	// SelectedAnim associated mirror state
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category=State)
	bool bIsMirrored = false;
	
	// SelectedAnim associated BlendParameters (if SelectedAnim is a UBlendSpace)
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category=State)
	FVector BlendParameters = FVector::ZeroVector;

	// database containig the SelectedAnim
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category=State)
	TObjectPtr<const UPoseSearchDatabase> SelectedDatabase = nullptr;

	// associated motion matching search cost for this result (how bad the SelectedAnim at SelectedTime etc looks in comparison to the query. The bigger the number, the more different the two are)
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category=State)
	float SearchCost = MAX_flt;

	// bIsInteraction is true if this is a result from a motion matching interaction search
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category="Experimental|Interaction", meta = (Experimental))
	bool bIsInteraction = false;

	// assigned role to this character. If this result is from a motion matched interaction, every interacting Actors will be assigned a different Role
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category="Experimental|Interaction", meta = (Experimental))
	FName Role = UE::PoseSearch::DefaultRole;

	// assigned role index to this character. If this result is from a motion matched interaction, every interacting Actors will be assigned a different Role index.
	// It can be used to find this actor properies from ActorRootTransforms, ActorRootBoneTransforms, and AnimContexts
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category="Experimental|Interaction", meta = (Experimental))
	int32 RoleIndex = INDEX_NONE;

	// cached actors root transforms for all the roles in SelectedAnim (as UMultiAnimAsset). For all the optional missing roles (where the associated AnimContexts[i] is null), the ActorRootTransform will be identity
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category="Experimental|Interaction", meta = (Experimental))
	TArray<FTransform> ActorRootTransforms;

	// cached actors root bone transforms for all the roles in SelectedAnim (as UMultiAnimAsset). For all the optional missing roles (where the associated AnimContexts[i] is null), the ActorRootBoneTransform will be identity
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category="Experimental|Interaction", meta = (Experimental))
	TArray<FTransform> ActorRootBoneTransforms;

	// Experimental, this feature might be removed without warning, not for production use
	// AnimContexts involved in this interaction for all the roles in SelectedAnim (as UMultiAnimAsset). If the role is missing because optional the AnimContext will be null
	// BE CAREFUL HOW YOU USE THEM! since accessing data from other actors without proper tick dependencies leads to threading problems!
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category="Experimental|Interaction", meta = (Experimental))
	TArray<TObjectPtr<const UObject>> AnimContexts;

	// Experimental, this feature might be removed without warning, not for production use
	UE_API bool InitFrom(const UE::PoseSearch::FSearchResult& SearchResult, float InWantedPlayRate);

	// Experimental, this feature might be removed without warning, not for production use
	UE_API UAnimationAsset* GetAnimationAssetForRole();

	// Experimental, this feature might be removed without warning, not for production use
	bool operator==(const FPoseSearchBlueprintResult& Other) const;
};

// Experimental, this feature might be removed without warning, not for production use
template<> struct TStructOpsTypeTraits<FPoseSearchBlueprintResult> : public TStructOpsTypeTraitsBase2<FPoseSearchBlueprintResult>
{
	enum { WithIdenticalViaEquality = true };
};

#undef UE_API
