// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchDefines.h"

namespace UE::PoseSearch
{
	using FRole = FName;
	static const FRole DefaultRole = FRole();

	// todo: enabling TMemStackSetAllocator?
	using FRoleToIndex = TMap<FRole, int32, TInlineSetAllocator<PreallocatedRolesNum /*, TMemStackSetAllocator<>*/>>;
	using FRoleToIndexPair = TPair<FRole, int32>;

	static bool IsValid(const FRoleToIndex& RoleToIndex)
	{
		for (int32 Index = 0; Index < RoleToIndex.Num(); ++Index)
		{
			bool bIndexFound = false;
			for (const FRoleToIndexPair& RoleToIndexPair : RoleToIndex)
			{
				if (RoleToIndexPair.Value == Index)
				{
					bIndexFound = true;
					break;
				}
			}
			if (!bIndexFound)
			{
				return false;
			}
		}
		return true;
	}

	static FRoleToIndex MakeRoleToIndex(const TConstArrayView<FRole> Roles)
	{
		FRoleToIndex RoleToIndex;
		RoleToIndex.Reserve(Roles.Num());
		for (int32 RoleIndex = 0; RoleIndex < Roles.Num(); ++RoleIndex)
		{
			RoleToIndex.Add(Roles[RoleIndex]) = RoleIndex;
		}
		return RoleToIndex;
	}
}



