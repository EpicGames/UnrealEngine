// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneContainer.h"
#include "Engine/DataAsset.h"
#include "PoseSearch/PoseSearchFeatureChannel.h"
#include "PoseSearchSchema.generated.h"

#define UE_API POSESEARCH_API

struct FBoneReference;
class UMirrorDataTable;

UENUM()
enum class EPoseSearchDataPreprocessor : int32
{
	// The data will be left untouched.
	None,

	// The data will be normalized against its deviation, and the user weights will be normalized to be a unitary vector.
	Normalize,

	// The data will be normalized against its deviation
	// Experimental, this feature might be removed without warning, not for production use
	NormalizeOnlyByDeviation UMETA(DisplayName = "Normalize Only By Deviation (Experimental)"),

	// same behavior as Normalize, but it'll index all the databases in the normalization set with the same schema
	// Experimental, this feature might be removed without warning, not for production use
	NormalizeWithCommonSchema UMETA(DisplayName = "Normalize With Common Schema (Experimental)"),
};


USTRUCT()
struct FPoseSearchRoledSkeleton
{
	GENERATED_BODY()

	// Declared to be able to disable deprecation warnings
	FPoseSearchRoledSkeleton() = default;
	UE_API FPoseSearchRoledSkeleton(const FPoseSearchRoledSkeleton&);
	UE_API FPoseSearchRoledSkeleton& operator=(const FPoseSearchRoledSkeleton&);

	// Skeleton Reference for Motion Matching Database assets. Must be set to a compatible skeleton to the animation data in the database.
	UPROPERTY(EditAnywhere, Category = "Schema")
	TObjectPtr<USkeleton> Skeleton;

	// Setting up and assigning a mirror data table will allow all your assets in your database to access the mirrored version of the data. This is required for mirroring to work with Motion Matching.
	UPROPERTY(EditAnywhere, Category = "Schema")
	TObjectPtr<UMirrorDataTable> MirrorDataTable;

	UPROPERTY(EditAnywhere, Category = "Schema")
	FName Role;

	UPROPERTY(Transient)
	TArray<FBoneReference> BoneReferences;

	UPROPERTY(Transient)
	TArray<FName> RequiredCurves;
};

/**
* Specifies the format of a pose search index. At runtime, queries are built according to the schema for searching.
*/
UCLASS(MinimalAPI, BlueprintType, Category = "Animation|Pose Search", meta = (DisplayName = "Pose Search Schema"), CollapseCategories)
class UPoseSearchSchema : public UDataAsset
{
	GENERATED_BODY()

public:
	// The update rate at which we sample the animation data in the database. The higher the SampleRate the more refined your searches will be, but the more memory will be required
	UPROPERTY(EditAnywhere, Category = "Schema", meta = (DisplayPriority = 3, ClampMin = "1", ClampMax = "240"))
	int32 SampleRate = 30;

private:
	UPROPERTY(EditAnywhere, Category = "Schema", meta = (DisplayPriority = 0))
	TArray<FPoseSearchRoledSkeleton> Skeletons;

	// Channels itemize the cost breakdown of the Schema in simpler parts such as position or velocity of a bones, or phase of limbs. The total cost of a query against an indexed database pose will be the sum of the combined channel costs
	UPROPERTY(EditAnywhere, Instanced, Category = "Schema")
	TArray<TObjectPtr<UPoseSearchFeatureChannel>> Channels;

	// FinalizedChannels gets populated with UPoseSearchFeatureChannel(s) from Channels and additional injected ones during the Finalize.
	UPROPERTY(Transient)
	TArray<TObjectPtr<UPoseSearchFeatureChannel>> FinalizedChannels;

public:
#if WITH_EDITORONLY_DATA
	// Type of operation performed to the full pose features dataset
	UPROPERTY(EditAnywhere, Category = "Schema", meta = (DisplayPriority = 2))
	EPoseSearchDataPreprocessor DataPreprocessor = EPoseSearchDataPreprocessor::Normalize;
#endif //WITH_EDITORONLY_DATA

	UPROPERTY(Transient)
	int32 SchemaCardinality = 0;

#if WITH_EDITORONLY_DATA
	// How many times the animation assets of the database using this schema will be indexed.
	UPROPERTY(EditAnywhere, Category = "Permutations", meta = (ClampMin = "1"))
	int32 NumberOfPermutations = 1;

	// Delta time between every permutation indexing.
	UPROPERTY(EditAnywhere, Category = "Permutations", meta = (ClampMin = "1", ClampMax = "240", EditCondition = "NumberOfPermutations > 1", EditConditionHides))
	int32 PermutationsSampleRate = 30;

	// Starting offset of the "PermutationTime" from the "SamplingTime" of the first permutation.
	// subsequent permutations will have PermutationTime = SamplingTime + PermutationsTimeOffset + PermutationIndex / PermutationsSampleRate.
	UPROPERTY(EditAnywhere, Category = "Permutations")
	float PermutationsTimeOffset = 0.f;
#endif // WITH_EDITORONLY_DATA

	// if true a padding channel will be added to make sure the data is 16 bytes (aligned) and padded, to facilitate performance improvements at cost of eventual additional memory
	UPROPERTY(EditAnywhere, Category = "Performance")
	bool bAddDataPadding = false;

	// If bInjectAdditionalDebugChannels is true, channels will be asked to inject additional channels into this schema.
	// the original intent is to add UPoseSearchFeatureChannel_Position(s) to help with the complexity of the debug drawing
	// (the database will have all the necessary positions to draw lines at the right location and time).
	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bInjectAdditionalDebugChannels = false;

#if WITH_EDITORONLY_DATA
	// if bDrawInjectAdditionalDebugChannels is true, all the channels added for debug purposes with 
	// bInjectAdditionalDebugChannels (as well as all those channels with an associated zero weight) will be drawn
	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bDrawInjectAdditionalDebugChannels = false;
#endif // WITH_EDITORONLY_DATA

	TConstArrayView<TObjectPtr<UPoseSearchFeatureChannel>> GetChannels() const { return FinalizedChannels; }

	UE_API void AddChannel(UPoseSearchFeatureChannel* Channel);
	UE_API void AddTemporaryChannel(UPoseSearchFeatureChannel* DependentChannel);

	template <typename FindPredicateType>
	const UPoseSearchFeatureChannel* FindChannel(FindPredicateType FindPredicate) const
	{
		return FindChannelRecursive(GetChannels(), FindPredicate);
	}

	template<typename ChannelType>
	const ChannelType* FindFirstChannelOfType() const
	{
		return static_cast<const ChannelType*>(FindChannel([this](const UPoseSearchFeatureChannel* Channel) -> const UPoseSearchFeatureChannel* { return Cast<ChannelType>(Channel); }));
	}

	template <typename IteratePredicateType>
	void IterateChannels(IteratePredicateType IteratePredicate) const
	{
		IterateChannelsRecursive(GetChannels(), IteratePredicate);
	}

	// UObject
	UE_API virtual void PostLoad() override;

	// Experimental, this feature might be removed without warning, not for production use
	UE_API int8 AddBoneReference(const FBoneReference& BoneReference, const UE::PoseSearch::FRole& Role, bool bDefaultWithRootBone);
	UE_API int8 AddBoneReference(const FBoneReference& BoneReference, const UE::PoseSearch::FRole& Role);
	UE_API int8 AddCurveReference(const FName& CurveReference, const UE::PoseSearch::FRole& Role);

	// Experimental, this feature might be removed without warning, not for production use
	int8 FindSchemaBoneIdx(const FName& BoneName, const UE::PoseSearch::FRole& Role, bool bDefaultWithRootBone) const;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	UE_API const UE::PoseSearch::FRole GetDefaultRole() const;
	const TArray<FPoseSearchRoledSkeleton>& GetRoledSkeletons() const { return Skeletons; }

	UE_API TConstArrayView<float> BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const;

	UE_API void AddSkeleton(USkeleton* Skeleton, UMirrorDataTable* MirrorDataTable = nullptr, const UE::PoseSearch::FRole& Role = UE::PoseSearch::DefaultRole);
	UE_API bool AreSkeletonsCompatible(const UPoseSearchSchema* Other) const;
	UE_API void AddDefaultChannels();

	UE_API const FPoseSearchRoledSkeleton* GetRoledSkeleton(const UE::PoseSearch::FRole& Role) const;
	UE_API FPoseSearchRoledSkeleton* GetRoledSkeleton(const UE::PoseSearch::FRole& Role);

	UE_API USkeleton* GetSkeleton(const UE::PoseSearch::FRole& Role) const;
	UE_API UMirrorDataTable* GetMirrorDataTable(const UE::PoseSearch::FRole& Role) const;
	UE_API TConstArrayView<FBoneReference> GetBoneReferences(const UE::PoseSearch::FRole& Role) const;

private:
	template <typename FindPredicateType>
	static const UPoseSearchFeatureChannel* FindChannelRecursive(TConstArrayView<TObjectPtr<UPoseSearchFeatureChannel>> Channels, FindPredicateType FindPredicate)
	{
		for (const TObjectPtr<UPoseSearchFeatureChannel>& ChannelPtr : Channels)
		{
			if (ChannelPtr)
			{
				if (const UPoseSearchFeatureChannel* Channel = FindPredicate(ChannelPtr))
				{
					return Channel;
				}

				if (const UPoseSearchFeatureChannel* Channel = FindChannelRecursive(ChannelPtr->GetSubChannels(), FindPredicate))
				{
					return Channel;
				}
			}
		}
		return nullptr;
	}

	template <typename IteratePredicateType>
	static void IterateChannelsRecursive(TConstArrayView<TObjectPtr<UPoseSearchFeatureChannel>> Channels, IteratePredicateType IteratePredicate)
	{
		for (const TObjectPtr<UPoseSearchFeatureChannel>& ChannelPtr : Channels)
		{
			if (ChannelPtr)
			{
				IteratePredicate(ChannelPtr);
				IterateChannelsRecursive(ChannelPtr->GetSubChannels(), IteratePredicate);
			}
		}
	}

	UE_API void Finalize();
	UE_API void ResetFinalize();
};

#undef UE_API
