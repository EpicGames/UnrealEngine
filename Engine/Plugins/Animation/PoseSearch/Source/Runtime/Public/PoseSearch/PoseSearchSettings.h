// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "PoseSearchSettings.generated.h"

/** Default PoseSearch settings. */
UCLASS(Config=Engine, defaultconfig, meta=(DisplayName="Pose Search"), MinimalAPI)
class UPoseSearchSettings : public UDeveloperSettings
{
	GENERATED_BODY()

	UPoseSearchSettings();

public:
	static POSESEARCH_API const UPoseSearchSettings& Get();

	/** Initial size for the interaction availabilities buffer */
	UPROPERTY(EditAnywhere, config, Category = "Interaction", meta = (ClampMin = "0"))
	int32 AvailabilitiesBufferSize = 8;

	/** Maximum number of skeletal-mesh preview actors spawned in the Pose Search Database editor viewport.
	 *  Lower this when previewing animations on complex meshes to keep the editor responsive. */
	UPROPERTY(EditAnywhere, config, Category = "Editor", meta = (ClampMin = "1"))
	int32 MaxDatabaseEditorPreviewActors = 50;
};