// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/CurveFloat.h"
#include "PoseSearch/PoseSearchTrajectoryTypes.h"
#include "Kismet/KismetSystemLibrary.h"
#include "PoseSearchTrajectoryLibrary.generated.h"

#define UE_API POSESEARCH_API

class UAnimInstance;
class UMotionWarpingComponent;
class IPoseSearchTrajectoryPredictorInterface;

struct FAnimMontageInstance;
struct FTransformTrajectory;
struct FTransformTrajectorySample;

USTRUCT(BlueprintType)
struct FPoseSearchTrajectoryData
{
public:
	GENERATED_BODY()

	struct FState
	{
		float DesiredControllerYawLastUpdate = 0.f;
	};

	struct FDerived
	{
		float ControllerYawRate = 0.f;

		float MaxSpeed = 0.f;
		float BrakingDeceleration = 0.f;
		float BrakingSubStepTime = 0.f;
		float Friction = 0.f;

		FVector Velocity = FVector::ZeroVector;
		FVector Acceleration = FVector::ZeroVector;

		FVector Position = FVector::ZeroVector;
		FQuat Facing = FQuat::Identity;
		FQuat MeshCompRelativeRotation = FQuat::Identity;
		bool bOrientRotationToMovement = false;
		bool bStepGroundPrediction = true;
	};

	struct FSampling
	{
		int32 NumHistorySamples = 0;
		// if SecondsPerHistorySample <= 0, then we collect every update
		float SecondsPerHistorySample = 0.f;

		int32 NumPredictionSamples = 0;
		float SecondsPerPredictionSample = 0.f;
	};

	UE_API bool UpdateData(float DeltaTime, const FAnimInstanceProxy& AnimInstanceProxy, FDerived& TrajectoryDataDerived, FState& TrajectoryDataState) const;
	UE_API bool UpdateData(float DeltaTime, const UObject* Context, FDerived& TrajectoryDataDerived, FState& TrajectoryDataState) const;
	UE_API FVector StepCharacterMovementGroundPrediction(float DeltaTime, const FVector& InVelocity, const FVector& InAcceleration, const FDerived& TrajectoryDataDerived) const;
	
	// If the character is forward facing (i.e. bOrientRotationToMovement is true), this controls how quickly the trajectory will rotate
	// to face acceleration. It's common for this to differ from the rotation rate of the character, because animations are often authored 
	// with different rotation speeds than the character. This is especially true in cases where the character rotation snaps to movement.
	UPROPERTY(EditDefaultsOnly, Category = "Trajectory Settings")
	float RotateTowardsMovementSpeed = 10.f;

	// Maximum controller yaw  rate in degrees per second used to clamp the character owner controller desired yaw to generate the prediction trajectory.
	// Negative values disable the clamping behavior
	UPROPERTY(EditDefaultsOnly, Category = "Trajectory Settings")
	float MaxControllerYawRate = 70.f;

	// artificially bend character velocity towards acceleration direction to compute trajectory prediction, to get sharper turns
	// 0: character velocity is used with no alteration, 1: the acceleration direction is used as velocity direction
	UPROPERTY(EditDefaultsOnly, Category = "Trajectory Settings", meta = (ClampMin = "0", ClampMax = "1"))
	float BendVelocityTowardsAcceleration = 0.f;

	UPROPERTY(EditDefaultsOnly, Category = "Trajectory Settings", meta = (InlineEditConditionToggle))
	bool bUseSpeedRemappingCurve = false;

	UPROPERTY(EditDefaultsOnly, Category = "Trajectory Settings", meta = (EditCondition = "bUseSpeedRemappingCurve"))
	FRuntimeFloatCurve SpeedRemappingCurve;

	UPROPERTY(EditDefaultsOnly, Category = "Trajectory Settings", meta = (InlineEditConditionToggle))
	bool bUseAccelerationRemappingCurve = false;

	UPROPERTY(EditDefaultsOnly, Category = "Trajectory Settings", meta = (EditCondition = "bUseAccelerationRemappingCurve"))
	FRuntimeFloatCurve AccelerationRemappingCurve;
};

USTRUCT(BlueprintType)
struct FPoseSearchTrajectory_WorldCollisionResults
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Trajectory Settings")
	float TimeToLand  = 0.0f;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Trajectory Settings")
	float LandSpeed  = 0.0f;
};

/**
 * Set of functions to help populate a FTransformTrajectory for motion matching.
 */
UCLASS(MinimalAPI)
class UPoseSearchTrajectoryLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	UE_DEPRECATED(5.7, "CurrentTime is no longer needed or used. Use UpdateHistory_TransformHistory with no CurrentTime")
	static UE_API void UpdateHistory_TransformHistory(FTransformTrajectory& Trajectory,
										   FVector CurrentPosition,
										   FVector CurrentVelocity,
										   const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling,
										   float DeltaTime,
										   float CurrentTime);

	// Initialize history and predicted samples based on sampling settings and a default state
	static UE_API void InitTrajectorySamples(FTransformTrajectory& Trajectory, FVector DefaultPosition, FQuat DefaultFacing, const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling, float DeltaTime);
	
	// Update history by tracking offsets that result from character intent (e.g. movement component velocity) and applying
	// that to the current world transform. This works well on linearly-translating platforms as it only stores a history
	// of movement that results from character intent, not movement from platforms. For rotating platforms use
	// UpdateHistory_TransformHistoryWithRotation instead.
	// Important: CurrentVelocity should be the velocity relative to the ground as reported by the character movement component or character mover etc
	static UE_API void UpdateHistory_TransformHistory(FTransformTrajectory& Trajectory,
													  FVector CurrentPosition,
													  FVector CurrentVelocity,
													  const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling,
													  float DeltaTime);

	// Like UpdateHistory_TransformHistory, but also applies the angular analogue of the ground-translation correction so that
	// history stays anchored to rotating platforms. The inferred ground rotation (actual rotation delta with the character's
	// angular intent removed) is applied to every history sample, rotating around CurrentPosition.
	// CurrentFacing is the character's current world-space facing. CurrentAngularVelocityDegrees is the character's angular
	// velocity in world-space degrees per second (from the same source as CurrentVelocity - i.e. excluding the movement base
	// contribution, e.g. FMoverDefaultSyncState::GetAngularVelocityDegrees_WorldSpace).
	UE_EXPERIMENTAL(5.8, "Experimental pose search function, API may change in the future")
	static UE_API void UpdateHistory_TransformHistoryWithRotation(FTransformTrajectory& Trajectory,
													  FVector CurrentPosition,
													  FVector CurrentVelocity,
													  FQuat CurrentFacing,
													  FVector CurrentAngularVelocityDegrees,
													  const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling,
													  float DeltaTime);

	// Update prediction by simulating the movement math for ground locomotion from UCharacterMovementComponent.
	static UE_API void UpdatePrediction_SimulateCharacterMovement(FTransformTrajectory& Trajectory, const FPoseSearchTrajectoryData& TrajectoryData, const FPoseSearchTrajectoryData::FDerived& TrajectoryDataDerived, const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling, float DeltaTime);

	// Update prediction by simulating an active montage's root motion.
	UE_EXPERIMENTAL(5.8, "Experimental pose search function, API may change in the future")
	static UE_API void UpdatePrediction_SimulateMontage(const UObject* InContext,
															  const FAnimMontageInstance* MontageInstance,
															  FTransformTrajectory& Trajectory,
															  const FPoseSearchTrajectoryData& TrajectoryData,
															  const FPoseSearchTrajectoryData::FDerived& TrajectoryDataDerived,
															  const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling,
															  float DeltaTime,
															  bool bSimulateWarping = true,
															  bool bSimulateCharacterMovement = true,
															  UMotionWarpingComponent* InWarpingComponent = nullptr);

	// Experimental: Update the history purely based on current position, without taking into account ground velocities
	UE_EXPERIMENTAL(5.7, "Experimental pose search function, API may change in the future")
	static UE_API void UpdateHistory_WorldSpace(FTransformTrajectory& Trajectory,
	                                     const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling,
	                                     float DeltaTime);

	// Generates a prediction trajectory based of the current character intent. For use with Character actors.
	UFUNCTION(BlueprintCallable, Category = "Animation|PoseSearch", meta = (BlueprintThreadSafe, DisplayName = "Pose Search Generate Trajectory (for Character)"))
	static UE_API void PoseSearchGenerateTransformTrajectory(const UObject* InAnimInstance, 
		UPARAM(ref) const FPoseSearchTrajectoryData& InTrajectoryData, float InDeltaTime,
		UPARAM(ref) FTransformTrajectory& InOutTrajectory, UPARAM(ref) float& InOutDesiredControllerYawLastUpdate, FTransformTrajectory& OutTrajectory,
		float InHistorySamplingInterval = 0.04f, int32 InTrajectoryHistoryCount = 10, float InPredictionSamplingInterval = 0.2f, int32 InTrajectoryPredictionCount = 8);

	UE_EXPERIMENTAL(5.8, "Experimental pose search function, API may change in the future")
	UFUNCTION(BlueprintCallable, Category = "Animation|PoseSearch|Experimental", meta = (BlueprintThreadSafe, DisplayName = "Pose Search Generate Warped Trajectory (Experimental)"))
	static UE_API void PoseSearchGenerateWarpedTransformTrajectory(const UObject* InAnimInstance,
		UPARAM(ref) const FPoseSearchTrajectoryData& InTrajectoryData, float InDeltaTime,
		UPARAM(ref) FTransformTrajectory& InOutTrajectory, UPARAM(ref) float& InOutDesiredControllerYawLastUpdate, FTransformTrajectory& OutTrajectory,
		float InHistorySamplingInterval = 0.04f, int32 InTrajectoryHistoryCount = 10, float InPredictionSamplingInterval = 0.2f, int32 InTrajectoryPredictionCount = 8,
		bool bSimulateMontages = false, bool bSimulateWarping = false);

	// Generates a prediction trajectory based of the current movement intent. For use with predictors. InPredictor must implement IPoseSearchTrajectoryPredictorInterface
	UFUNCTION(BlueprintCallable, Category = "Animation|PoseSearch|Experimental", meta = (BlueprintThreadSafe, DisplayName = "Pose Search Generate Trajectory (using Predictor)", DeprecatedFunction, DeprecationMessage="Use PoseSearchGeneratePredictorTrajectory that does not take FPoseSearchTrajectoryData."))
	static UE_API void PoseSearchGeneratePredictorTransformTrajectory(UObject* InPredictor,
	UPARAM(ref) const FPoseSearchTrajectoryData& InTrajectoryData, float InDeltaTime,
	UPARAM(ref) FTransformTrajectory& InOutTrajectory, UPARAM(ref) float& InOutDesiredControllerYawLastUpdate, FTransformTrajectory& OutTrajectory,
	float InHistorySamplingInterval = 0.04f, int32 InTrajectoryHistoryCount = 10, float InPredictionSamplingInterval = 0.2f, int32 InTrajectoryPredictionCount = 8);
	
	// Generates a prediction trajectory based of the current movement intent. For use with predictors. InPredictor must implement IPoseSearchTrajectoryPredictorInterface
	UFUNCTION(BlueprintCallable, Category = "Animation|PoseSearch|Experimental", meta = (BlueprintThreadSafe, DisplayName = "Pose Search Generate Trajectory (using Predictor)"))
	static UE_API void PoseSearchGenerateTransformTrajectoryWithPredictor(TScriptInterface<IPoseSearchTrajectoryPredictorInterface> InPredictor,
		float InDeltaTime,
		UPARAM(ref) FTransformTrajectory& InOutTrajectory, UPARAM(ref) float& InOutDesiredControllerYawLastUpdate, FTransformTrajectory& OutTrajectory,
		float InHistorySamplingInterval = 0.04f, int32 InTrajectoryHistoryCount = 10, float InPredictionSamplingInterval = 0.2f, int32 InTrajectoryPredictionCount = 8);

	// Experimental: Process InTrajectory to apply gravity and handle collisions. Eventually returns the modified OutTrajectory.
	// If bApplyGravity is true, gravity from the UCharacterMovementComponent will be applied.
	// If FloorCollisionsOffset > 0, vertical collision will be performed to every sample of the trajectory to have the samples float over the geometry (by FloorCollisionsOffset).
	UE_EXPERIMENTAL(5.6, "Solution for CMC based workflow not Mover.")
	UFUNCTION(BlueprintCallable, Category="Animation|PoseSearch|Experimental", meta=(BlueprintThreadSafe, DisplayName="HandleTrajectoryWorldCollisions", WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", AdvancedDisplay="TraceChannel,bTraceComplex,ActorsToIgnore,DrawDebugType,bIgnoreSelf,MaxObstacleHeight,TraceColor,TraceHitColor,DrawTime"))
	static UE_API void HandleTransformTrajectoryWorldCollisions(const UObject* WorldContextObject, const UAnimInstance* AnimInstance, UPARAM(ref) const FTransformTrajectory& InTrajectory, bool bApplyGravity, float FloorCollisionsOffset, FTransformTrajectory& OutTrajectory, FPoseSearchTrajectory_WorldCollisionResults& CollisionResult,
		ETraceTypeQuery TraceChannel, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, bool bIgnoreSelf = true, float MaxObstacleHeight = 10000.f, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	// Experimental: Process InTrajectory to apply gravity and handle collisions. Eventually returns the modified OutTrajectory.
	// If bApplyGravity is true, GravityAccel will be applied.
	// If FloorCollisionsOffset > 0, vertical collision will be performed to every sample of the trajectory to have the samples float over the geometry (by FloorCollisionsOffset).
	UE_EXPERIMENTAL(5.6, "Solution for CMC based workflow not Mover.")
	UFUNCTION(BlueprintCallable, Category="Animation|PoseSearch|Experimental", meta=(BlueprintThreadSafe, DisplayName="HandleTrajectoryWorldCollisionsWithGravity", WorldContext="WorldContextObject", AutoCreateRefTerm="ActorsToIgnore", AdvancedDisplay="TraceChannel,bTraceComplex,ActorsToIgnore,DrawDebugType,bIgnoreSelf,MaxObstacleHeight,TraceColor,TraceHitColor,DrawTime"))
	static UE_API void HandleTransformTrajectoryWorldCollisionsWithGravity(const UObject* WorldContextObject, 
		UPARAM(ref) const FTransformTrajectory& InTrajectory, FVector StartingVelocity, bool bApplyGravity, FVector GravityAccel, float FloorCollisionsOffset, FTransformTrajectory& OutTrajectory, FPoseSearchTrajectory_WorldCollisionResults& CollisionResult,
		ETraceTypeQuery TraceChannel, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, bool bIgnoreSelf = true, float MaxObstacleHeight = 10000.f, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);

	UFUNCTION(BlueprintPure, Category="Animation|PoseSearch", meta = (BlueprintThreadSafe, DisplayName="GetTrajectorySampleAtTime"))
	static UE_API void GetTransformTrajectorySampleAtTime(UPARAM(ref) const FTransformTrajectory& InTrajectory, float Time, FTransformTrajectorySample& OutTrajectorySample, bool bExtrapolate = false);
	
	UFUNCTION(BlueprintPure, Category="Animation|PoseSearch", meta = (BlueprintThreadSafe, DisplayName="GetTrajectoryVelocity"))
	static UE_API void GetTransformTrajectoryVelocity(UPARAM(ref) const FTransformTrajectory& InTrajectory, float Time1, float Time2, FVector& OutVelocity, bool bExtrapolate = false);
	
	UFUNCTION(BlueprintPure, Category="Animation|PoseSearch", meta = (BlueprintThreadSafe, DisplayName="GetTrajectoryAngularVelocity"))
	static UE_API void GetTransformTrajectoryAngularVelocity(UPARAM(ref) const FTransformTrajectory& InTrajectory, float Time1, float Time2, FVector& OutAngularVelocity, bool bExtrapolate = false);
	
	UFUNCTION(BlueprintPure, Category="Animation|PoseSearch", meta = (BlueprintThreadSafe, DisplayName="GetTrajectorySampleTransform"))
	static UE_API FTransform GetTransformTrajectorySampleTransform(UPARAM(ref) const FTransformTrajectorySample& InTrajectorySample);
	
	UFUNCTION(BlueprintCallable, Category="Animation|PoseSearch|Experimental", meta=(DisplayName="DrawTrajectory", WorldContext="WorldContextObject"))
	static UE_API void DrawTransformTrajectory(const UObject* WorldContextObject, UPARAM(ref) const FTransformTrajectory& InTrajectory, const float DebugThickness, float HeightOffset);
	
private:

	static UE_API void UpdateHistory_TransformHistoryInternal(FTransformTrajectory& Trajectory,
														  FVector CurrentPosition,
														  FVector CurrentVelocity,
														  FQuat CurrentFacing,
														  FVector CurrentAngularVelocityDegrees,
														  bool bApplyAngularCorrection,
														  const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling,
														  float DeltaTime);

	static UE_API FVector RemapVectorMagnitudeWithCurve(const FVector& Vector, bool bUseCurve, const FRuntimeFloatCurve& Curve);

	static UE_API void UpdatePrediction_SimulateCharacterMovementInit(FTransformTrajectory& Trajectory, 
																	  FVector& OutCurrentPositionWS,
																	  FVector& OutCurrentVelocityWS,
																	  FVector& OutCurrentAccelerationWS,
																	  FQuat& OutCurrentFacingWS,
																	  const FPoseSearchTrajectoryData& TrajectoryData, 
																	  const FPoseSearchTrajectoryData::FDerived& TrajectoryDataDerived);

	static UE_API void UpdatePrediction_SimulateCharacterMovementStep(FTransformTrajectory& Trajectory, 
																	  FVector& InOutCurrentPositionWS,
																	  FVector& InOutCurrentVelocityWS,
																	  FVector& InOutCurrentAccelerationWS,
																	  FQuat& InOutCurrentFacingWS,
																	  const FPoseSearchTrajectoryData& TrajectoryData, 
																	  const FPoseSearchTrajectoryData::FDerived& TrajectoryDataDerived, 
																	  const FPoseSearchTrajectoryData::FSampling& TrajectoryDataSampling, 
																	  float& InOutAccumulatedSeconds);
};

#undef UE_API
