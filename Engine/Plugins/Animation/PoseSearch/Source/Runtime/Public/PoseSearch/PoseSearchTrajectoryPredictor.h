// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "PoseSearchTrajectoryPredictor.generated.h"

struct FTransformTrajectory;


UINTERFACE(MinimalAPI, BlueprintType, NotBlueprintable, Experimental)
class UPoseSearchTrajectoryPredictorInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * PoseSearchTrajectoryPredictor: API for an object to implement to act as a predictor of the future trajectory for motion matching animation purposes
 */
class IPoseSearchTrajectoryPredictorInterface : public IInterface
{
	GENERATED_BODY()

public:
	
	UFUNCTION(BlueprintCallable, Category = "Animation|PoseSearch|Experimental")
	virtual void Predict(FTransformTrajectory& InOutTrajectory, int32 NumPredictionSamples, float SecondsPerPredictionSample, int32 NumHistorySamples) = 0;

	UFUNCTION(BlueprintCallable, Category="Animation|PoseSearch|Experimental")
	virtual void GetGravity(FVector& OutGravityAccel) = 0;

	UFUNCTION(BlueprintCallable, Category = "Animation|PoseSearch|Experimental")
	virtual void GetCurrentState(FVector& OutPosition, FQuat& OutFacing, FVector& OutVelocity) = 0;

	UFUNCTION(BlueprintCallable, Category = "Animation|PoseSearch|Experimental")
	virtual void GetVelocity(FVector& OutVelocity) = 0;

	UE_EXPERIMENTAL(5.8, "Experimental pose search function, API may change in the future")
	UFUNCTION(BlueprintCallable, Category = "Animation|PoseSearch|Experimental")
	virtual void GetAngularVelocity(FVector& OutAngularVelocityDegrees) = 0;
};
