// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimTypes.h"
#include "PoseSearchTrajectoryTypes.generated.h"

#define UE_API POSESEARCH_API

struct FAnimInstanceProxy;
struct FTransformTrajectory;
struct FTransformTrajectorySample;


USTRUCT(BlueprintType, Category="Pose Search Trajectory")
struct UE_DEPRECATED(5.6, "FPoseSearchQueryTrajectorySample has been deprecated. Use FTransformTrajectorySample instead.")
FPoseSearchQueryTrajectorySample 
{
	GENERATED_BODY()

	UE_API FPoseSearchQueryTrajectorySample(const FQuat& InFacing, const FVector& InPosition, float InAccumulatedSeconds);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	FPoseSearchQueryTrajectorySample() = default;
	~FPoseSearchQueryTrajectorySample() = default;
	FPoseSearchQueryTrajectorySample(const FPoseSearchQueryTrajectorySample&) = default;
	FPoseSearchQueryTrajectorySample(FPoseSearchQueryTrajectorySample&&) = default;
	FPoseSearchQueryTrajectorySample& operator=(const FPoseSearchQueryTrajectorySample&) = default;
	FPoseSearchQueryTrajectorySample& operator=(FPoseSearchQueryTrajectorySample&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	UE_DEPRECATED(5.6, "Use FTransfromTrajectorySample instead. See FTransfromTrajectorySample::Facing.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use FTransfromTrajectorySample instead. See FTransfromTrajectorySample::Facing."))
	FQuat Facing = FQuat::Identity;

	UE_DEPRECATED(5.6, "Use FTransfromTrajectorySample instead. See FTransfromTrajectorySample::Position.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use FTransfromTrajectorySample instead. See FTransfromTrajectorySample::Position."))
	FVector Position = FVector::ZeroVector;

	UE_DEPRECATED(5.6, "Use FTransfromTrajectorySample instead. See FTransfromTrajectorySample::AccumulatedSeconds.")
	UPROPERTY( meta = (DeprecatedProperty, DeprecationMessage = "Use FTransfromTrajectorySample instead. See FTransfromTrajectorySample::AccumulatedSeconds."))
	float AccumulatedSeconds = 0.f;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_API FPoseSearchQueryTrajectorySample Lerp(const FPoseSearchQueryTrajectorySample& Other, float Alpha) const;
	UE_API void SetTransform(const FTransform& Transform);
	FTransform GetTransform() const { return FTransform(Facing, Position); }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UE_API FArchive& operator<<(FArchive& Ar, FPoseSearchQueryTrajectorySample& TrajectorySample);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

USTRUCT(BlueprintType, Category="Pose Search Trajectory")
struct UE_DEPRECATED(5.6, "FPoseSearchQueryTrajectory has been deprecated. Use FTransformTrajectory instead.")
FPoseSearchQueryTrajectory
{
	GENERATED_BODY()

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	FPoseSearchQueryTrajectory() = default;
	~FPoseSearchQueryTrajectory() = default;
	FPoseSearchQueryTrajectory(const FPoseSearchQueryTrajectory&) = default;
	FPoseSearchQueryTrajectory(FPoseSearchQueryTrajectory&&) = default;
	FPoseSearchQueryTrajectory& operator=(const FPoseSearchQueryTrajectory&) = default;
	FPoseSearchQueryTrajectory& operator=(FPoseSearchQueryTrajectory&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Implicit conversion constructor in oreder to facilitate deprecations.
	UE_API FPoseSearchQueryTrajectory(const FTransformTrajectory& InTrajectory);
	
	// Implicit conversion operator in order to facilitate deprecations.
	UE_API operator FTransformTrajectory() const;
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// This contains zero or more history samples, a current sample, and zero or more future predicted samples.
	UE_DEPRECATED(5.6, "Use FTransformTrajectory instead. See FTransfromTrajectory::Samples.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use FTransfromTrajectory instead. See FTransfromTrajectory:Samples."))
	TArray<FPoseSearchQueryTrajectorySample> Samples;
	
	UE_API FPoseSearchQueryTrajectorySample GetSampleAtTime(float Time, bool bExtrapolate = false) const;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if ENABLE_ANIM_DEBUG
	UE_API void DebugDrawTrajectory(const UWorld* World, const float DebugThickness = 0.f, float HeightOffset = 0.f) const;
	UE_API void DebugDrawTrajectory(FAnimInstanceProxy& AnimInstanceProxy, const float DebugThickness = 0.f, float HeightOffset = 0.f, int MaxHistorySamples = -1, int MaxPredictionSamples = -1) const;
	UE_API void DebugDrawTrajectory(const UObject* Owner, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, const float DebugThickness = 0.f, float HeightOffset = 0.f) const;
#endif // ENABLE_ANIM_DEBUG
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UE_API FArchive& operator<<(FArchive& Ar, FPoseSearchQueryTrajectory& Trajectory);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
