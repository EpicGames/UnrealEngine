// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectTraceDefines.h"
#if OBJECT_TRACE_ENABLED
#include "ObjectTrace.h"
#endif
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchCustomVersion.h"
#include "PoseSearch/PoseSearchLibrary.h"

#define UE_API POSESEARCH_API

UE_TRACE_CHANNEL_EXTERN(PoseSearchChannel, POSESEARCH_API);

namespace UE::PoseSearch
{

struct UE_DEPRECATED(5.7, "FTraceLogger is no longer necessary and it'll be removed") FTraceLogger
{
	/** Used for reading trace data */
    static UE_API const FName Name;
};

// Message types for appending / reading to the timeline
/** Base message type for common data */
struct FTraceMessage
{
	uint64 Cycle = 0;

	// @todo: rename it to AnimContextId
	uint64 AnimInstanceId = 0;

	// motion matching Search Id associated with this message
	// @todo: rename it to SearchId
	int32 NodeId = InvalidSearchId;
	int32 GetSearchId() const { return NodeId; }
};
POSESEARCH_API FArchive& operator<<(FArchive& Ar, FTraceMessage& State);

struct FTraceMotionMatchingStatePoseEntry
{
	int32 DbPoseIdx = INDEX_NONE;
	EPoseCandidateFlags PoseCandidateFlags = EPoseCandidateFlags::None;
	FPoseSearchCost Cost;

	bool operator==(const FTraceMotionMatchingStatePoseEntry& Other) const { return DbPoseIdx == Other.DbPoseIdx; }
};
POSESEARCH_API FArchive& operator<<(FArchive& Ar, FTraceMotionMatchingStatePoseEntry& Entry);

struct FTraceMotionMatchingStateDatabaseEntry
{
	uint64 DatabaseId = 0;
	TArray<float> QueryVector;
	TArray<FTraceMotionMatchingStatePoseEntry> PoseEntries;
	
	bool operator==(const FTraceMotionMatchingStateDatabaseEntry& Other) const { return DatabaseId == Other.DatabaseId; }
};
POSESEARCH_API FArchive& operator<<(FArchive& Ar, FTraceMotionMatchingStateDatabaseEntry& Entry);

/**
 * Used to trace motion matching state data via the logger, which is then placed into a timeline
 */
struct FTraceMotionMatchingStateMessage : public FTraceMessage
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FTraceMotionMatchingStateMessage() = default;
	FTraceMotionMatchingStateMessage(const FTraceMotionMatchingStateMessage& Other) = default;
	FTraceMotionMatchingStateMessage(FTraceMotionMatchingStateMessage&& Other) = default;
	FTraceMotionMatchingStateMessage& operator=(const FTraceMotionMatchingStateMessage& Other) = default;
	FTraceMotionMatchingStateMessage& operator=(FTraceMotionMatchingStateMessage&& Other) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Amount of time since the last pose switch */
	float ElapsedPoseSearchTime = 0.f;

	float AssetPlayerTime = 0.f;
	float DeltaTime = 0.f;
	float SimLinearVelocity = 0.f;
	float SimAngularVelocity = 0.f;
	float AnimLinearVelocity = 0.f;
	float AnimAngularVelocity = 0.f;
	float Playrate = 0.f;
	float AnimLinearVelocityNoTimescale = 0.f;
	float AnimAngularVelocityNoTimescale = 0.f;
	
	float RecordingTime = 0.f;
	float SearchBestCost = 0.f;
	float SearchBruteForceCost = 0.f;
	int32 SearchBestPosePos = 0;

	TArray<uint64> SkeletalMeshComponentIds;
	
	TArray<FRole> Roles;

	TArray<FTraceMotionMatchingStateDatabaseEntry> DatabaseEntries;

	TArray<UE::PoseSearch::FArchivedPoseHistory> PoseHistories;

	UE_DEPRECATED(5.7, "This API will soon be removed. Look for a FTraceMotionMatchingStateDatabaseEntry::PoseEntries with PoseCandidateFlags containg EPoseCandidateFlags::Valid_ContinuingPose for the 'current database and pose index' instead")
	int32 CurrentDbEntryIdx = INDEX_NONE;

	UE_DEPRECATED(5.7, "This API will soon be removed. Look for a FTraceMotionMatchingStateDatabaseEntry::PoseEntries with PoseCandidateFlags containg EPoseCandidateFlags::Valid_ContinuingPose for the 'current database and pose index' instead")
	int32 CurrentPoseEntryIdx = INDEX_NONE;

	EPoseSearchInterruptMode InterruptMode = EPoseSearchInterruptMode::DoNotInterrupt;
	
	// Input the current state from EventData
	UE_API void Input(FPoseSearchCustomVersion::Type CustomVersionType, TConstArrayView<uint8> Data);
	// Output the current state info to the logger
	UE_API void Output();

	UE_DEPRECATED(5.7, "This API will soon be removed. Look for a FTraceMotionMatchingStateDatabaseEntry::PoseEntries with PoseCandidateFlags containg EPoseCandidateFlags::Valid_ContinuingPose for the 'current database and pose index' instead")
	UE_API const UPoseSearchDatabase* GetCurrentDatabase() const;
	UE_DEPRECATED(5.7, "This API will soon be removed. Look for a FTraceMotionMatchingStateDatabaseEntry::PoseEntries with PoseCandidateFlags containg EPoseCandidateFlags::Valid_ContinuingPose for the 'current database and pose index' instead")
	UE_API int32 GetCurrentDatabasePoseIndex() const;
	UE_DEPRECATED(5.7, "This API will soon be removed. Look for a FTraceMotionMatchingStateDatabaseEntry::PoseEntries with PoseCandidateFlags containg EPoseCandidateFlags::Valid_ContinuingPose for the 'current database and pose index' instead")
	UE_API const FTraceMotionMatchingStatePoseEntry* GetCurrentPoseEntry() const;

	template<typename T>
	UE_DEPRECATED(5.7, "This API will soon be removed. Look for ResolveDatabaseFromId to resolve databases from ids")
	static const T* GetObjectFromId(uint64 ObjectId)
	{
#if OBJECT_TRACE_ENABLED
		if (ObjectId)
		{
			UObject* Object = FObjectTrace::GetObjectFromId(ObjectId);
			if (Object)
			{
				return CastChecked<T>(Object);
			}
		}
#endif

		return nullptr;
	}

	UE_DEPRECATED(5.7, "FTraceMotionMatchingStateMessage::Name is no longer necessary and it'll be removed")
	static UE_API const FName Name;
};

POSESEARCH_API FArchive& operator<<(FArchive& Ar, FTraceMotionMatchingStateMessage& State);

} // namespace UE::PoseSearch

#undef UE_API
