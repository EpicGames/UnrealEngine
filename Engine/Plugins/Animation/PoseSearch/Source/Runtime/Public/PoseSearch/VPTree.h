// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Vantage-point tree implementation https://en.wikipedia.org/wiki/Vantage-point_tree
// it relies on templated methods to specify the
// - data point type (typename T, usually a multidimensional point like TArray<float>) 
// - data source struct (typename TDataSource) that needs to implement
//		struct FDataSource
//		{
//			const T& operator[](int32 Index) const;
//			int32 Num() const;
//			static float GetDistance(const T& A, const T& B);
//		};
//   and can be specialized for runtime calls at FindNeighbors to laverage vectorized code with aliged and padded data
// - the result set struct (typename TVPTreeResultSet), see struct FVPTreeResultSet below as example

#include "Containers/ArrayView.h"
#include "Math/RandomStream.h"
#include "PoseSearch/PoseSearchDefines.h"

namespace UE::PoseSearch
{

#define VALIDATE_FINDNEIGHBORS 0

typedef float VPTreeScalar;

struct FIndexDistance
{
    FIndexDistance(int32 InIndex = INDEX_NONE, VPTreeScalar InDistance = VPTreeScalar(0))
    : Index(InIndex)
    , Distance(InDistance)
    {
    }
        
    bool operator<(const FIndexDistance& Other) const
    {
        return Distance < Other.Distance;   
    }

	bool operator==(const FIndexDistance& Other) const
	{
		return Index == Other.Index && Distance == Other.Distance;
	}

	friend FArchive& operator<<(FArchive& Ar, FIndexDistance& IndexDistance)
	{
		Ar << IndexDistance.Index;
		Ar << IndexDistance.Distance;
		return Ar;
	}

    int32 Index;
    VPTreeScalar Distance;
};

struct FVPTreeNode : public FIndexDistance
{
    int32 LeftIndex = INDEX_NONE;
    int32 RightIndex = INDEX_NONE;

	bool operator==(const FVPTreeNode& Other) const
	{
		return FIndexDistance::operator==(Other) && LeftIndex == Other.LeftIndex && RightIndex == Other.RightIndex;
	}

	friend FArchive& operator<<(FArchive& Ar, FVPTreeNode& Node)
	{
		Ar << static_cast<FIndexDistance&>(Node);
		Ar << Node.LeftIndex;
		Ar << Node.RightIndex;
		return Ar;
	}
};

struct FVPTreeResultSet
{
    FVPTreeResultSet(int32 InNumNeighbors)
        : NumNeighbors(InNumNeighbors)
    {
#if !NO_LOGGING
		if (InNumNeighbors > HeapDefaultMaxSize)
		{
			UE_LOGF(LogPoseSearch, Warning, "FVPTreeResultSet - preallocated Heap data size 'HeapDefaultMaxSize' of %d is less than requested %d num neighbors. Performances will be negatively impacted", HeapDefaultMaxSize, InNumNeighbors);
		}
#endif // !NO_LOGGING
    }

    VPTreeScalar GetWorstDistance() const
    {
        return Heap.HeapTop().Distance;
    }
        
    bool IsFull() const
    {
        return Heap.Num() == NumNeighbors;
    }

    void AddPoint(float Distance, int32 Index
#if VALIDATE_FINDNEIGHBORS
		, int32 BestIndex = INDEX_NONE
#endif // VALIDATE_FINDNEIGHBORS
	)
    {
		if (!IsFull())
		{
			Heap.HeapPush(FIndexDistance(Index, Distance), HeapCompare());

#if VALIDATE_FINDNEIGHBORS
			Heap.VerifyHeap(HeapCompare());
#endif // VALIDATE_FINDNEIGHBORS
		}
        else if (Distance < GetWorstDistance())
        {
			// popping the worst FIndexDistance
			FIndexDistance Removed;
            Heap.HeapPop(Removed, HeapCompare());
			
#if VALIDATE_FINDNEIGHBORS
			if (BestIndex != INDEX_NONE && Removed.Index == BestIndex)
			{
				UE_LOGF(LogPoseSearch, Error, "FVPTreeResultSet::AddPoint - we removed the best node from the ResultSet...");
			}
#endif // VALIDATE_FINDNEIGHBORS

			// and replacing it with FIndexDistance(Index, Distance) 
			Heap.HeapPush(FIndexDistance(Index, Distance), HeapCompare());

#if VALIDATE_FINDNEIGHBORS
			Heap.VerifyHeap(HeapCompare());
#endif // VALIDATE_FINDNEIGHBORS
        }
    }

    TArray<FIndexDistance> GetSortedResults()
    {
		TArray<FIndexDistance> Results;
		Results.SetNumUninitialized(Heap.Num());

		int32 Index = 0;
        while (!Heap.IsEmpty())
        {
            Heap.HeapPop(Results[Index++], HeapCompare());
        }
		return Results;
    }

	TConstArrayView<FIndexDistance> GetUnsortedResults() const
	{
		return Heap;
	}

	bool operator==(const FVPTreeResultSet& Other) const
	{
		return NumNeighbors == Other.NumNeighbors && Heap == Other.Heap;
	}

private:
	struct HeapCompare
	{
		bool operator()(const FIndexDistance& A, const FIndexDistance& B) const
		{
			// using > to create a heap where Heap.HeapTop().Distance is the greater Distance (to behave like the std::priority_queue)
			return A.Distance > B.Distance;
		}
	};

    const int32 NumNeighbors;
	enum { HeapDefaultMaxSize = 256 };
    TArray<FIndexDistance, TInlineAllocator<HeapDefaultMaxSize>> Heap;
};

struct FVPTree
{
    template<typename TDataSource>
    void Construct(const TDataSource& DataSource, FRandomStream& RandStream)
    {
		Reset();

		// initializing the DataSourceIndexMapping, array of indeces pointing to the original TDataSource
		const int32 Count = DataSource.Num();
        TArray<FIndexDistance> DataSourceIndexMapping;
        DataSourceIndexMapping.SetNumUninitialized(Count);
        for (int32 Index = 0; Index < DataSourceIndexMapping.Num(); ++Index)
        {
            DataSourceIndexMapping[Index].Index = Index;
        }
        ConstructRecursive(DataSource, DataSourceIndexMapping, RandStream);
    }

	template<typename TDataSource>
	bool TestConstruct(const TDataSource& DataSource, int32 NodeIndex = 0) const
	{
		TArray<int32> SubTreeNodeIndexesTempBuffer;
		SubTreeNodeIndexesTempBuffer.Reserve(Nodes.Num());
		return TestConstructRecursive(DataSource, NodeIndex, SubTreeNodeIndexesTempBuffer);
	}

    template<typename T, typename TDataSource, typename TVPTreeResultSet>
    void FindNeighbors(const T& Query, TVPTreeResultSet& ResultSet, const TDataSource& DataSource) const
    {
#if VALIDATE_FINDNEIGHBORS
		int32 NumEvaluatedQueryDistances = 0;

		// brute forcing to find the best Node
		VPTreeScalar BestDistance = VPTreeScalar(UE_BIG_NUMBER);
		int32 BestNodeIndex = INDEX_NONE;

		TArray<bool> DoWeHaveAllNodes;
		DoWeHaveAllNodes.SetNumZeroed(Nodes.Num());

		for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); ++NodeIndex)
		{
			check(Nodes[NodeIndex].Index != INDEX_NONE);

			DoWeHaveAllNodes[Nodes[NodeIndex].Index] = true;

			const VPTreeScalar Distance = TDataSource::GetDistance(Query, DataSource[Nodes[NodeIndex].Index]);
			if (Distance < BestDistance)
			{
				// @todo: handle eventual duplicates
				BestDistance = Distance;
				BestNodeIndex = NodeIndex;
			}
		}

		check(!DoWeHaveAllNodes.Contains(false));

		TArray<int32> SubTreeNodeIndexesTempBuffer;
		SubTreeNodeIndexesTempBuffer.Reserve(Nodes.Num());

		bool bBestNodeIndexFound = false;
#endif // VALIDATE_FINDNEIGHBORS

		enum { NodesToSearchDefaultMaxSize = 512 };
		TArray<int32, TInlineAllocator<NodesToSearchDefaultMaxSize>> NodesToSearch;
		NodesToSearch.Emplace(0);

        VPTreeScalar Tau = VPTreeScalar(UE_BIG_NUMBER);
		while (!NodesToSearch.IsEmpty())
		{
			const int32 NodeIndex = NodesToSearch.Pop();
			const FVPTreeNode& Node = Nodes[NodeIndex];

			// query distance from vantage point
			const VPTreeScalar QueryDistance = TDataSource::GetDistance(DataSource[Node.Index], Query);

#if VALIDATE_FINDNEIGHBORS
			++NumEvaluatedQueryDistances;

			bBestNodeIndexFound |= NodeIndex == BestNodeIndex;
			bool bNodeIndexContainsBestNodeIndex = false;
			if (!bBestNodeIndexFound)
			{
				SubTreeNodeIndexesTempBuffer.Reset();
				GetSubTreeNodeIndexes(NodeIndex, SubTreeNodeIndexesTempBuffer);

				bNodeIndexContainsBestNodeIndex = SubTreeNodeIndexesTempBuffer.Contains(BestNodeIndex);

				if (!bNodeIndexContainsBestNodeIndex)
				{
					bool bOtherNodeIndexContainsBestNodeIndex = false;
					for (int32 OtherNodeIndex : NodesToSearch)
					{
						SubTreeNodeIndexesTempBuffer.Reset();
						GetSubTreeNodeIndexes(OtherNodeIndex, SubTreeNodeIndexesTempBuffer);
						bOtherNodeIndexContainsBestNodeIndex = SubTreeNodeIndexesTempBuffer.Contains(BestNodeIndex);
						if (bOtherNodeIndexContainsBestNodeIndex)
						{
							break;
						}
					}

					if (!bOtherNodeIndexContainsBestNodeIndex)
					{
						UE_LOGF(LogPoseSearch, Error, "FVPTree::FindNeighbors - we couldn't find the best node...");
					}
				}
			}
#endif // VALIDATE_FINDNEIGHBORS

			// Tau represents the worst distance of the points in the TVPTreeResultSet
			if (QueryDistance <= Tau)
			{
#if VALIDATE_FINDNEIGHBORS
				if (bBestNodeIndexFound)
				{
					ResultSet.AddPoint(QueryDistance, Node.Index, BestNodeIndex);
				}
				else
#endif // VALIDATE_FINDNEIGHBORS
				{
					ResultSet.AddPoint(QueryDistance, Node.Index);
				}

				if (ResultSet.IsFull())
				{
					Tau = ResultSet.GetWorstDistance();
				}
			}
#if VALIDATE_FINDNEIGHBORS
			else if (NodeIndex == BestNodeIndex)
			{
				UE_LOGF(LogPoseSearch, Error, "FVPTree::FindNeighbors - we had the best node in our hands and let it slipped away...");
			}

			bool bIsBestNodeStillReachable = false;
#endif // VALIDATE_FINDNEIGHBORS
			if (Node.LeftIndex != INDEX_NONE && QueryDistance < Node.Distance + Tau)
			{
				NodesToSearch.Emplace(Node.LeftIndex);
#if !NO_LOGGING
				if (NodesToSearch.Num() > NodesToSearchDefaultMaxSize)
				{
					UE_LOGF(LogPoseSearch, Warning, "FVPTree::FindNeighbors - requested more than preallocated NodesToSearch data size 'NodesToSearchDefaultMaxSize' (%d / %d)", NodesToSearch.Num(), NodesToSearchDefaultMaxSize);
				}
#endif // !NO_LOGGING

#if VALIDATE_FINDNEIGHBORS
				if (bNodeIndexContainsBestNodeIndex)
				{
					SubTreeNodeIndexesTempBuffer.Reset();
					GetSubTreeNodeIndexes(Node.LeftIndex, SubTreeNodeIndexesTempBuffer);
					const bool bLeftIndexContainsBestNodeIndex = SubTreeNodeIndexesTempBuffer.Contains(BestNodeIndex);
					bIsBestNodeStillReachable |= bLeftIndexContainsBestNodeIndex;
				}
#endif // VALIDATE_FINDNEIGHBORS
			}

			if (Node.RightIndex != INDEX_NONE && QueryDistance >= Node.Distance - Tau)
			{
				NodesToSearch.Emplace(Node.RightIndex);
#if !NO_LOGGING
				if (NodesToSearch.Num() > NodesToSearchDefaultMaxSize)
				{
					UE_LOGF(LogPoseSearch, Warning, "FVPTree::FindNeighbors - requested more than preallocated NodesToSearch data size 'NodesToSearchDefaultMaxSize' (%d / %d)", NodesToSearch.Num(), NodesToSearchDefaultMaxSize);
				}
#endif // !NO_LOGGING

#if VALIDATE_FINDNEIGHBORS
				if (bNodeIndexContainsBestNodeIndex)
				{
					SubTreeNodeIndexesTempBuffer.Reset();
					GetSubTreeNodeIndexes(Node.RightIndex, SubTreeNodeIndexesTempBuffer);
					const bool bRightIndexContainsBestNodeIndex = SubTreeNodeIndexesTempBuffer.Contains(BestNodeIndex);
					bIsBestNodeStillReachable |= bRightIndexContainsBestNodeIndex;
				}
#endif // VALIDATE_FINDNEIGHBORS
			}

#if VALIDATE_FINDNEIGHBORS
			if (bNodeIndexContainsBestNodeIndex != bIsBestNodeStillReachable)
			{
				UE_LOGF(LogPoseSearch, Error, "FVPTree::FindNeighbors - we lost strack of the best node...");
			}
#endif // VALIDATE_FINDNEIGHBORS
		}
    }

	void Reset();

	POSESEARCH_API SIZE_T GetAllocatedSize() const;

	bool operator==(const FVPTree& Other) const;

	friend FArchive& operator<<(FArchive& Ar, FVPTree& VPTree);

private:
    void GetSubTreeNodeIndexes(int32 RootNodeIndex, TArray<int32>& NodeIndexes) const
    {
		if (RootNodeIndex != INDEX_NONE)
		{
			NodeIndexes.Add(RootNodeIndex);
			GetSubTreeNodeIndexes(Nodes[RootNodeIndex].LeftIndex, NodeIndexes);
			GetSubTreeNodeIndexes(Nodes[RootNodeIndex].RightIndex, NodeIndexes);
		}
    }

	// choose and swap vantage point into DataSourceIndexMapping[0]
    template<typename TDataSource>
    void ChooseVantagePointMappingIndex(const TDataSource& DataSource, TArrayView<FIndexDistance> DataSourceIndexMapping, FRandomStream& RandStream) const
    {
        // choosing a random, but deterministic, vantage point within DataSourceIndexMapping
		const int32 VantagePointMappingIndex = FMath::FloorToInt32(RandStream.FRand() * DataSourceIndexMapping.Num());
		
		// swapping DataSourceIndexMapping[VantagePointMappingIndex] with DataSourceIndexMapping[0];
		const int32 VantagePointDataIndex = DataSourceIndexMapping[VantagePointMappingIndex].Index;

		DataSourceIndexMapping[VantagePointMappingIndex] = DataSourceIndexMapping[0].Index;
		DataSourceIndexMapping[0].Index = VantagePointDataIndex;
		DataSourceIndexMapping[0].Distance = VPTreeScalar(0);
    }

    template<typename TDataSource>
    void CacheDistancesFromVantagePoint(const TDataSource& DataSource, TArrayView<FIndexDistance> DataSourceIndexMapping) const
    {
		DataSourceIndexMapping[0].Distance = VPTreeScalar(0);
		const int32 VantagePointDataIndex = DataSourceIndexMapping[0].Index;

		ParallelFor(DataSourceIndexMapping.Num() - 1, [&DataSource, &DataSourceIndexMapping, VantagePointDataIndex](int32 ParallelForIndex)
			{
				FIndexDistance& IndexDistance = DataSourceIndexMapping[ParallelForIndex + 1];
				IndexDistance.Distance = TDataSource::GetDistance(DataSource[VantagePointDataIndex], DataSource[IndexDistance.Index]);
			});
    }

	template<typename TDataSource>
    int32 ConstructRecursive(const TDataSource& DataSource, TArrayView<FIndexDistance> DataSourceIndexMapping, FRandomStream& RandStream)
    {
		int32 NodeIndex = INDEX_NONE;
		const int32 Num = DataSourceIndexMapping.Num();
		if (Num > 0)
		{
			NodeIndex = Nodes.Num();
			Nodes.SetNum(NodeIndex + 1);

			if (Num > 1)
			{
				ChooseVantagePointMappingIndex(DataSource, DataSourceIndexMapping, RandStream);
				CacheDistancesFromVantagePoint(DataSource, DataSourceIndexMapping);

				Nodes[NodeIndex].Index = DataSourceIndexMapping[0].Index;

				// @todo: use something like std::nth_element since we don't need to sort the MedianMappingIndex -> end section of the DataSourceIndexMapping array!
				DataSourceIndexMapping.Slice(1, Num - 1).StableSort([](const FIndexDistance& A, const FIndexDistance& B)
					{
						return A.Distance < B.Distance;
					});

				// @todo: maybe use the average delta distance between MedianMappingIndex - 1 and MedianMappingIndex, to round up eventual search numerical errors
				const int32 MedianMappingIndex = Num / 2;
				Nodes[NodeIndex].Distance = DataSourceIndexMapping[MedianMappingIndex - 1].Distance;
				Nodes[NodeIndex].LeftIndex = ConstructRecursive(DataSource, DataSourceIndexMapping.Slice(1, MedianMappingIndex - 1), RandStream);
				Nodes[NodeIndex].RightIndex = ConstructRecursive(DataSource, DataSourceIndexMapping.Slice(MedianMappingIndex, Num - MedianMappingIndex), RandStream);
			}
			else
			{
				Nodes[NodeIndex].Index = DataSourceIndexMapping[0].Index;
			}
		}
        return NodeIndex;
    }

	template<typename TDataSource>
    bool TestConstructRecursive(const TDataSource& DataSource, int32 NodeIndex, TArray<int32>& SubTreeNodeIndexesTempBuffer) const
    {
		if (NodeIndex == INDEX_NONE)
		{
			return true;
		}

		const FVPTreeNode Node = Nodes[NodeIndex];

		// making sure all the left sub tree nodes are within Nodes[NodeIndex].Distance to the vantage point Nodes[NodeIndex].Index
		SubTreeNodeIndexesTempBuffer.Reset();
		GetSubTreeNodeIndexes(Node.LeftIndex, SubTreeNodeIndexesTempBuffer);
		for (int32 SubTreeNodeIndex : SubTreeNodeIndexesTempBuffer)
		{
			const VPTreeScalar Distance = TDataSource::GetDistance(DataSource[Nodes[NodeIndex].Index], DataSource[Nodes[SubTreeNodeIndex].Index]);
			if (Distance > Nodes[NodeIndex].Distance)
			{
				return false;
			}
		}

		// making sure all the right sub tree nodes are outside the hypersphere of radius Nodes[NodeIndex].Distance from the vantage point Nodes[NodeIndex].Index
		SubTreeNodeIndexesTempBuffer.Reset();
		GetSubTreeNodeIndexes(Node.RightIndex, SubTreeNodeIndexesTempBuffer);
		for (int32 SubTreeNodeIndex : SubTreeNodeIndexesTempBuffer)
		{
			const VPTreeScalar Distance = TDataSource::GetDistance(DataSource[Nodes[NodeIndex].Index], DataSource[Nodes[SubTreeNodeIndex].Index]);
			if (Distance <= Nodes[NodeIndex].Distance)
			{
				return false;
			}
		}

		// test recursively left and right sub trees
		if (!TestConstructRecursive(DataSource, Node.LeftIndex, SubTreeNodeIndexesTempBuffer))
		{
			return false;
		}

		return TestConstructRecursive(DataSource, Node.RightIndex, SubTreeNodeIndexesTempBuffer);
    }

	TArray<FVPTreeNode> Nodes;
};

} // namespace UE::PoseSearch